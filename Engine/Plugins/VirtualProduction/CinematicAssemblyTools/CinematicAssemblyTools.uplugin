{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Cinematic Assembly Tools (CAT)",
	"Description": "CAT is a suite of cinematic pipeline tools for shot management and linear content creation",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Hidden": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "CineAssemblyTools",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CineAssemblyToolsEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "LevelSequenceEditor",
			"Enabled": true
		},
		{
			"Name": "DirectoryPlaceholder",
			"Enabled": true
		},
		{
			"Name": "MovieRenderPipeline",
			"Enabled": true
		},
		{
			"Name": "NamingTokens",
			"Enabled": true
		},
		{
			"Name": "Takes",
			"Enabled": true
		}
	]
}