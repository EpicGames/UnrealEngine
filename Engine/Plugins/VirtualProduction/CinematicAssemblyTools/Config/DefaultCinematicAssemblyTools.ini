[CoreRedirects]

+ClassRedirects=(OldName="/Script/ShotsEditor.CineAssembly", NewName="/Script/CineAssemblyTools.CineAssembly")
+ClassRedirects=(OldName="/Script/ShotsEditor.CineAssemblySchema", NewName="/Script/CineAssemblyTools.CineAssemblySchema")