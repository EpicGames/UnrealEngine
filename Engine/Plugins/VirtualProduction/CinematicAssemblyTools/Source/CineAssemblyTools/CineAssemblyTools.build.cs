// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class CineAssemblyTools : ModuleRules
	{
		public CineAssemblyTools(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"LevelSequence",
					"NamingTokens",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Json",
					"JsonUtilities",
					"MovieScene",
					"MovieSceneTracks",
					"SlateCore",
					"UniversalObjectLocator"
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("MovieSceneTools");
			}
		}
	}
}
