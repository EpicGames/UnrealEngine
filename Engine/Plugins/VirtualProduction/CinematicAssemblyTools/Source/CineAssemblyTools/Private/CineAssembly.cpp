// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssembly.h"

#include "Algo/Contains.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CineAssemblyNamingTokens.h"
#include "HAL/FileManager.h"
#include "JsonObjectConverter.h"
#include "LevelSequenceShotMetaDataLibrary.h"
#include "MovieScene.h"
#include "MovieSceneFolder.h"
#include "MovieSceneSubAssemblySection.h"
#include "MovieSceneSubAssemblyTrack.h"
#include "NamingTokensEngineSubsystem.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "UniversalObjectLocator.h"
#include "UObject/AssetRegistryTagsContext.h"

#if WITH_EDITOR
#include "AssetToolsModule.h"
#include "MovieSceneMetaData.h"
#include "MovieSceneToolsProjectSettings.h"
#endif

#define LOCTEXT_NAMESPACE "CineAssembly"

DEFINE_LOG_CATEGORY_STATIC(LogCineAssembly, Log, All)

const FName UCineAssembly::AssetRegistryTag_AssemblyType = "AssemblyType";
const FName UCineAssembly::AssetRegistryTag_SchemaGuid = "CineAssemblySchemaID";
const FName UCineAssembly::AssemblyGuidPropertyName = GET_MEMBER_NAME_CHECKED(UCineAssembly, AssemblyGuid);

FOnCineAssemblyMetadataChanged& UCineAssembly::OnAssemblyMetadataChanged()
{
	static FOnCineAssemblyMetadataChanged Instance;
	return Instance;
}

namespace UE::CineAssembly
{
	int32 GetLocalTimezone()
	{
		const FTimespan DifferentLocalToUTC = FDateTime::Now() - FDateTime::UtcNow();
		const int32 DifferenceMinutes = FMath::RoundToInt(DifferentLocalToUTC.GetTotalMinutes());

		const int32 Hours = DifferenceMinutes / 60;
		const int32 Minutes = DifferenceMinutes % 60;

		const int32 Timezone = (Hours * 100) + Minutes;
		return Timezone;
	}

	// Taken from FDateTimeStructCustomization
	int32 ConvertShortTimezone(int32 ShortTimezone)
	{
		// Convert timezones from short-format into long format, -5 -> -0500
		// Timezone Hour ranges go from -12 to +14 from UTC
		if (ShortTimezone >= -12 && ShortTimezone <= 14)
		{
			return ShortTimezone * 100;
		}

		// Not a short-form timezone
		return ShortTimezone;
	}

	// Taken from FDateTimeStructCustomization
	FDateTime ConvertTime(const FDateTime& InDate, int32 InTimezone, int32 OutTimezone)
	{
		if (InTimezone == OutTimezone)
		{
			return InDate;
		}

		// Timezone Hour ranges go from -12 to +14 from UTC
		// Convert from whole-hour to the full-format HHMM (-5 -> -0500, 0 -> +0000, etc)
		InTimezone = ConvertShortTimezone(InTimezone);
		OutTimezone = ConvertShortTimezone(OutTimezone);

		// Extract timezone minutes
		const int32 InTimezoneMinutes = (FMath::Abs(InTimezone) % 100);
		const int32 OutTimezoneMinutes = (FMath::Abs(OutTimezone) % 100);

		// Calculate our Minutes difference
		const int32 MinutesDifference = OutTimezoneMinutes - InTimezoneMinutes;

		// Calculate our Hours difference
		const int32 HoursDifference = (OutTimezone / 100) - (InTimezone / 100);

		return FDateTime(InDate + FTimespan(HoursDifference, MinutesDifference, 0));
	}
} // namespace UE::CineAssembly

UCineAssembly::UCineAssembly()
{
	MetadataJsonObject = MakeShared<FJsonObject>();
}

void UCineAssembly::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject | RF_NeedLoad | RF_WasLoaded) && !AssemblyGuid.IsValid())
	{
		AssemblyGuid = FGuid::NewGuid();
	}
}

void UCineAssembly::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (!bDuplicateForPIE)
	{
		AssemblyGuid = FGuid::NewGuid();
	}
}

void UCineAssembly::PostLoad()
{
	Super::PostLoad();

	if (!AssemblyGuid.IsValid())
	{
		AssemblyGuid = FGuid::NewGuid();
	}
}

bool UCineAssembly::IsCompatibleAsSubSequence(const UMovieSceneSequence& ParentSequence) const
{
	return !bIsDataOnly;
}

#if WITH_EDITOR
ETrackSupport UCineAssembly::IsTrackSupportedImpl(TSubclassOf<class UMovieSceneTrack> InTrackClass) const
{
	// If the UObject that owns this assembly is a Schema (rather than a UPackage, for example), then we know that this Assembly is actually a Schema Template that is only ever edited in the Schema Sequencer.
	if (GetTypedOuter<UCineAssemblySchema>())
	{
		// Event tracks are not supported in the Schema Sequencer
		// Subsequence Tracks and Cinematic Shot Tracks are not supported in the Schema Sequencer, because the custom SubAssembly Track is used instead
		if ((InTrackClass == UMovieSceneEventTrack::StaticClass()) || (InTrackClass == UMovieSceneSubTrack::StaticClass()) || (InTrackClass == UMovieSceneCinematicShotTrack::StaticClass()))
		{
			return ETrackSupport::NotSupported;
		}
	}

	return Super::IsTrackSupportedImpl(InTrackClass);
}
#endif //WITH_EDITOR

void UCineAssembly::Initialize()
{
	Super::Initialize();

#if WITH_EDITOR
	const UMovieSceneToolsProjectSettings* MovieSceneToolsProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();

	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameNumber DefaultStartFrame = (MovieSceneToolsProjectSettings->DefaultStartTime * TickResolution).FloorToFrame();
	const int32 DefaultDuration = (MovieSceneToolsProjectSettings->DefaultDuration * TickResolution).FloorToFrame().Value;

	MovieScene->SetPlaybackRange(DefaultStartFrame, DefaultDuration);
#endif // WITH_EDITOR
}

void UCineAssembly::InitializeFromSchema(UCineAssemblySchema* InSchema, bool bDuplicateMovieScene)
{
	TSet<const UCineAssemblySchema*> SchemaAncestors;
	InitializeFromSchemaInternal(InSchema, SchemaAncestors, bDuplicateMovieScene);
}

void UCineAssembly::InitializeFromSchemaInternal(UCineAssemblySchema* InSchema, TSet<const UCineAssemblySchema*>& SchemaAncestors, bool bDuplicateMovieScene)
{
	ChangeSchema(InSchema);

	if (InSchema)
	{
		SchemaAncestors.Add(InSchema);
		InitializeFromTemplate(InSchema->TemplateSequence, bDuplicateMovieScene);
		ConvertSubAssemblyTracksInternal(SchemaAncestors);
		SchemaAncestors.Remove(InSchema);
	}
}

void UCineAssembly::InitializeFromTemplate(ULevelSequence* Template, bool bDuplicateMovieScene)
{
	if (!MovieScene)
	{
		Initialize();
	}

	if (Template == nullptr)
	{
		return;
	}

	if (UCineAssembly* TemplateAssembly = Cast<UCineAssembly>(Template))
	{
		PathRelativeToRoot = TemplateAssembly->PathRelativeToRoot;
		DefaultFolderNames = TemplateAssembly->DefaultFolderNames;
		AssociatedAssets = TemplateAssembly->AssociatedAssets;
		MetadataLinks = TemplateAssembly->MetadataLinks;
	}

	if (!bDuplicateMovieScene)
	{
		return;
	}

	if (UMovieScene* TemplateMovieScene = Template->GetMovieScene())
	{
		// Attempt to duplicate the inner MovieScene. Assign only if duplication is a success to avoid leaving the assembly in a partially valid state.
		// Duplicating the MovieScene will null any event track bindings which is the currently supported behaviour.
		UMovieScene* DuplicateMovieScene = DuplicateObject(TemplateMovieScene, this);

		if (DuplicateMovieScene == nullptr)
		{
			return;
		}

		// Cache the display rate of the original initialized movie scene, which will be restored after the movie scene is replaced with the one from the template
		const FFrameRate OriginalDisplayRate = MovieScene->GetDisplayRate();

		MovieScene = DuplicateMovieScene;

		MovieScene->SetDisplayRate(OriginalDisplayRate);

		// Fix up supported bindings
		if (const FMovieSceneBindingReferences* TemplateBindingReferences = Template->GetBindingReferences())
		{
			FMovieSceneBindingReferences& ThisBindingReferencesRef = BindingReferences;

			for (const FMovieSceneBindingReference& Reference : TemplateBindingReferences->GetAllReferences())
			{
				FUniversalObjectLocator NewLocator = Reference.Locator;

				UMovieSceneCustomBinding* CustomBindingDuplicate = Reference.CustomBinding != nullptr 
					? Cast<UMovieSceneCustomBinding>(StaticDuplicateObject(Reference.CustomBinding, MovieScene))
					: nullptr;
				
				ThisBindingReferencesRef.AddBinding(Reference.ID, MoveTemp(NewLocator), Reference.ResolveFlags, CustomBindingDuplicate);
			}
		}
	}
}

FGuid UCineAssembly::GetAssemblyGuid() const
{
	return AssemblyGuid;
}

const UCineAssemblySchema* UCineAssembly::GetSchema() const
{
	return BaseSchema.Get();
}

void UCineAssembly::SetSchema(UCineAssemblySchema* InSchema)
{
	if (BaseSchema == nullptr)
	{
		BaseSchema = InSchema;
		ChangeSchema(InSchema);
	}
}

void UCineAssembly::ChangeSchema(UCineAssemblySchema* InSchema)
{
	// Remove all metadata associated with the old schema before changing it
	if (BaseSchema)
	{
		for (const FAssemblyMetadataDesc& MetadataDesc : BaseSchema->AssemblyMetadata)
		{
			MetadataJsonObject->RemoveField(MetadataDesc.Key);
		}
	}

	BaseSchema = InSchema;

	// Reset the assembly's name based on the schema template
	if (BaseSchema)
	{
		AssemblyName.Template = BaseSchema->GetDefaultAssemblyName();
	}
	else
	{
		AssemblyName.Template = TEXT("");
	}

	// Add all metadata associated with the new schema (initialized to the default values for each field)
	if (BaseSchema)
	{
		for (const FAssemblyMetadataDesc& MetadataDesc : BaseSchema->AssemblyMetadata)
		{
			if (MetadataDesc.DefaultValue.IsType<FString>())
			{
				if (MetadataDesc.bEvaluateTokens)
				{
					FTemplateString DefaultTemplateString;
					DefaultTemplateString.Template = MetadataDesc.DefaultValue.Get<FString>();
					DefaultTemplateString.Resolved = UCineAssemblyNamingTokens::GetResolvedText(DefaultTemplateString.Template, this);
					SetMetadataAsTokenString(MetadataDesc.Key, DefaultTemplateString);
				}
				else
				{
					const FString& DefaultValue = MetadataDesc.DefaultValue.Get<FString>();
					SetMetadataAsString(MetadataDesc.Key, DefaultValue);
				}
			}
			else if (MetadataDesc.DefaultValue.IsType<bool>())
			{
				const bool& DefaultValue = MetadataDesc.DefaultValue.Get<bool>();
				SetMetadataAsBool(MetadataDesc.Key, DefaultValue);
			}
			else if (MetadataDesc.DefaultValue.IsType<int32>())
			{
				const int32& DefaultValue = MetadataDesc.DefaultValue.Get<int32>();
				SetMetadataAsInteger(MetadataDesc.Key, DefaultValue);
			}
			else if (MetadataDesc.DefaultValue.IsType<float>())
			{
				const float& DefaultValue = MetadataDesc.DefaultValue.Get<float>();
				SetMetadataAsFloat(MetadataDesc.Key, DefaultValue);
			}
		}
	}

	// Reset the list of folders names to create from the Schema
	PathRelativeToRoot.Template.Reset();
	PathRelativeToRoot.Resolved = FText::GetEmpty();
	DefaultFolderNames.Reset();

	SubAssemblies.Empty();

	if (BaseSchema)
	{
		// Update the Data Only flag on the assembly
		bIsDataOnly = BaseSchema->bIsDataOnly;

		// Apply the schema's default Level if one is specified
		if (BaseSchema->bOverrideDefaultLevel)
		{
			Level = BaseSchema->DefaultLevel;
		}
	}
}

#if WITH_EDITOR
void UCineAssembly::CreateSubAssemblies()
{
	// Deprecated
}
void UCineAssembly::CreateSubFolders()
{
	// Deprecated
}
#endif // WITH_EDITOR

UCineAssembly* UCineAssembly::CreateSubAssemblyFromTemplate(UObject* TemplateObject, TSet<const UCineAssemblySchema*>& SchemaAncestors)
{
	// The new SubAssembly begins as a transient object so that it can be configured before it is named and given a package.
	// This allows us to set its metadata first before trying to resolve any tokens which might appear in the SubAssemblyName.
	UCineAssembly* SubAssembly = nullptr;

	if (UCineAssemblySchema* SchemaTemplate = Cast<UCineAssemblySchema>(TemplateObject))
	{
		// Early-out if a cycle is detected to prevent infinite recursion
		if (SchemaAncestors.Contains(SchemaTemplate))
		{
			UE_LOGF(LogCineAssembly, Error, "Skipped SubAssembly from Schema '%ls' because the Schema is already an ancestor in the current assembly creation call stack (this would cause infinite recursion)", *SchemaTemplate->GetPathName());
			return nullptr;
		}

		SubAssembly = NewObject<UCineAssembly>(GetTransientPackage());
		SubAssembly->InitializeFromSchemaInternal(SchemaTemplate, SchemaAncestors);
	}
	else if (UCineAssembly* AssemblyTemplate = Cast<UCineAssembly>(TemplateObject))
	{
		SubAssembly = Cast<UCineAssembly>(StaticDuplicateObject(AssemblyTemplate, GetTransientPackage(), NAME_None, RF_Transient));

		if (!SubAssembly)
		{
			UE_LOGF(LogCineAssembly, Error, "Failed to duplicate the UCineAssembly template '%ls'", *AssemblyTemplate->GetPathName());
			return nullptr;
		}

		SubAssembly->SourceAssemblyPath = FPaths::GetPath(AssemblyTemplate->GetPathName());
		SubAssembly->DuplicateManagedSubAssemblies();
	}
	else if (ULevelSequence* SequenceTemplate = Cast<ULevelSequence>(TemplateObject))
	{
		SubAssembly = NewObject<UCineAssembly>(GetTransientPackage());
		SubAssembly->InitializeFromTemplate(SequenceTemplate);

		SubAssembly->SourceAssemblyPath = FPaths::GetPath(SequenceTemplate->GetPathName());
	}
	else
	{
		// No template is specified, so create a new blank Assembly
		SubAssembly = NewObject<UCineAssembly>(GetTransientPackage());
		SubAssembly->Initialize();
	}

	SubAssembly->ParentAssembly = this;
	return SubAssembly;
}

void UCineAssembly::RemoveAssociatedAsset(const FGuid& InAssetID)
{
	Modify();

	// Remove the input asset from the associate asset list
	AssociatedAssets.RemoveAll([&InAssetID](const FAssemblyAssociatedAssetDesc& AssetDesc) { return AssetDesc.AssetID == InAssetID; });

	// Remove any metadata links that link to the removed asset
	for (auto It = MetadataLinks.CreateIterator(); It; ++It)
	{
		if (It->Value == InAssetID)
		{
			It.RemoveCurrent();
		}
	}
}

UCineAssembly* UCineAssembly::FindSubAssembly(const FGuid& InAssemblyID) const
{
	for (UMovieSceneSubSection* SubSection : SubAssemblies)
	{
		if (UCineAssembly* SubAssembly = Cast<UCineAssembly>(SubSection ? SubSection->GetSequence() : nullptr))
		{
			if (SubAssembly->GetAssemblyGuid() == InAssemblyID)
			{
				return SubAssembly;
			}
		}
	}
	return nullptr;
}

void UCineAssembly::ValidateMetadataLinks()
{
	if (MetadataLinks.IsEmpty())
	{
		return;
	}

	const UCineAssemblySchema* Schema = BaseSchema.Get() ? BaseSchema.Get() : GetTypedOuter<UCineAssemblySchema>();
	if (!Schema)
	{
		return;
	}

	// Gather all valid GUIDs from associated assets
	TSet<FGuid> ValidGUIDs;
	for (const FAssemblyAssociatedAssetDesc& AssetDesc : AssociatedAssets)
	{
		ValidGUIDs.Add(AssetDesc.AssetID);
	}

	// Gather valid GUIDs from SubAssembly sections in the MovieScene
	if (UMovieScene* InMovieScene = GetMovieScene())
	{
		for (UMovieSceneSection* Section : InMovieScene->GetAllSections())
		{
			if (UMovieSceneSubAssemblySection* SubAssemblySection = Cast<UMovieSceneSubAssemblySection>(Section))
			{
				ValidGUIDs.Add(SubAssemblySection->GetSectionID());
			}
		}
	}

	// Gather valid GUIDs from managed SubAssemblies
	for (UMovieSceneSubSection* SubSection : SubAssemblies)
	{
		if (UCineAssembly* SubAssembly = Cast<UCineAssembly>(SubSection ? SubSection->GetSequence() : nullptr))
		{
			ValidGUIDs.Add(SubAssembly->GetAssemblyGuid());
		}
	}

	// Gather valid metadata keys from the schema
	TSet<FString> ValidKeys;
	Schema->ForEachMetadataDesc([&ValidKeys](const FAssemblyMetadataDesc& MetadataDesc)
	{
		ValidKeys.Add(MetadataDesc.Key);
		return true;
	});

	// Remove entries with stale keys or stale GUIDs
	for (auto It = MetadataLinks.CreateIterator(); It; ++It)
	{
		if (!ValidKeys.Contains(It->Key) || !ValidGUIDs.Contains(It->Value))
		{
			Modify();
			It.RemoveCurrent();
		}
	}
}

void UCineAssembly::DuplicateManagedSubAssemblies()
{
	if (!MovieScene)
	{
		return;
	}

#if WITH_EDITOR
	const bool bIsReadOnly = MovieScene->IsReadOnly();
	MovieScene->SetReadOnly(false);
#endif // WITH_EDITOR

	for (TObjectPtr<UMovieSceneSubSection>& SubSection : SubAssemblies)
	{
		if (!SubSection)
		{
			continue;
		}

		UCineAssembly* OriginalSubAssembly = Cast<UCineAssembly>(SubSection->GetSequence());
		if (!OriginalSubAssembly)
		{
			continue;
		}

		UCineAssembly* DuplicateSubAssembly = Cast<UCineAssembly>(StaticDuplicateObject(OriginalSubAssembly, GetTransientPackage(), NAME_None, RF_Transient));
		if (DuplicateSubAssembly)
		{
			DuplicateSubAssembly->ParentAssembly = this;
			DuplicateSubAssembly->SourceAssemblyPath = FPaths::GetPath(OriginalSubAssembly->GetPathName());
			DuplicateSubAssembly->SourceAssembly = OriginalSubAssembly;

			const bool bIsSectionLocked = SubSection->IsLocked();
			SubSection->SetIsLocked(false);
			SubSection->SetSequence(DuplicateSubAssembly);
			SubSection->SetIsLocked(bIsSectionLocked);

			DuplicateSubAssembly->DuplicateManagedSubAssemblies();
		}
	}

#if WITH_EDITOR
	MovieScene->SetReadOnly(bIsReadOnly);
#endif // WITH_EDITOR
}

void UCineAssembly::ConvertSubAssemblyTracks()
{
	TSet<const UCineAssemblySchema*> SchemaAncestors;
	if (BaseSchema)
	{
		SchemaAncestors.Add(BaseSchema);
	}
	ConvertSubAssemblyTracksInternal(SchemaAncestors);
	RenameSubAssemblies();
}

void UCineAssembly::ConvertSubAssemblyTracksInternal(TSet<const UCineAssemblySchema*>& SchemaAncestors)
{
	// Replace any SubAssemblyTracks we find with either a MovieSceneSubTrack or MovieSceneCinematicShotTrack, depending on the TrackType
	while (UMovieSceneSubAssemblyTrack* SubAssemblyTrack = MovieScene->FindTrack<UMovieSceneSubAssemblyTrack>())
	{
		// Add a new track to the MovieScene. The SubAssemblyTrack will be removed from the MovieScene at the bottom of this loop.
		UClass* TrackClass = SubAssemblyTrack->IsSubsequenceTrack() ? UMovieSceneSubTrack::StaticClass() : UMovieSceneCinematicShotTrack::StaticClass();
		UMovieSceneNameableTrack* NewTrack = Cast<UMovieSceneNameableTrack>(MovieScene->AddTrack(TrackClass));

#if WITH_EDITOR
		NewTrack->SetDisplayName(SubAssemblyTrack->GetDisplayName());
#endif // WITH_EDITOR

		for (UMovieSceneSection* Section : SubAssemblyTrack->GetAllSections())
		{
			if (UMovieSceneSubAssemblySection* SubAssemblySection = Cast<UMovieSceneSubAssemblySection>(Section))
			{
				// Create a new section, with the relevant properties copied from the SubAssemblySection, and add it to the new track
				UClass* SectionClass = SubAssemblyTrack->IsSubsequenceTrack() ? UMovieSceneSubSection::StaticClass() : UMovieSceneCinematicShotSection::StaticClass();
				UMovieSceneSubSection* NewSubSection = NewObject<UMovieSceneSubSection>(NewTrack, SectionClass, NAME_None, SubAssemblySection->GetFlags(), SubAssemblySection);

				NewTrack->AddSection(*NewSubSection);

				// Remove the tint from the section that was used in the Schema editor to differentiate between template and reference sections
				if (NewSubSection->GetColorTint() == UMovieSceneSubAssemblySection::TemplateSectionColorTint)
				{
					NewSubSection->SetColorTint(UMovieSceneSubAssemblySection::ReferenceSectionColorTint);
				}

				if (SubAssemblySection->IsTemplateSection())
				{
					UCineAssembly* NewSubAssembly = CreateSubAssemblyFromTemplate(SubAssemblySection->GetAssemblyTemplate(), SchemaAncestors);
					if (!NewSubAssembly)
					{
						continue;
					}

					// Set the name and path of the new SubAssembly from the data stored in the SubAssembly section
					NewSubAssembly->AssemblyName.Template = SubAssemblySection->GetSequenceName().ToString();
					NewSubAssembly->PathRelativeToParent.Template = SubAssemblySection->GetSequencePath();

					// Apply section metadata overrides before name token resolution so tokens like {cat:shotNum} resolve correctly
					if (!SubAssemblySection->MetadataOverrides.IsEmpty())
					{
						NewSubAssembly->ApplyMetadata(SubAssemblySection->MetadataOverrides);
					}

					// Carry the semantic label from the section to the SubAssembly
					NewSubAssembly->Label = SubAssemblySection->Label;

					// Track non-empty relative paths as additional default folders
					const FString& NewPath = NewSubAssembly->PathRelativeToParent.Template;
					if (!NewPath.IsEmpty() && !Algo::ContainsBy(DefaultFolderNames, NewPath, &FTemplateString::Template))
					{
						DefaultFolderNames.Add(NewSubAssembly->PathRelativeToParent);
					}

					NewSubSection->SetSequence(NewSubAssembly);
					SubAssemblies.Add(NewSubSection);

					// Remap any MetadataLinks entries that reference this section's SectionID to the new SubAssembly's AssemblyGuid
					for (TPair<FString, FGuid>& MetadataLink : MetadataLinks)
					{
						if (MetadataLink.Value == SubAssemblySection->GetSectionID())
						{
							MetadataLink.Value = NewSubAssembly->GetAssemblyGuid();
						}
					}
				}
			}
		}

		// Now remove the SubAssemblyTrack from the MovieScene. 
		MovieScene->RemoveTrack(*SubAssemblyTrack);
	}
}

void UCineAssembly::RenameSubAssemblies()
{
	for (UMovieSceneSubSection* SubSection : SubAssemblies)
	{
		UCineAssembly* SubAssembly = SubSection ? Cast<UCineAssembly>(SubSection->GetSequence()) : nullptr;
		if (!SubAssembly || SubAssembly->Label.IsNone())
		{
			continue;
		}

		FString DesiredName = SubAssembly->Label.ToString();

		// Only prefix the SubAssembly name if the parent has a meaningful semantic name
		const bool bUseParentPrefix = !Label.IsNone();
		if (bUseParentPrefix)
		{
			DesiredName = this->GetName() + TEXT("_") + DesiredName;
		}

		FName TargetName(*DesiredName);
		if (StaticFindObject(nullptr, GetTransientPackage(), *DesiredName))
		{
			TargetName = MakeUniqueObjectName(GetTransientPackage(), UCineAssembly::StaticClass(), TargetName);
		}

		SubAssembly->Rename(*TargetName.ToString(), GetTransientPackage(), REN_DontCreateRedirectors | REN_DoNotDirty | REN_NonTransactional);

		// Recursively rename any SubAssemblies of this SubAssembly
		SubAssembly->RenameSubAssemblies();
	}
}

#if WITH_EDITOR
void UCineAssembly::ResolveMovieSceneTokens()
{
	// Recursively gather all of the folders in the movie scene
	TArray<UMovieSceneFolder*> AllFolders;
	for (UMovieSceneFolder* RootFolder : MovieScene->GetRootFolders())
	{
		AllFolders.Add(RootFolder);
		GetAllMovieSceneFoldersRecursive(RootFolder, AllFolders);
	}

	// Resolve the tokens in each folder name
	for (UMovieSceneFolder* Folder : AllFolders)
	{
		const FText ResolvedFolderName = UCineAssemblyNamingTokens::GetResolvedText(Folder->GetFolderName().ToString(), this);
		Folder->SetFolderName(*ResolvedFolderName.ToString());
	}

	// Resolve the tokens in each track name
	for (UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		if (UMovieSceneNameableTrack* NameableTrack = Cast<UMovieSceneNameableTrack>(Track))
		{
			const FText ResolvedTrackName = UCineAssemblyNamingTokens::GetResolvedText(NameableTrack->GetDisplayName().ToString(), this);
			NameableTrack->SetDisplayName(ResolvedTrackName);
		}
	}

	const TArray<FMovieSceneBinding>& Bindings = Cast<const UMovieScene>(MovieScene)->GetBindings();
	for (const FMovieSceneBinding& Binding : Bindings)
	{
		TArray<UMovieSceneTrack*> Tracks = MovieScene->FindTracks(UMovieSceneNameableTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None);
		for (UMovieSceneTrack* Track : Tracks)
		{
			if (UMovieSceneNameableTrack* NameableTrack = Cast<UMovieSceneNameableTrack>(Track))
			{
				const FText ResolvedTrackName = UCineAssemblyNamingTokens::GetResolvedText(NameableTrack->GetDisplayName().ToString(), this);
				NameableTrack->SetDisplayName(ResolvedTrackName);
			}
		}
	}
}

void UCineAssembly::GetAllMovieSceneFoldersRecursive(UMovieSceneFolder* RootFolder, TArray<UMovieSceneFolder*>& OutFolders)
{
	for (UMovieSceneFolder* ChildFolder : RootFolder->GetChildFolders())
	{
		OutFolders.Add(ChildFolder);
		GetAllMovieSceneFoldersRecursive(ChildFolder, OutFolders);
	}
}
#endif // WITH_EDITOR

void UCineAssembly::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	const FString AssemblyType = BaseSchema ? BaseSchema->SchemaName : TEXT("");
	Context.AddTag(FAssetRegistryTag(AssetRegistryTag_AssemblyType, AssemblyType, FAssetRegistryTag::ETagType::TT_Alphabetical, FAssetRegistryTag::TD_None));

	const FGuid SchemaID = BaseSchema ? BaseSchema->GetSchemaGuid() : FGuid();
	Context.AddTag(FAssetRegistryTag(AssetRegistryTag_SchemaGuid, SchemaID.ToString(), FAssetRegistryTag::TT_Hidden));

	// Add tags associated with the assembly metadata
	for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : MetadataJsonObject->Values)
	{
		if (!Pair.Key.IsEmpty())
		{
			FString ValueString;
			if (MetadataJsonObject->TryGetStringField(TStringView<TCHAR>(Pair.Key), ValueString))
			{
				Context.AddTag(FAssetRegistryTag(*Pair.Key, ValueString, FAssetRegistryTag::ETagType::TT_Alphabetical, FAssetRegistryTag::TD_None));
			}
		}
	}
}

#if WITH_EDITOR

void UCineAssembly::GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	Super::GetAssetRegistryTagMetadata(OutMetadata);

	OutMetadata.Add(AssetRegistryTag_AssemblyType, FAssetRegistryTagMetadata()
		.SetDisplayName(LOCTEXT("AssemblyType_Label", "AssemblyType"))
		.SetTooltip(LOCTEXT("AssemblyType_Tooltip", "The assembly type of this instance"))
	);
}

void UCineAssembly::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UCineAssembly, InstanceMetadata))
	{
		UpdateInstanceMetadata();
	}
}

#endif // WITH_EDITOR

void UCineAssembly::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	FString JsonString;

	if (Ar.IsSaving())
 	{
 		TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&JsonString);
 		FJsonSerializer::Serialize(MetadataJsonObject.ToSharedRef(), JsonWriter);
 	}

	Ar << JsonString;

	if (Ar.IsLoading())
	{
		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(JsonString);
		FJsonSerializer::Deserialize(JsonReader, MetadataJsonObject);

		// After the JsonObject has been loaded, add a naming token for each of its keys
		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : MetadataJsonObject->Values)
		{
			AddMetadataNamingToken(FString(Pair.Key));
		}
	}
}

TSoftObjectPtr<UWorld> UCineAssembly::GetLevel()
{
	if (Level.IsValid())
	{
		return TSoftObjectPtr<UWorld>(Level);
	}
	return nullptr;
}

void UCineAssembly::SetLevel(TSoftObjectPtr<UWorld> InLevel)
{
	Level = InLevel.ToSoftObjectPath();
}

FString UCineAssembly::GetNoteText()
{
	return AssemblyNote;
}

void UCineAssembly::SetNoteText(FString InNote)
{
	AssemblyNote = InNote;
}

void UCineAssembly::AppendToNoteText(FString InNote)
{
	AssemblyNote.Append(TEXT("\n"));
	AssemblyNote.Append(InNote);
}

FGuid UCineAssembly::GetProductionID()
{
	return Production;
}

FString UCineAssembly::GetProductionName()
{
	return ProductionName;
}

TSoftObjectPtr<UCineAssembly> UCineAssembly::GetParentAssembly()
{
	if (ParentAssembly.IsValid())
	{
		return TSoftObjectPtr<UCineAssembly>(ParentAssembly);
	}
	return nullptr;
}

void UCineAssembly::SetParentAssembly(TSoftObjectPtr<UCineAssembly> InParent)
{
	ParentAssembly = InParent.ToSoftObjectPath();
}

FName UCineAssembly::GetLabel() const
{
	return Label;
}

void UCineAssembly::SetLabel(FName InLabel)
{
	Modify();
	Label = InLabel;
}

TArray<UCineAssembly*> UCineAssembly::GetSubAssemblies() const
{
	TArray<UCineAssembly*> Result;
	Result.Reserve(SubAssemblies.Num());

	for (const TObjectPtr<UMovieSceneSubSection>& SubSection : SubAssemblies)
	{
		if (UCineAssembly* SubAssembly = Cast<UCineAssembly>(SubSection ? SubSection->GetSequence() : nullptr))
		{
			Result.Add(SubAssembly);
		}
	}

	return Result;
}

TArray<TSoftObjectPtr<UObject>> UCineAssembly::GetAssociatedAssets() const
{
	TArray<TSoftObjectPtr<UObject>> Result;
	Result.Reserve(AssociatedAssets.Num());

	for (const FAssemblyAssociatedAssetDesc& Desc : AssociatedAssets)
	{
		if (!Desc.CreatedAsset.IsNull())
		{
			Result.Add(Desc.CreatedAsset);
		}
	}

	return Result;
}

TArray<UCineAssembly*> UCineAssembly::FindSubAssembliesByLabel(FName InLabel) const
{
	TArray<UCineAssembly*> Result;
	for (UCineAssembly* SubAssembly : GetSubAssemblies())
	{
		if (SubAssembly && SubAssembly->GetLabel() == InLabel)
		{
			Result.Add(SubAssembly);
		}
	}
	return Result;
}

TArray<TSoftObjectPtr<UObject>> UCineAssembly::FindAssociatedAssetsByLabel(FName InLabel) const
{
	TArray<TSoftObjectPtr<UObject>> Result;
	for (const FAssemblyAssociatedAssetDesc& Desc : AssociatedAssets)
	{
		if (Desc.Label == InLabel && !Desc.CreatedAsset.IsNull())
		{
			Result.Add(Desc.CreatedAsset);
		}
	}
	return Result;
}

#if WITH_EDITOR

FString UCineAssembly::GetAuthor() const
{
	UMovieSceneMetaData* MetaData = FindMetaData<UMovieSceneMetaData>();
	return MetaData != nullptr ? MetaData->GetAuthor() : FString();
}

void UCineAssembly::SetAuthor(const FString& InAuthor)
{
	UMovieSceneMetaData* MetaData = FindOrAddMetaData<UMovieSceneMetaData>();
	MetaData->Modify();
	MetaData->SetAuthor(InAuthor);
}

FString UCineAssembly::GetCreatedString() const
{
	if (TOptional<FDateTime> CreatedLocalTime = TryGetCreatedAsLocalTime(); CreatedLocalTime.IsSet())
	{
		return CreatedLocalTime->ToString();
	}
	
	return FString();
}

FString UCineAssembly::GetDateCreatedString() const
{
	if (TOptional<FDateTime> CreatedLocalTime = TryGetCreatedAsLocalTime(); CreatedLocalTime.IsSet())
	{
		return CreatedLocalTime->ToFormattedString(TEXT("%Y-%m-%d"));
	}

	return FString();
}

FString UCineAssembly::GetTimeCreatedString() const
{
	if (TOptional<FDateTime> CreatedLocalTime = TryGetCreatedAsLocalTime(); CreatedLocalTime.IsSet())
	{
		return CreatedLocalTime->ToFormattedString(TEXT("%H:%M:%S"));
	}

	return FString();
}

#endif // WITH_EDITOR

FString UCineAssembly::GetFullMetadataString() const
{
	FString JsonString;

	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&JsonString);
	FJsonSerializer::Serialize(MetadataJsonObject.ToSharedRef(), JsonWriter);

	return JsonString;
}

TArray<FString> UCineAssembly::GetMetadataKeys() const
{
	TArray<FJsonObject::FStringType> Keys;
	MetadataJsonObject->Values.GetKeys(Keys);

	TArray<FString> Result;
	Result.Reserve(Keys.Num());
	for (const FJsonObject::FStringType& Key : Keys)
	{
		Result.Emplace(FString(Key));
	}
	return Result;
}

void UCineAssembly::SetMetadataAsString(FString InKey, FString InValue)
{
	Modify();
	MetadataJsonObject->SetStringField(InKey, InValue);
	AddMetadataNamingToken(InKey);
	OnAssemblyMetadataChanged().Broadcast(this, InKey);
}

void UCineAssembly::SetMetadataAsTokenString(FString InKey, FTemplateString InValue)
{
	Modify();
	TSharedPtr<FJsonObject> TemplateStringObject = FJsonObjectConverter::UStructToJsonObject<FTemplateString>(InValue);
	MetadataJsonObject->SetObjectField(InKey, TemplateStringObject);
	AddMetadataNamingToken(InKey);
	OnAssemblyMetadataChanged().Broadcast(this, InKey);
}

void UCineAssembly::SetMetadataAsBool(FString InKey, bool InValue)
{
	Modify();
	MetadataJsonObject->SetBoolField(InKey, InValue);
	AddMetadataNamingToken(InKey);
	OnAssemblyMetadataChanged().Broadcast(this, InKey);
}

void UCineAssembly::SetMetadataAsInteger(FString InKey, int32 InValue)
{
	Modify();
	MetadataJsonObject->SetNumberField(InKey, InValue);
	AddMetadataNamingToken(InKey);
	OnAssemblyMetadataChanged().Broadcast(this, InKey);
}

void UCineAssembly::SetMetadataAsFloat(FString InKey, float InValue)
{
	Modify();
	MetadataJsonObject->SetNumberField(InKey, InValue);
	AddMetadataNamingToken(InKey);
	OnAssemblyMetadataChanged().Broadcast(this, InKey);
}

bool UCineAssembly::GetMetadataAsString(FString InKey, FString& OutValue) const
{
	if (!MetadataJsonObject->TryGetStringField(InKey, OutValue))
	{
		FTemplateString TemplateString;
		if (!GetMetadataAsTokenString(InKey, TemplateString))
		{
			OutValue = TEXT("");
			return false;
		}

		OutValue = TemplateString.Resolved.ToString();
	}
	return true;
}

bool UCineAssembly::GetMetadataAsTokenString(FString InKey, FTemplateString& OutValue) const
{
	const TSharedPtr<FJsonObject>* TemplateStringObject;
	if (!MetadataJsonObject->TryGetObjectField(InKey, TemplateStringObject))
	{
		OutValue.Template = TEXT("");
		OutValue.Resolved = FText::GetEmpty();
		return false;
	}

	FJsonObjectConverter::JsonObjectToUStruct<FTemplateString>(TemplateStringObject->ToSharedRef(), &OutValue);
	return true;
}

bool UCineAssembly::GetMetadataAsBool(FString InKey, bool& OutValue) const
{
	if (!MetadataJsonObject->TryGetBoolField(InKey, OutValue))
	{
		OutValue = false;
		return false;
	}
	return true;
}

bool UCineAssembly::GetMetadataAsInteger(FString InKey, int32& OutValue) const
{
	if (!MetadataJsonObject->TryGetNumberField(InKey, OutValue))
	{
		OutValue = 0;
		return false;
	}
	return true;
}

bool UCineAssembly::GetMetadataAsFloat(FString InKey, float& OutValue) const
{
	if (!MetadataJsonObject->TryGetNumberField(InKey, OutValue))
	{
		OutValue = 0;
		return false;
	}
	return true;
}

void UCineAssembly::ApplyMetadata(const TMap<FString, FString>& InMetadata)
{
	const UCineAssemblySchema* Schema = GetSchema();

	for (const TPair<FString, FString>& Pair : InMetadata)
	{
		if (Pair.Key.IsEmpty())
		{
			continue;
		}

		const FAssemblyMetadataDesc* MetadataDesc = Schema ? Algo::FindBy(Schema->AssemblyMetadata, Pair.Key, &FAssemblyMetadataDesc::Key) : nullptr;
		if (MetadataDesc)
		{
			switch (MetadataDesc->Type)
			{
			case ECineAssemblyMetadataType::String:
				if (MetadataDesc->bEvaluateTokens)
				{
					FTemplateString TemplateString;
					TemplateString.Template = Pair.Value;
					TemplateString.Resolved = UCineAssemblyNamingTokens::GetResolvedText(TemplateString.Template, this);

					SetMetadataAsTokenString(Pair.Key, TemplateString);
				}
				else
				{
					SetMetadataAsString(Pair.Key, Pair.Value);
				}
				break;

			case ECineAssemblyMetadataType::Bool:
				SetMetadataAsBool(Pair.Key, Pair.Value.ToBool());
				break;

			case ECineAssemblyMetadataType::Integer:
				SetMetadataAsInteger(Pair.Key, FCString::Atoi(*Pair.Value));
				break;

			case ECineAssemblyMetadataType::Float:
				SetMetadataAsFloat(Pair.Key, FCString::Atof(*Pair.Value));
				break;

			case ECineAssemblyMetadataType::AssetPath:
			case ECineAssemblyMetadataType::CineAssembly:
				SetMetadataAsString(Pair.Key, Pair.Value);
				break;

			default:
				checkNoEntry();
			}
		}
		else
		{
			// If the key does not match anything specified by the schema, store it as InstanceMetadata
			InstanceMetadata.Add(*Pair.Key, Pair.Value);
			SetMetadataAsString(Pair.Key, Pair.Value);
		}
	}
}

void UCineAssembly::UpdateInstanceMetadata()
{
	// Copy our metadata key list so that we can remove keys as we encounter them in the map non-destructively
	TArray<FName> InstanceMetadataKeysCopy = InstanceMetadataKeys;
	for (const TPair<FName, FString>& Pair : InstanceMetadata)
	{
		if (!Pair.Key.IsNone())
		{
			if (InstanceMetadataKeys.Contains(Pair.Key))
			{
				// This is an existing metadata key that we are already tracking
				InstanceMetadataKeysCopy.Remove(Pair.Key);
			}
			else
			{
				// This is a new metadata key that we were not previously tracking
				InstanceMetadataKeys.Add(Pair.Key);
			}

			SetMetadataAsString(Pair.Key.ToString(), Pair.Value);
		}
	}

	// If there are any keys remaining in our copy of the metadata key list, then those keys must have been removed from the instance metadata map
	for (const FName& Key : InstanceMetadataKeysCopy)
	{
		MetadataJsonObject->RemoveField(Key.ToString());
	}
}

TOptional<FDateTime> UCineAssembly::TryGetCreatedAsLocalTime() const
{
#if WITH_EDITOR
	// Assuming that the Created field from the Metadata is in UTC.
	if (UMovieSceneMetaData* MetaData = FindMetaData<UMovieSceneMetaData>(); MetaData != nullptr)
	{
		return UE::CineAssembly::ConvertTime(MetaData->GetCreated(), 0 /* UTC */, UE::CineAssembly::GetLocalTimezone());
	}
#endif // WITH_EDITOR

	return TOptional<FDateTime>();
}

void UCineAssembly::AddMetadataNamingToken(const FString& InKey)
{
	UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();
	UCineAssemblyNamingTokens* CineAssemblyNamingTokens = Cast<UCineAssemblyNamingTokens>(NamingTokensSubsystem->GetNamingTokens(UCineAssemblyNamingTokens::TokenNamespace));

	CineAssemblyNamingTokens->AddMetadataToken(InKey);
}

void UCineAssembly::GetAssetPathAndRootFolder(FString& OutAssetPath, FString& OutRootFolder)
{
	const FAssetData AssemblyAssetData = FAssetData(this);
	OutAssetPath = AssemblyAssetData.PackagePath.ToString();

	const FString ResolvedAssemblyPath = UCineAssemblyNamingTokens::GetResolvedText(PathRelativeToRoot.Template, this).ToString();
	OutRootFolder = OutAssetPath.Replace(*ResolvedAssemblyPath, TEXT(""));
}

#undef LOCTEXT_NAMESPACE
