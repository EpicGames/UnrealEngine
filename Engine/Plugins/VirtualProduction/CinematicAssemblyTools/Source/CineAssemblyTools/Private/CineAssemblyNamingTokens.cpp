// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblyNamingTokens.h"

#include "Misc/PackageName.h"
#include "MovieSceneMetaData.h"
#include "NamingTokensEngineSubsystem.h"

#include "Algo/IndexOf.h"

#define LOCTEXT_NAMESPACE "CineAssemblyNamingTokens"

FString UCineAssemblyNamingTokens::TokenNamespace = TEXT("cat");

FText UCineAssemblyNamingTokens::GetResolvedText(const FString& InStringToEvaluate, UCineAssembly* InAssembly)
{
	UCineAssemblyNamingTokensContext* NamingTokenContext = NewObject<UCineAssemblyNamingTokensContext>();
	NamingTokenContext->Assembly = InAssembly;

	FNamingTokenFilterArgs FilterArgs;
	FilterArgs.AdditionalNamespacesToInclude.Add(TokenNamespace);

	UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();
	FNamingTokenResultData Result = NamingTokensSubsystem->EvaluateTokenString(InStringToEvaluate, FilterArgs, { NamingTokenContext });

	return Result.EvaluatedText;
}

UCineAssemblyNamingTokens::UCineAssemblyNamingTokens()
{
	Namespace = TokenNamespace;
}

void UCineAssemblyNamingTokens::OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens)
{
	Super::OnCreateDefaultTokens(Tokens);

	auto AssemblyNameTokenFunc = [](TWeakObjectPtr<UCineAssembly> InAssembly) -> FText
		{
			if (InAssembly.IsValid())
			{
				return InAssembly->AssemblyName.Resolved;
			}
			return FText::GetEmpty();
		};

	auto SchemaTokenFunc = [](TWeakObjectPtr<UCineAssembly> InAssembly) -> FText
		{
			if (InAssembly.IsValid())
			{
				if (const UCineAssemblySchema* Schema = InAssembly->GetSchema())
				{
					return FText::FromString(Schema->SchemaName);
				}
			}
			return FText::GetEmpty();
		};

	auto LevelTokenFunc = [](TWeakObjectPtr<UCineAssembly> InAssembly) -> FText
		{
			if (InAssembly.IsValid() && InAssembly->Level.IsValid())
			{
				return FText::FromString(InAssembly->Level.GetAssetName());
			}
			return FText::GetEmpty();
		};

	auto ParentTokenFunc = [](TWeakObjectPtr<UCineAssembly> InAssembly) -> FText
		{
			if (InAssembly.IsValid() && InAssembly->ParentAssembly.IsValid())
			{
				const FString PackageName = InAssembly->ParentAssembly.GetLongPackageName();
				if (FPackageName::IsInEngineTransientPackages(PackageName))
				{
					// If the parent assembly is transient, use the resolved AssemblyName value
					const UCineAssembly* ParentAssembly = Cast<UCineAssembly>(InAssembly->ParentAssembly.ResolveObject());
					return ParentAssembly ? ParentAssembly->AssemblyName.Resolved : FText::GetEmpty();
				}
				else
				{
					return FText::FromString(InAssembly->ParentAssembly.GetAssetName());
				}
			}
			return FText::GetEmpty();
		};

	auto ProductionTokenFunc = [](TWeakObjectPtr<UCineAssembly> InAssembly) -> FText
		{
			if (InAssembly.IsValid() && InAssembly->Production.IsValid())
			{
				return FText::FromString(InAssembly->ProductionName);
			}
			return FText::GetEmpty();
		};

	auto AuthorTokenFunc = [](TWeakObjectPtr<UCineAssembly> InAssembly) -> FText
		{
#if WITH_EDITOR
			return InAssembly.IsValid()
				? FText::FromString(InAssembly->GetAuthor()) 
				: FText::GetEmpty();
#else // WITH_EDITOR
			return FText::GetEmpty();
#endif // WITH_EDITOR
		};

	auto CreatedTokenFunc = [](TWeakObjectPtr<UCineAssembly> InAssembly) -> FText
		{
#if WITH_EDITOR
			return InAssembly.IsValid()
				? FText::FromString(InAssembly->GetCreatedString()) 
				: FText::GetEmpty();
#else // WITH_EDITOR
			return FText::GetEmpty();
#endif // WITH_EDITOR
		};

	auto DateCreatedTokenFunc = [](TWeakObjectPtr<UCineAssembly> InAssembly) -> FText
		{
#if WITH_EDITOR
			return InAssembly.IsValid()
				? FText::FromString(InAssembly->GetDateCreatedString())
				: FText::GetEmpty();
#else // WITH_EDITOR
			return FText::GetEmpty();
#endif // WITH_EDITOR
		};

	auto TimeCreatedTokenFunc = [](TWeakObjectPtr<UCineAssembly> InAssembly) -> FText
		{
#if WITH_EDITORONLY_DATA
			return InAssembly.IsValid()
				? FText::FromString(InAssembly->GetTimeCreatedString())
				: FText::GetEmpty();
#else // WITH_EDITORONLY_DATA
			return FText::GetEmpty();
#endif // WITH_EDITORONLY_DATA
		};

	FNamingTokenData AssemblyNameToken;
	AssemblyNameToken.TokenKey = TEXT("assembly");
	AssemblyNameToken.DisplayName = LOCTEXT("AssemblyNameToken", "Assembly Name");
	AssemblyNameToken.TokenProcessorNative.BindLambda([this, AssemblyNameTokenFunc]() { return ExecuteTokenFunc(AssemblyNameTokenFunc); });

	FNamingTokenData SchemaToken;
	SchemaToken.TokenKey = TEXT("schema");
	SchemaToken.DisplayName = LOCTEXT("SchemaToken", "Base Schema");
	SchemaToken.TokenProcessorNative.BindLambda([this, SchemaTokenFunc]() { return ExecuteTokenFunc(SchemaTokenFunc); });

	FNamingTokenData LevelToken;
	LevelToken.TokenKey = TEXT("level");
	LevelToken.DisplayName = LOCTEXT("TargetLevelTokenName", "Target Level");
	LevelToken.TokenProcessorNative.BindLambda([this, LevelTokenFunc]() { return ExecuteTokenFunc(LevelTokenFunc); });

	FNamingTokenData ParentToken;
	ParentToken.TokenKey = TEXT("parent");
	ParentToken.DisplayName = LOCTEXT("ParentTokenName", "Parent Assembly");
	ParentToken.TokenProcessorNative.BindLambda([this, ParentTokenFunc]() { return ExecuteTokenFunc(ParentTokenFunc); });

	FNamingTokenData ProductionToken;
	ProductionToken.TokenKey = TEXT("production");
	ProductionToken.DisplayName = LOCTEXT("ProductionTokenName", "Production");
	ProductionToken.TokenProcessorNative.BindLambda([this, ProductionTokenFunc]() { return ExecuteTokenFunc(ProductionTokenFunc); });

	FNamingTokenData AuthorToken;
	AuthorToken.TokenKey = TEXT("author");
	AuthorToken.DisplayName = LOCTEXT("AuthorTokenName", "Author");
	AuthorToken.TokenProcessorNative.BindLambda([this, AuthorTokenFunc]() { return ExecuteTokenFunc(AuthorTokenFunc); });

	FNamingTokenData CreatedToken;
	CreatedToken.TokenKey = TEXT("created");
	CreatedToken.DisplayName = LOCTEXT("CreatedTokenName", "Created");
	CreatedToken.TokenProcessorNative.BindLambda([this, CreatedTokenFunc]() { return ExecuteTokenFunc(CreatedTokenFunc); });

	FNamingTokenData DateCreatedToken;
	DateCreatedToken.TokenKey = TEXT("dateCreated");
	DateCreatedToken.DisplayName = LOCTEXT("DateCreatedTokenName", "Date Created");
	DateCreatedToken.TokenProcessorNative.BindLambda([this, DateCreatedTokenFunc]() { return ExecuteTokenFunc(DateCreatedTokenFunc); });

	FNamingTokenData TimeCreatedToken;
	TimeCreatedToken.TokenKey = TEXT("timeCreated");
	TimeCreatedToken.DisplayName = LOCTEXT("TimeCreatedTokenName", "Time Created");
	TimeCreatedToken.TokenProcessorNative.BindLambda([this, TimeCreatedTokenFunc]() { return ExecuteTokenFunc(TimeCreatedTokenFunc); });

	Tokens.Add(AssemblyNameToken);
	Tokens.Add(SchemaToken);
	Tokens.Add(LevelToken);
	Tokens.Add(ParentToken);
	Tokens.Add(ProductionToken);
	Tokens.Add(AuthorToken);
	Tokens.Add(CreatedToken);
	Tokens.Add(DateCreatedToken);
	Tokens.Add(TimeCreatedToken);
}

void UCineAssemblyNamingTokens::OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData)
{
	Super::OnPreEvaluate_Implementation(InEvaluationData);

	UCineAssemblyNamingTokensContext* MatchingContext = nullptr;
	InEvaluationData.Contexts.FindItemByClass<UCineAssemblyNamingTokensContext>(&MatchingContext);
	Context = MatchingContext;
}

void UCineAssemblyNamingTokens::OnPostEvaluate_Implementation()
{
	Super::OnPostEvaluate_Implementation();
	Context = nullptr;
}

void UCineAssemblyNamingTokens::AddMetadataToken(const FString& InTokenKey)
{
	// If the token already exists, remove it, and replace it with the new one
	const int32 TokenIndex = Algo::IndexOfBy(CustomTokens, InTokenKey, &FNamingTokenData::TokenKey);
	if (CustomTokens.IsValidIndex(TokenIndex))
	{
		CustomTokens.RemoveAt(TokenIndex);
	}

	auto MetadataTokenFunc = [InTokenKey](TWeakObjectPtr<UCineAssembly> InAssembly) -> FText
		{
			if (InAssembly.IsValid())
			{
				FString ValueString;
				if (InAssembly->GetMetadataAsString(InTokenKey, ValueString))
				{
					// Test if the value string could be an object path, and if it is, return just the filename instead of the full path
					FSoftObjectPath ObjectPath = FSoftObjectPath(ValueString);
					if (ObjectPath.IsValid())
					{
						ValueString = FPaths::GetBaseFilename(ValueString);
					}

					return FText::FromString(ValueString);
				}
			}

			return FText::GetEmpty();
		};

	FNamingTokenData NewToken;
	NewToken.TokenKey = InTokenKey;
	NewToken.DisplayName = FText::Format(LOCTEXT("MetadataTokenDisplayName", "{0} Metadata"), FText::FromString(InTokenKey));
	NewToken.TokenProcessorNative.BindLambda([this, MetadataTokenFunc]() { return ExecuteTokenFunc(MetadataTokenFunc); });

	CustomTokens.Add(NewToken);
}

FText UCineAssemblyNamingTokens::ExecuteTokenFunc(TFunction<FText(TWeakObjectPtr<UCineAssembly>)> TokenFunc)
{
	// Try evaluating the token function using the assembly from the context.
	if (Context)
	{
		return TokenFunc(Context->Assembly);
	}

	// Cannot evaluate token function without a valid assembly
	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
