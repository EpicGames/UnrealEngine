// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblySchema.h"

#include "Algo/Contains.h"
#include "Algo/Find.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CineAssembly.h"
#include "Engine/World.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneSubAssemblySection.h"
#include "MovieSceneSubAssemblyTrack.h"
#include "UObject/AssetRegistryTagsContext.h"

#if WITH_EDITOR
#include "AssetToolsModule.h"
#endif

const FName UCineAssemblySchema::SchemaGuidPropertyName = GET_MEMBER_NAME_CHECKED(UCineAssemblySchema, SchemaGuid);
const FName UCineAssemblySchema::AssetRegistryTag_ThumbnailTexture = "SchemaThumbnailTexture";

const FString UCineAssemblySchema::DefaultLevelMetadataKey = TEXT("DefaultLevel");

UCineAssemblySchema::UCineAssemblySchema()
{
}

FGuid UCineAssemblySchema::GetSchemaGuid() const
{
	return SchemaGuid;
}

void UCineAssemblySchema::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	for (FAssemblyMetadataDesc& Desc : AssemblyMetadata)
	{
		Ar << Desc.DefaultValue;
	}
}

void UCineAssemblySchema::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		if (!HasAnyFlags(RF_NeedLoad | RF_WasLoaded) && !SchemaGuid.IsValid())
		{
			SchemaGuid = FGuid::NewGuid();
		}

		if (!TemplateSequence)
		{
			TemplateSequence = NewObject<UCineAssembly>(this);
			TemplateSequence->Initialize();
		}

		// Create a metadata desc for each linkable Assembly property
		FAssemblyMetadataDesc& LevelPropertyMetadataDesc = AssemblyPropertyMetadata.AddDefaulted_GetRef();
		LevelPropertyMetadataDesc.Key = UCineAssemblySchema::DefaultLevelMetadataKey;
		LevelPropertyMetadataDesc.Type = ECineAssemblyMetadataType::AssetPath;
		LevelPropertyMetadataDesc.AssetClass = UWorld::StaticClass();
	}
}

void UCineAssemblySchema::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (!bDuplicateForPIE)
	{
		SchemaGuid = FGuid::NewGuid();

		// Change the name property to match the actual name of the duplicated asset
		SchemaName = this->GetName();
	}
}

void UCineAssemblySchema::PostLoad()
{
	Super::PostLoad();

	if (!SchemaGuid.IsValid())
	{
		SchemaGuid = FGuid::NewGuid();
	}

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (ThumbnailImage_DEPRECATED && ThumbnailTexture.IsNull())
	{
		ThumbnailTexture = ThumbnailImage_DEPRECATED.Get();
		ThumbnailImage_DEPRECATED = nullptr;
	}

	// Fix-up old schema assets that were created before support was added for the fully editable template sequence
	// If a top-level Template was previously set, use it to initialize the TemplateSequence
	if (TemplateSequence && TemplateSequence->GetMovieScene())
	{
		if (ULevelSequence* OldTemplate = Cast<ULevelSequence>(Template_DEPRECATED.TryLoad()))
		{
			TemplateSequence->InitializeFromTemplate(OldTemplate);
		}
		Template_DEPRECATED.Reset();

		UMovieScene* TemplateSequenceMovieScene = TemplateSequence->GetMovieScene();
		const TRange<FFrameNumber> TemplateSequencePlaybackRange = TemplateSequenceMovieScene->GetPlaybackRange();

		// If any named subsequences were previously specified in SubsequencesToCreate, add new SubAssembly tracks/sections to the TemplateSequence instead
		for (const FString& SubAssemblyName : SubsequencesToCreate_DEPRECATED)
		{
			// Add a new SubAssembly Track to the TemplateSequence
			UMovieSceneSubAssemblyTrack* SubAssemblyTrack = Cast<UMovieSceneSubAssemblyTrack>(TemplateSequenceMovieScene->AddTrack(UMovieSceneSubAssemblyTrack::StaticClass()));
			SubAssemblyTrack->TrackType = ESubAssemblyTrackType::SubsequenceTrack;

			SubAssemblyTrack->SetLocalEvalDisabled(true);

			// Add a new SubAssembly Section to the SubAssembly Track
			const FFrameNumber StartFrame = TemplateSequencePlaybackRange.GetLowerBoundValue();
			const int32 Duration = TemplateSequencePlaybackRange.Size<FFrameNumber>().Value;

			UMovieSceneSubAssemblySection* SubAssemblySection = Cast<UMovieSceneSubAssemblySection>(SubAssemblyTrack->AddSequence(nullptr, StartFrame, Duration));

			// Set the Assembly Template of the new section 
			if (FSoftObjectPath* TemplatePath = SubsequenceTemplates_DEPRECATED.Find(SubAssemblyName))
			{
				ULevelSequence* SubAssemblyTemplate = Cast<ULevelSequence>(TemplatePath->TryLoad());
				SubAssemblySection->SetAssemblyTemplate(SubAssemblyTemplate);
			}
			else
			{
				SubAssemblySection->SetAssemblyTemplate(nullptr);
			}

			// Set the name and path of the new section based on the old name of this SubsequenceToCreate property
			FString NewSequenceName = FPaths::GetPathLeaf(SubAssemblyName);
			FString NewSequencePath = FPaths::GetPath(SubAssemblyName);

			// Replace any occurrences of the "{assembly}" token in the NewSequenceName with the "{parent}" token
			// Assemblies cannot be named using the "{assembly}" token because it would be self-referencing
			// The prior intent of using the "{assembly}" token in the name of subsequences was to name them after their parent sequence
			// Updating the tokens here correctly expresses that intent and ensures the subsequences will be named correctly. 
			NewSequenceName = NewSequenceName.Replace(TEXT("{cat:assembly}"), TEXT("{cat:parent}"));
			NewSequenceName = NewSequenceName.Replace(TEXT("{assembly}"), TEXT("{parent}"));

			SubAssemblySection->SetSequenceName(FText::AsCultureInvariant(NewSequenceName));
			SubAssemblySection->SetSequencePath(NewSequencePath);
			SubAssemblySection->SetDefaultLabel();
		}

		SubsequencesToCreate_DEPRECATED.Empty();

		// Store the old deprecated default assembly path in the template sequence
		if (!DefaultAssemblyPath_DEPRECATED.IsEmpty())
		{
			TemplateSequence->PathRelativeToRoot.Template = DefaultAssemblyPath_DEPRECATED;
			DefaultAssemblyPath_DEPRECATED.Empty();
		}

		// Store the old deprecated list of folders to create in the template sequence
		if (!FoldersToCreate_DEPRECATED.IsEmpty())
		{
			TemplateSequence->DefaultFolderNames.Reset(FoldersToCreate_DEPRECATED.Num());
			Algo::Transform(FoldersToCreate_DEPRECATED, TemplateSequence->DefaultFolderNames, [](const FString& TemplateString) { return FTemplateString(TemplateString, FText::GetEmpty()); });
			FoldersToCreate_DEPRECATED.Empty();
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR

	// Assign default labels to any SubAssembly template sections that do not have one.
	const UMovieScene* TemplateMovieScene = TemplateSequence ? TemplateSequence->GetMovieScene() : nullptr;
	if (TemplateMovieScene)
	{
		for (UMovieSceneSection* Section : TemplateMovieScene->GetAllSections())
		{
			UMovieSceneSubAssemblySection* SubAssemblySection = Cast<UMovieSceneSubAssemblySection>(Section);
			if (SubAssemblySection && SubAssemblySection->IsTemplateSection() && SubAssemblySection->Label.IsNone())
			{
				SubAssemblySection->SetDefaultLabel();
			}
		}
	}
}

void UCineAssemblySchema::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	const FString TagValue = !ThumbnailTexture.IsNull() ? ThumbnailTexture.ToSoftObjectPath().ToString() : FString();
	Context.AddTag(FAssetRegistryTag(AssetRegistryTag_ThumbnailTexture, TagValue, FAssetRegistryTag::TT_Hidden));
}

FString UCineAssemblySchema::GetDefaultAssemblyName() const
{
	if (DefaultAssemblyName.IsEmpty())
	{
		return TEXT("New") + SchemaName;
	}
	return DefaultAssemblyName;
}

FString UCineAssemblySchema::GetDefaultAssemblyPath() const
{
	return TemplateSequence ? TemplateSequence->PathRelativeToRoot.Template : FString();
}

void UCineAssemblySchema::SetDefaultAssemblyPath(const FString& InPath)
{
	if (TemplateSequence)
	{
		TemplateSequence->Modify();
		TemplateSequence->PathRelativeToRoot.Template = InPath;
	}
}

TSoftObjectPtr<UWorld> UCineAssemblySchema::GetDefaultLevel() const
{
	if (bOverrideDefaultLevel)
	{
		return TSoftObjectPtr<UWorld>(DefaultLevel);
	}
	return nullptr;
}

void UCineAssemblySchema::SetDefaultLevel(TSoftObjectPtr<UWorld> InLevel)
{
	bOverrideDefaultLevel = true;
	DefaultLevel = InLevel.ToSoftObjectPath();
}

void UCineAssemblySchema::ClearDefaultLevel()
{
	bOverrideDefaultLevel = false;
	DefaultLevel.Reset();
}

TArray<FString> UCineAssemblySchema::GetDefaultFolders() const
{
	TArray<FString> FolderPaths;
	if (TemplateSequence)
	{
		FolderPaths.Reserve(TemplateSequence->DefaultFolderNames.Num());
		Algo::Transform(TemplateSequence->DefaultFolderNames, FolderPaths, [](const FTemplateString& FolderName) { return FolderName.Template; });
	}
	return FolderPaths;
}

void UCineAssemblySchema::AddDefaultFolder(const FString& FolderPath)
{
	if (!TemplateSequence)
	{
		return;
	}

	TemplateSequence->Modify();

	FString NormalizedPath = FolderPath;
	NormalizedPath.RemoveFromEnd(TEXT("/"));

	// Split the path into segments and add each intermediate path (e.g. "A/B/C" adds "A", "A/B", "A/B/C")
	TArray<FString> Segments;
	NormalizedPath.ParseIntoArray(Segments, TEXT("/"));

	FString AccumulatedPath;
	for (const FString& Segment : Segments)
	{
		AccumulatedPath = AccumulatedPath.IsEmpty() ? Segment : AccumulatedPath / Segment;

		if (!Algo::ContainsBy(TemplateSequence->DefaultFolderNames, AccumulatedPath, &FTemplateString::Template))
		{
			TemplateSequence->DefaultFolderNames.Add(FTemplateString(AccumulatedPath, FText::GetEmpty()));
		}
	}
}

bool UCineAssemblySchema::RemoveDefaultFolder(const FString& FolderPath)
{
	if (!TemplateSequence)
	{
		return false;
	}

	TemplateSequence->Modify();

	FString NormalizedPath = FolderPath;
	NormalizedPath.RemoveFromEnd(TEXT("/"));

	// Remove the folder and all descendant folders
	const int32 NumRemoved = TemplateSequence->DefaultFolderNames.RemoveAll([&NormalizedPath](const FTemplateString& FolderName)
		{
			return FolderName.Template.Equals(NormalizedPath) || FolderName.Template.StartsWith(NormalizedPath + TEXT("/"));
		});

	return NumRemoved > 0;
}

bool UCineAssemblySchema::SupportsRename() const
{
	return bSupportsRename;
}

#if WITH_EDITOR
void UCineAssemblySchema::RenameAsset(const FString& InNewName)
{
	// Early-out if the input name already matches the name of this schema
	if (SchemaName == InNewName)
	{
		return;
	}

	// If this schema does not yet have a valid package yet (i.e. it is still being configured), then there is no need to use Asset Tools to rename it
	if (GetPackage() == GetTransientPackage())
	{
		SchemaName = InNewName;
		return;
	}

	// The default behavior for schema assets is to not allow renaming from Content Browser.
	// However, this function relies on renaming being supported, so we temporarily enable to do the programmatic rename.
	bSupportsRename = true;

	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();

	const FString PackagePath = FPackageName::GetLongPackagePath(GetOutermost()->GetName());

	TArray<FAssetRenameData> AssetsAndNames;
	const bool bSoftReferenceOnly = false;
	const bool bAlsoRenameLocalizedVariants = true;
	AssetsAndNames.Emplace(FAssetRenameData(this, PackagePath, InNewName, bSoftReferenceOnly, bAlsoRenameLocalizedVariants));

	EAssetRenameResult Result = AssetTools.RenameAssetsWithDialog(AssetsAndNames);
	if (Result != EAssetRenameResult::Failure)
	{
		SchemaName = InNewName;
	}

	bSupportsRename = false;
}
#endif

const TArray<FAssemblyMetadataDesc>& UCineAssemblySchema::GetAssemblyPropertyMetadata() const
{
	return AssemblyPropertyMetadata;
}

const FAssemblyMetadataDesc* UCineAssemblySchema::FindMetadataDesc(const FString& InMetadataKey) const
{
	if (const FAssemblyMetadataDesc* Desc = Algo::FindBy(AssemblyMetadata, InMetadataKey, &FAssemblyMetadataDesc::Key))
	{
		return Desc;
	}
	return Algo::FindBy(AssemblyPropertyMetadata, InMetadataKey, &FAssemblyMetadataDesc::Key);
}

void UCineAssemblySchema::ForEachMetadataDesc(TFunctionRef<bool(const FAssemblyMetadataDesc&)> Visitor) const
{
	for (const FAssemblyMetadataDesc& Desc : AssemblyMetadata)
	{
		if (!Visitor(Desc))
		{
			return;
		}
	}
	for (const FAssemblyMetadataDesc& Desc : AssemblyPropertyMetadata)
	{
		if (!Visitor(Desc))
		{
			return;
		}
	}
}

FName UCineAssemblySchema::MakeUniqueAssemblyLabel(const FString& BaseName) const
{
	FString NormalizedBase = BaseName;
	NormalizedBase.RemoveSpacesInline();

	UMovieScene* MovieScene = TemplateSequence ? TemplateSequence->GetMovieScene() : nullptr;
	if (NormalizedBase.IsEmpty() || !MovieScene)
	{
		return NAME_None;
	}

	int32 Suffix = 1;
	FName CandidateLabel;
	do
	{
		CandidateLabel = FName(*FString::Printf(TEXT("%s%d"), *NormalizedBase, Suffix));
		++Suffix;
	} while (Algo::AnyOf(MovieScene->GetAllSections(), [CandidateLabel](UMovieSceneSection* Section)
		{
			const UMovieSceneSubAssemblySection* SubSection = Cast<UMovieSceneSubAssemblySection>(Section);
			return SubSection && SubSection->Label == CandidateLabel;
		}));

	return CandidateLabel;
}

FName UCineAssemblySchema::MakeUniqueAssetLabel(const FString& BaseName) const
{
	FString NormalizedBase = BaseName;
	NormalizedBase.RemoveSpacesInline();

	if (NormalizedBase.IsEmpty() || !TemplateSequence)
	{
		return NAME_None;
	}

	const TArray<FAssemblyAssociatedAssetDesc>& Assets = TemplateSequence->AssociatedAssets;

	int32 Suffix = 1;
	FName CandidateLabel;
	do
	{
		CandidateLabel = FName(*FString::Printf(TEXT("%s%d"), *NormalizedBase, Suffix));
		++Suffix;
	} while (Algo::AnyOf(Assets, [CandidateLabel](const FAssemblyAssociatedAssetDesc& AssetDesc) { return AssetDesc.Label == CandidateLabel; }));

	return CandidateLabel;
}
