// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "CineAssembly.h"
#include "MovieSceneMetaData.h"
#include "MetaData/MovieSceneShotMetaData.h"

#define LOCTEXT_NAMESPACE "CineAssemblyToolsModule"

/**
 * Implements the CineAssemblyTools module.
 */
class FCineAssemblyToolsModule : public IModuleInterface
{
public:

	//~ Begin IModuleInterface interface
	void StartupModule() override
	{
#if WITH_EDITOR
		// Add empty movie scene meta data to the UCineAssembly CDO to ensure that
		// asset registry tooltips show up in the editor
		CineAssemblyCDO = GetMutableDefault<UCineAssembly>();

		if (UCineAssembly* CDO = CineAssemblyCDO.Get())
		{
			UMovieSceneMetaData* MetaData = CDO->FindOrAddMetaData<UMovieSceneMetaData>();
			MetaData->SetFlags(RF_Transient);
			UMovieSceneShotMetaData* ShotMetaData = CDO->FindOrAddMetaData<UMovieSceneShotMetaData>();
			ShotMetaData->SetFlags(RF_Transient);
		}
#endif
	}

	void ShutdownModule() override
	{
#if WITH_EDITOR
		if (UCineAssembly* CDO = CineAssemblyCDO.Get())
		{
			CDO->RemoveMetaData<UMovieSceneShotMetaData>();
			CDO->RemoveMetaData<UMovieSceneMetaData>();
		}
#endif
	}
	//~ End IModuleInterface interface

private:
	TWeakObjectPtr<UCineAssembly> CineAssemblyCDO;
};

IMPLEMENT_MODULE(FCineAssemblyToolsModule, CineAssemblyTools);

#undef LOCTEXT_NAMESPACE
