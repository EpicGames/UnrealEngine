// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSubAssemblySection.h"

#include "CineAssembly.h"
#include "CineAssemblySchema.h"
#include "LevelSequence.h"
#include "MovieSceneSubAssemblyTrack.h"

#define LOCTEXT_NAMESPACE "MovieSceneSubAssemblySection"

const FName UMovieSceneSubAssemblySection::AssemblyTemplatePropertyName = GET_MEMBER_NAME_CHECKED(UMovieSceneSubAssemblySection, AssemblyTemplate);
const FColor UMovieSceneSubAssemblySection::ReferenceSectionColorTint = FColor(); // No Tint
const FColor UMovieSceneSubAssemblySection::TemplateSectionColorTint = FColor(229, 45, 113); // The CineAssembly asset color

UMovieSceneSubAssemblySection::UMovieSceneSubAssemblySection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMovieSceneSubAssemblySection::PostInitProperties()
{
	Super::PostInitProperties();

	if (!SectionID.IsValid())
	{
		SectionID = FGuid::NewGuid();
	}
}

void UMovieSceneSubAssemblySection::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	// Assembly metadata linking depends on SubAssembly GUIDs being stable during Assembly initialization.
	// During initialization, the Assembly will duplicate its MovieScene from a template sequence, and we want the duplicate section to have the same GUID.
	// The transient duplicate check strongly suggests that this duplication is happening during Assembly initialization.
	if (!bDuplicateForPIE && GetOutermost() != GetTransientPackage())
	{
		SectionID = FGuid::NewGuid();
	}
}

void UMovieSceneSubAssemblySection::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading() && !SectionID.IsValid())
	{
		SectionID = FGuid::NewGuid();
	}
}

FGuid UMovieSceneSubAssemblySection::GetSectionID() const
{
	return SectionID;
}

const UCineAssemblySchema* UMovieSceneSubAssemblySection::GetTemplateSchema() const
{
	if (const UCineAssemblySchema* Schema = Cast<UCineAssemblySchema>(AssemblyTemplate))
	{
		return Schema;
	}
	if (const UCineAssembly* Assembly = Cast<UCineAssembly>(AssemblyTemplate))
	{
		return Assembly->GetSchema();
	}
	return nullptr;
}

bool UMovieSceneSubAssemblySection::IsReferenceSection() const
{ 
	return SectionType == ESubAssemblySectionType::Reference; 
}

bool UMovieSceneSubAssemblySection::IsTemplateSection() const
{ 
	return SectionType == ESubAssemblySectionType::Template;
}

bool UMovieSceneSubAssemblySection::IsSubsequenceSection() const
{
	UMovieSceneSubAssemblyTrack* SubAssemblyTrack = GetTypedOuter<UMovieSceneSubAssemblyTrack>();
	return (SubAssemblyTrack && SubAssemblyTrack->IsSubsequenceTrack());
}

bool UMovieSceneSubAssemblySection::IsCinematicShotSection() const
{
	UMovieSceneSubAssemblyTrack* SubAssemblyTrack = GetTypedOuter<UMovieSceneSubAssemblyTrack>();
	return (SubAssemblyTrack && SubAssemblyTrack->IsCinematicShotTrack());
}

UObject* UMovieSceneSubAssemblySection::GetAssemblyTemplate() const
{
	return AssemblyTemplate;
}

void UMovieSceneSubAssemblySection::SetAssemblyTemplate(UObject* TemplateObject)
{
	Modify();

	SectionType = ESubAssemblySectionType::Template;

	// Reset the section tint (unless the user has manually changed it to something else)
	if (GetColorTint() == ReferenceSectionColorTint)
	{
		SetColorTint(TemplateSectionColorTint);
	}

	// Early-out if the template object has not actually changed to avoid unintentionally resetting the NewSequenceName
	if (TemplateObject && (AssemblyTemplate == TemplateObject))
	{
		return;
	}

	AssemblyTemplate = TemplateObject;

	// Update the name of the NewSequence that this section will create based on the new template object
	if (UCineAssemblySchema* SchemaTemplate = Cast<UCineAssemblySchema>(TemplateObject))
	{
		NewSequenceName = FText::FromString(SchemaTemplate->GetDefaultAssemblyName());
	}
	else if (UCineAssembly* TemplateAssembly = Cast<UCineAssembly>(TemplateObject))
	{	
		NewSequenceName = FText::FromString(TemplateAssembly->AssemblyName.Template);
	}
	else if (ULevelSequence* SequenceTemplate = Cast<ULevelSequence>(TemplateObject))
	{
#if WITH_EDITOR
		NewSequenceName = SequenceTemplate->GetDisplayName();
#else
		NewSequenceName = FText::FromString(SequenceTemplate->GetName());
#endif // WITH_EDITOR
	}
	else
	{
		NewSequenceName = LOCTEXT("NewSubSequenceName", "NewSubSequence");
	}
}

FText UMovieSceneSubAssemblySection::GetSequenceName() const
{
	return NewSequenceName;
}

void UMovieSceneSubAssemblySection::SetSequenceName(const FText& InName)
{
	Modify();
	NewSequenceName = InName;
}

FString UMovieSceneSubAssemblySection::GetSequencePath() const
{
	return NewSequencePath;
}

void UMovieSceneSubAssemblySection::SetSequencePath(const FString& InPath)
{
	Modify();
	NewSequencePath = InPath;
}

void UMovieSceneSubAssemblySection::SetDefaultLabel()
{
	if (!IsTemplateSection())
	{
		return;
	}

	const UCineAssemblySchema* TemplateSchema = GetTemplateSchema();
	const FString SchemaName = TemplateSchema ? TemplateSchema->SchemaName : FString();
	const FString BaseName = !SchemaName.IsEmpty() ? SchemaName : TEXT("CineAssembly");

	if (const UCineAssemblySchema* OwningSchema = GetTypedOuter<UCineAssemblySchema>())
	{
		Modify();
		Label = OwningSchema->MakeUniqueAssemblyLabel(BaseName);
	}
}

#if WITH_EDITOR
void UMovieSceneSubAssemblySection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UMovieSceneSubAssemblySection, SectionType))
	{
		if (SectionType == ESubAssemblySectionType::Template)
		{
			// The section was previously a Reference section, so set the Assembly Template to be the SubSequence that was previously referenced
			SetAssemblyTemplate(SubSequence);
		}
		else
		{
			// The section was previously a Template section, so set the SubSequence that will be referenced to be the Assembly Template object
			// Note: This is only valid if the Assembly Template is a valid sequence (i.e. not null, and not a Schema).
			// The detail customization disables the SectionType property in cases where this would not be valid.
			if (UMovieSceneSequence* Sequence = Cast<UMovieSceneSequence>(AssemblyTemplate))
			{
				SetSequence(Sequence);

				// Reset the section tint (unless the user has manually changed it to something else)
				if (GetColorTint() == TemplateSectionColorTint)
				{
					SetColorTint(ReferenceSectionColorTint);
				}
			}
		}
	}
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
