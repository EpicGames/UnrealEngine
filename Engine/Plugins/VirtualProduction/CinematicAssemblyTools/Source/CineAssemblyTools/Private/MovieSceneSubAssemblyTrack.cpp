// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSubAssemblyTrack.h"

#include "MovieSceneSubAssemblySection.h"

#define LOCTEXT_NAMESPACE "MovieSceneSubAssemblyTrack"

UMovieSceneSubAssemblyTrack::UMovieSceneSubAssemblyTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UMovieSceneSection* UMovieSceneSubAssemblyTrack::CreateNewSection()
{
	return NewObject<UMovieSceneSubAssemblySection>(this, NAME_None, RF_Transactional);
}

bool UMovieSceneSubAssemblyTrack::IsSubsequenceTrack() const
{
	return TrackType == ESubAssemblyTrackType::SubsequenceTrack;
}

bool UMovieSceneSubAssemblyTrack::IsCinematicShotTrack() const
{
	return TrackType == ESubAssemblyTrackType::CinematicShotTrack;
}

#if WITH_EDITOR
FText UMovieSceneSubAssemblyTrack::GetDefaultDisplayName() const
{
	return IsSubsequenceTrack() ? LOCTEXT("SubsequenceTrackName", "Subsequences") : LOCTEXT("ShotTrackName", "Shots");
}

FName UMovieSceneSubAssemblyTrack::GetTrackIconBrush() const
{
	return IsSubsequenceTrack() ? TEXT("Sequencer.Tracks.Sub") : TEXT("Sequencer.Tracks.CinematicShot");
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
