// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSequence.h"

#include "CineAssemblySchema.h"
#include "Dom/JsonObject.h"
#include "UObject/TemplateString.h"

#include "CineAssembly.generated.h"

class UCineAssembly;
class UMovieSceneFolder;
class UMovieSceneSubSection;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnCineAssemblyMetadataChanged, UCineAssembly*, const FString&);

/** 
 * Defines an additional asset that should be created alongside an assembly. These definitions are configured by the Schema.
 */
USTRUCT(BlueprintType)
struct FAssemblyAssociatedAssetDesc
{
	GENERATED_BODY()

	/** The class of asset to create */
	UPROPERTY(BlueprintReadOnly, Category = "Cine Assembly Tools")
	TSoftClassPtr<UObject> AssetClass;

	/** The template name for the associated asset */
	UPROPERTY(BlueprintReadOnly, Category = "Cine Assembly Tools")
	FTemplateString AssetName;

	/** The path relative to the assembly root where this asset should be created */
	UPROPERTY(BlueprintReadOnly, Category = "Cine Assembly Tools")
	FTemplateString RelativePath;

	/** Template asset that will be duplicated when creating an asset for this descriptor. If null, a new asset of type AssetClass will be created. */
	UPROPERTY(BlueprintReadOnly, Category = "Cine Assembly Tools")
	TSoftObjectPtr<UObject> TemplateAsset;

	/** Reference to the asset created from this definition (populated after the asset is created) */
	UPROPERTY(BlueprintReadOnly, Category = "Cine Assembly Tools")
	TSoftObjectPtr<UObject> CreatedAsset;

	/** Stable identifier for this definition */
	UPROPERTY(BlueprintReadOnly, Category = "Cine Assembly Tools")
	FGuid AssetID;

	/** Semantic label for identifying this associated asset */
	UPROPERTY(BlueprintReadOnly, Category = "Cine Assembly Tools")
	FName Label;

	/** If true, the CineAssembly factory will create an asset for this descriptor. If false, it is skipped. */
	UPROPERTY(Transient, BlueprintReadOnly, Category = "Cine Assembly Tools")
	bool bShouldCreate = true;
};

/**
 * A cinematic building block that associates a level sequence with a level
 */
UCLASS(MinimalAPI, BlueprintType, HideCategories=Animation)
class UCineAssembly : public ULevelSequence
{
	GENERATED_BODY()

public:
	UCineAssembly();

	friend class SCineAssemblyConfigWindow;

	//~ Begin UMovieSceneSequence Interface
	CINEASSEMBLYTOOLS_API virtual bool IsCompatibleAsSubSequence(const UMovieSceneSequence& ParentSequence) const override;
	//~ End UMovieSceneSequence Interface

	//~ Begin ULevelSquence Interface
#if WITH_EDITOR
	CINEASSEMBLYTOOLS_API virtual ETrackSupport IsTrackSupportedImpl(TSubclassOf<class UMovieSceneTrack> InTrackClass) const override;
#endif //WITH_EDITOR

	CINEASSEMBLYTOOLS_API virtual void Initialize() override;
	//~ End ULevelSquence Interface

	/** 
	 * Set the Schema for this Assembly and initialize the MovieScene from the Schema's template sequence.
	 * bDuplicateMovieScene determines whether the MovieScene of this Assembly is replaced with a duplicate of the MovieScene from the Schema's TemplateSequence
	 */
	CINEASSEMBLYTOOLS_API void InitializeFromSchema(UCineAssemblySchema* InSchema, bool bDuplicateMovieScene = true);

	/** 
	 * Given a ULevelSequence as a template, initialize this assembly from it.
	 * bDuplicateMovieScene determines whether the MovieScene of this Assembly is replaced with a duplicate of the MovieScene on the input Template LevelSequence
	 */
	CINEASSEMBLYTOOLS_API void InitializeFromTemplate(ULevelSequence * Template, bool bDuplicateMovieScene = true);

	//~ Begin UObject Interface
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	virtual void PostInitProperties() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void PostLoad() override;

#if WITH_EDITOR
	virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	/** Get the unique ID of this assembly */
	CINEASSEMBLYTOOLS_API FGuid GetAssemblyGuid() const;

	/** Get the base schema for this assembly */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API const UCineAssemblySchema* GetSchema() const;

	/** Set the base schema for this assembly, only if one is not already set */
	CINEASSEMBLYTOOLS_API void SetSchema(UCineAssemblySchema* InSchema);

#if WITH_EDITOR
	/** Creates one or more subsequence assets, parented to this assembly, based on the schema */
	UE_DEPRECATED(5.8, "SubAssemblies are created automatically when initializing an assembly with a Schema")
	CINEASSEMBLYTOOLS_API void CreateSubAssemblies();

	/** Creates the content folders specified by the schema */
	UE_DEPRECATED(5.8, "Subfolders are created automatically when the Assembly is created through the asset factory")
	CINEASSEMBLYTOOLS_API void CreateSubFolders();

	/** Resolve any tokens in the names of this Assembly's MovieScene folders and tracks */
	CINEASSEMBLYTOOLS_API void ResolveMovieSceneTokens();
#endif

	/** Get the target level associated with this assembly */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API TSoftObjectPtr<UWorld> GetLevel();

	/** Set the target level associated with this assembly */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void SetLevel(TSoftObjectPtr<UWorld> InLevel);

	/** Get the note text associated with this assembly */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API FString GetNoteText();

	/** Set the note text associated with this assembly */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void SetNoteText(FString InNote);

	/** Append to the note text associated with this assembly */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void AppendToNoteText(FString InNote);

	/** Get the production ID associated with this assembly */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API FGuid GetProductionID();

	/** Get the production name associated with this assembly */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API FString GetProductionName();

	/** Get the parent assembly of this assembly */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API TSoftObjectPtr<UCineAssembly> GetParentAssembly();

	/** Set the parent assembly of this assembly */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void SetParentAssembly(TSoftObjectPtr<UCineAssembly> InParent);

	/** Get the semantic label for this assembly */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API FName GetLabel() const;

	/** Set the semantic label for this assembly */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void SetLabel(FName InLabel);

	/** Get the direct child SubAssemblies of this assembly */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API TArray<UCineAssembly*> GetSubAssemblies() const;

	/** Get the list of Associated Assets configured on this assembly */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API TArray<TSoftObjectPtr<UObject>> GetAssociatedAssets() const;

	/** Get the direct child SubAssemblies of this assembly whose Label matches the input */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API TArray<UCineAssembly*> FindSubAssembliesByLabel(FName InLabel) const;

	/** Get the list of Associated Assets configured on this assembly whose Label matches the input */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API TArray<TSoftObjectPtr<UObject>> FindAssociatedAssetsByLabel(FName InLabel) const;

#if WITH_EDITOR
	/** Gets the author of this assembly */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API FString GetAuthor() const;

	/** Set the author of this assembly */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void SetAuthor(const FString& InAuthor);

	/** Gets the created date-time of this assembly as a localtime formatted string. */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API FString GetCreatedString() const;

	/** Gets the date this assembly was created as a YYYY-MM-DD string. */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API FString GetDateCreatedString() const;

	/** Gets the 24-hour time of day this assembly was created as a HH:MM:SS string. */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API FString GetTimeCreatedString() const;
#endif // WITH_EDITOR

	/** Get all of the metadata for this assembly as a formatted JSON string */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API FString GetFullMetadataString() const;

	/** Get all of the metadata keys for this assembly */
	CINEASSEMBLYTOOLS_API TArray<FString> GetMetadataKeys() const;

	/** Add a string as metadata to this assembly */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void SetMetadataAsString(FString InKey, FString InValue);

	/** Add a tokenized string as metadata to this assembly */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void SetMetadataAsTokenString(FString InKey, FTemplateString InValue);

	/** Add a boolean as metadata to this assembly */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void SetMetadataAsBool(FString InKey, bool InValue);

	/** Add an integer as metadata to this assembly */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void SetMetadataAsInteger(FString InKey, int32 InValue);

	/** Add a floating point number as metadata to this assembly */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void SetMetadataAsFloat(FString InKey, float InValue);

	/** Get the metadata value for the input key as a string (if it exists) */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools", meta = (ReturnDisplayName = "FoundKey"))
	CINEASSEMBLYTOOLS_API bool GetMetadataAsString(FString InKey, FString& OutValue) const;

	/** Get the metadata value for the input key as a token string (if it exists) */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools", meta = (ReturnDisplayName = "FoundKey"))
	CINEASSEMBLYTOOLS_API bool GetMetadataAsTokenString(FString InKey, FTemplateString& OutValue) const;

	/** Get the metadata value for the input key as a boolean (if it exists) */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools", meta = (ReturnDisplayName = "FoundKey"))
	CINEASSEMBLYTOOLS_API bool GetMetadataAsBool(FString InKey, bool& OutValue) const;

	/** Get the metadata value for the input key as an integer (if it exists) */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools", meta = (ReturnDisplayName = "FoundKey"))
	CINEASSEMBLYTOOLS_API bool GetMetadataAsInteger(FString InKey, int32& OutValue) const;

	/** Get the metadata value for the input key as a floating-point number (if it exists) */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools", meta = (ReturnDisplayName = "FoundKey"))
	CINEASSEMBLYTOOLS_API bool GetMetadataAsFloat(FString InKey, float& OutValue) const;

	/**
	 * Apply a map of metadata key-value pairs to this assembly.
	 * Keys matching the schema's AssemblyMetadata are applied with type-awareness based on each field's defined type.
	 * Keys not found in the schema are stored as InstanceMetadata.
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", meta = (AutoCreateRefTerm = "InMetadata"))
	CINEASSEMBLYTOOLS_API void ApplyMetadata(const TMap<FString, FString>& InMetadata);

	/** Adds a new metadata key to the list of supported naming tokens for assemblies */
	CINEASSEMBLYTOOLS_API void AddMetadataNamingToken(const FString& InKey);

	/** Returns the asset path and its root folder (which might be different if the schema defines a default assembly path) */
	CINEASSEMBLYTOOLS_API void GetAssetPathAndRootFolder(FString& OutAssetPath, FString& OutRootFolder);

	/** Removes an associated asset by its AssetID, and cleans up any MetadataLinks that referenced it */
	CINEASSEMBLYTOOLS_API void RemoveAssociatedAsset(const FGuid& InAssetID);

	/** Finds a managed SubAssembly whose AssemblyGuid matches the input ID. Returns nullptr if not found. */
	CINEASSEMBLYTOOLS_API UCineAssembly* FindSubAssembly(const FGuid& InAssemblyID) const;

	/** Removes stale entries from MetadataLinks whose keys or values no longer match any metadata field, associated asset, or SubAssembly section */
	CINEASSEMBLYTOOLS_API void ValidateMetadataLinks();

	/** Recursively duplicates all managed SubAssemblies, replacing each with a new transient copy */
	CINEASSEMBLYTOOLS_API void DuplicateManagedSubAssemblies();

	/**
	 * Convert all SubAssembly Tracks found in this Assembly's MovieScene into SubTracks and CinematicShotTracks.
	 * A new transient SubAssembly will be created for every template SubAssembly Section that is found.
	 */
	CINEASSEMBLYTOOLS_API void ConvertSubAssemblyTracks();

private:
	/** Sets the base schema for this assembly and re-initializes the metadata inherited from the schema */
	CINEASSEMBLYTOOLS_API void ChangeSchema(UCineAssemblySchema* InSchema);

	/**
	 * Set the Schema for this Assembly and initialize the MovieScene from the Schema's template sequence.
	 * If that template sequence had SubAssemblies with template schemas, this will be recursive.
	 * Tracks the visited schemas it has seen so far as it goes to detect cycles (A->A, A->B->A) so it can avoid infinite recursion.
	 */
	void InitializeFromSchemaInternal(UCineAssemblySchema* InSchema, TSet<const UCineAssemblySchema*>& SchemaAncestors, bool bDuplicateMovieScene = true);

	/**
	 * Convert all SubAssembly Tracks found in this Assembly's MovieScene into SubTracks and CinematicShotTracks.
	 * A new transient SubAssembly will be created for every template SubAssembly Section that is found.
	 * Tracks the visited schemas it has seen so far as it goes to detect cycles (A->A, A->B->A) so it can avoid infinite recursion.
	 */
	void ConvertSubAssemblyTracksInternal(TSet<const UCineAssemblySchema*>& SchemaAncestors);

	/** Assigns meaningful, semantic names to the managed transient SubAssemblies using each SubAssembly's label */
	void RenameSubAssemblies();

	/** Creates a transient CineAssembly object using the input object as a template. Threads the visited schemas for cycle detection. */
	UCineAssembly* CreateSubAssemblyFromTemplate(UObject* TemplateObject, TSet<const UCineAssemblySchema*>& SchemaAncestors);

	/** Update the underlying json object whenever keys/values in the instance metadata map are added/removed/modified */
	void UpdateInstanceMetadata();

	/** Get the created FDateTime in local time. If no metadata found, returns an unset optional. */
	TOptional<FDateTime> TryGetCreatedAsLocalTime() const;

#if WITH_EDITOR
	/** Recursively gather the movie scene folders within a root folder */
	void GetAllMovieSceneFoldersRecursive(UMovieSceneFolder* RootFolder, TArray<UMovieSceneFolder*>& OutFolders);
#endif // WITH_EDITOR

public:
	/** The assembly name, which supports tokens */
	UPROPERTY()
	FTemplateString AssemblyName;

	/** The path where this assembly should be created, relative to its content root */
	UPROPERTY()
	FTemplateString PathRelativeToRoot;

	/** The path where this assembly should be created, relative to its parent assembly */
	UPROPERTY()
	FTemplateString PathRelativeToParent;

	/** If this assembly is a duplicate of another, this represents the absolute path of the original asset */
	FString SourceAssemblyPath;

	/** If this assembly was duplicated from another, this holds a soft reference to the source. */
	UPROPERTY(BlueprintReadOnly, Category = "Cine Assembly Tools")
	TSoftObjectPtr<UCineAssembly> SourceAssembly;

	/** The level to open before opening this asset in Sequencer */
	UPROPERTY(EditAnywhere, Category = "Default", meta = (AllowedClasses = "/Script/Engine.World"))
	FSoftObjectPath Level;

	/** User added metadata key/value pairs, which will be added as additional asset registry tags */
	UPROPERTY(EditAnywhere, Category = "Instance Metadata")
	TMap<FName, FString> InstanceMetadata;

	/** Semantic label for identifying this assembly */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default")
	FName Label;

	/** User-facing notes about this assembly asset */
	UPROPERTY()
	FString AssemblyNote;

	/** Reference to another assembly asset that is the parent of this assembly */
	UPROPERTY(EditAnywhere, Category = "Default", meta = (AllowedClasses = "/Script/CineAssemblyTools.CineAssembly"))
	FSoftObjectPath ParentAssembly;

	/** The ID of the Cinematic Production that this assembly is associated with */
	UPROPERTY(EditAnywhere, Category = "Default")
	FGuid Production;

	/** The name of the Cinematic Production that this assembly is associated with */
	UPROPERTY()
	FString ProductionName;

	/** Array of template names (possibly containing tokens), based on the schema, used to create the SubAssemblies */
	UPROPERTY()
	TArray<FTemplateString> SubAssemblyNames;

	/** Array of template names (possibly containing tokens), based on the schema, used to create folders for this assembly */
	UPROPERTY()
	TArray<FTemplateString> DefaultFolderNames;

	/** Array of associated asset definitions, used to create additional assets alongside this assembly */
	UPROPERTY()
	TArray<FAssemblyAssociatedAssetDesc> AssociatedAssets;

	/** Links metadata keys to AssociatedAssets and SubAssemblies to auto-populate the value of the linked metadata field when the asset is created */
	UPROPERTY()
	TMap<FString, FGuid> MetadataLinks;

	/** Array of Subsequence Sections created based on the schema */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSubSection>> SubAssemblies;

	/** Mark this CineAssembly as Data-Only. When Data-Only, opening a CineAssembly opens only it's details. */
	UPROPERTY(AssetRegistrySearchable, BlueprintReadWrite, EditAnywhere, Category="Default", AdvancedDisplay, meta=(DisplayName="Data-Only"))
	bool bIsDataOnly = false;

	/** If true, the CineAssembly factory will create this Assembly. If false, it (and its SubAssemblies and Associated Assets) is skipped. */
	UPROPERTY(Transient, BlueprintReadWrite, Category = "Default")
	bool bShouldCreate = true;

	/** The asset registry tag that contains the assembly type information */
	static CINEASSEMBLYTOOLS_API const FName AssetRegistryTag_AssemblyType;

	/** The asset registry tag that contains the value of the associated Schema ID */
	static CINEASSEMBLYTOOLS_API const FName AssetRegistryTag_SchemaGuid;

	static CINEASSEMBLYTOOLS_API const FName AssemblyGuidPropertyName;

	/** Native multicast event fired after a metadata field is added or updated on a CineAssembly. */
	static CINEASSEMBLYTOOLS_API FOnCineAssemblyMetadataChanged& OnAssemblyMetadataChanged();

private:
	/** Unique ID for this assembly, assigned at object creation */
	UPROPERTY(AssetRegistrySearchable, meta = (IgnoreForMemberInitializationTest))
	FGuid AssemblyGuid;

	/** The schema that was used as a base when creating this assembly (can be null if no schema was used) */
	UPROPERTY()
	TObjectPtr<UCineAssemblySchema> BaseSchema;

	/** Copy of the keys present in the InstanceMetadata map, used to keep the json representation consistent with the map contents */
	TArray<FName> InstanceMetadataKeys;

	/** Json object responsible for storing the schema and instance metadata for this assembly */
	TSharedPtr<FJsonObject> MetadataJsonObject;
};
