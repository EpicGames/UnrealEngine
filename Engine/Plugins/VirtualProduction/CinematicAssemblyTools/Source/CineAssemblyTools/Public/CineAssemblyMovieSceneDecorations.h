// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "CineAssemblyMovieSceneDecorations.generated.h"

/**
 * MovieScene decoration used to tag Tracks (and, transitively, the bindings those tracks might belong to) that came from a Schema's TemplateSequence
 */
UCLASS(MinimalAPI)
class UCineAssemblySchemaContentDecoration : public UObject
{
	GENERATED_BODY()
};

/**
 * MovieScene decoration used to tag UMovieSceneSubSections whose outer range matches the playback range of the Schema's TemplateSequence
 */
UCLASS(MinimalAPI)
class UCineAssemblyFullRangeDecoration : public UObject
{
	GENERATED_BODY()
};
