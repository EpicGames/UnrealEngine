// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NamingTokens.h"
#include "CineAssembly.h"

#include "CineAssemblyNamingTokens.generated.h"

/** Context object that lets callers supply an assembly asset when evaluating tokens */
UCLASS()
class UCineAssemblyNamingTokensContext : public UObject
{
	GENERATED_BODY()

public:
	/** The assembly asset to use when evaluating tokens */
	TWeakObjectPtr<UCineAssembly> Assembly;
};

/* Naming Tokens related to Cinematic Assemblies */
UCLASS(MinimalAPI, NotBlueprintable)
class UCineAssemblyNamingTokens : public UNamingTokens
{
	GENERATED_BODY()

protected:
	// ~Begin UNamingTokens
	virtual void OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens) override;
	virtual void OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData) override;
	virtual void OnPostEvaluate_Implementation() override;
	// ~End UNamingTokens

public:

	UCineAssemblyNamingTokens();

	static CINEASSEMBLYTOOLS_API FString TokenNamespace;

	/** Utility function that will evaluate the input string with the naming tokens subsystem and return the result evaluated text using the input assembly */
	static CINEASSEMBLYTOOLS_API FText GetResolvedText(const FString& InStringToEvaluate, UCineAssembly* InAssembly);

	/** Adds a metadata token with the same name as the input key which will return the result of value of that metadata if found in the Cine Assembly */
	CINEASSEMBLYTOOLS_API void AddMetadataToken(const FString& InTokenKey);

private:
	/** Runs the input Token evaluation function, which takes a Cine Assembly as input and outputs FText */
	FText ExecuteTokenFunc(TFunction<FText(TWeakObjectPtr<UCineAssembly>)> TokenFunc);

private:
	/** The current context to use when evaluating tokens */
	TObjectPtr<UCineAssemblyNamingTokensContext> Context;
};
