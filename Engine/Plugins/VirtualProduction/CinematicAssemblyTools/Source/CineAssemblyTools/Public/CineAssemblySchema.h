// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture2D.h"
#include "LevelSequence.h"
#include "Styling/SlateBrush.h"

#include "CineAssemblySchema.generated.h"

class UCineAssembly;

/** The types of assembly metadata supported by Cine Assembly Schemas */
UENUM()
enum class ECineAssemblyMetadataType : uint8
{
	String = 0,
	Bool,
	Integer,
	Float,
	AssetPath,
	CineAssembly
};

/** Structure defining a single metadata field that can be associated with an assembly built from this schema, including its type, key, and default value */
USTRUCT()
struct FAssemblyMetadataDesc
{
	GENERATED_BODY()

	/** Metadata type */
	UPROPERTY(EditAnywhere, Category = "Metadata")
	ECineAssemblyMetadataType Type = ECineAssemblyMetadataType::String;

	/** The key associated with this field */
	UPROPERTY(EditAnywhere, Category = "Metadata")
	FString Key;

	/** For AssetPath types, the class to restrict the value of this metadata field to */
	UPROPERTY(EditAnywhere, Category = "Metadata", meta = (AllowAbstract = false, HideViewOptions, EditCondition = "Type == ECineAssemblyMetadataType::AssetPath", EditConditionHides))
	FSoftClassPath AssetClass;

	/** For CineAssembly types, the schema type to restrict the value of this metadata field to */
	UPROPERTY(EditAnywhere, Category = "Metadata", meta = (AllowedClasses="/Script/CineAssemblyTools.CineAssemblySchema", EditCondition = "Type == ECineAssemblyMetadataType::CineAssembly", EditConditionHides))
	FSoftObjectPath SchemaType;

	/** The default value for this metadata field */
	TVariant<FString, bool, int32, float> DefaultValue;

	/** If string field types should be evaluated as template token strings. */
	UPROPERTY(EditAnywhere, Category = "Metadata", meta = (EditCondition = "Type == ECineAssemblyMetadataType::String", EditConditionHides))
	bool bEvaluateTokens = false;
};

/**
 * A template object for building different Cine Assembly types
 */
UCLASS(MinimalAPI, BlueprintType)
class UCineAssemblySchema : public UObject
{
	GENERATED_BODY()

public:
	UCineAssemblySchema();

	//~ Begin UObject overrides
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostInitProperties() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void PostLoad() override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	//~ End UObject overrides

	/** Get the unique ID of this schema */
	CINEASSEMBLYTOOLS_API FGuid GetSchemaGuid() const;

	/** Get the default name for a new Assembly made from this schema */
	CINEASSEMBLYTOOLS_API FString GetDefaultAssemblyName() const;

	/** Get the default path (relative to the content root) for assemblies created from this schema */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API FString GetDefaultAssemblyPath() const;

	/** Set the default path (relative to the content root) for assemblies created from this schema */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void SetDefaultAssemblyPath(const FString& InPath);

	/** Get the default level for assemblies created from this schema */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API TSoftObjectPtr<UWorld> GetDefaultLevel() const;

	/** Set the default level for assemblies created from this schema */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void SetDefaultLevel(TSoftObjectPtr<UWorld> InLevel);

	/** Clear the default level for assemblies created from this schema */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void ClearDefaultLevel();

	/** Get the list of default folder paths that will be created for assemblies made from this schema */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API TArray<FString> GetDefaultFolders() const;

	/**
	 * Add a default folder path that will be created for assemblies made from this schema.
	 * Also adds all intermediate parent paths (e.g. adding "A/B/C" will add "A", "A/B", and "A/B/C").
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API void AddDefaultFolder(const FString& FolderPath);

	/** Remove a default folder path and all of its descendant folder paths. Returns true if any folders were found and removed. */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	CINEASSEMBLYTOOLS_API bool RemoveDefaultFolder(const FString& FolderPath);

	/** Whether or not this schema can be renamed */
	CINEASSEMBLYTOOLS_API bool SupportsRename() const;

#if WITH_EDITOR
	/** Renames the underlying schema asset */
	CINEASSEMBLYTOOLS_API void RenameAsset(const FString& InNewName);
#endif

	/** Returns the list of metadata descriptors for linkable assembly properties */
	CINEASSEMBLYTOOLS_API const TArray<FAssemblyMetadataDesc>& GetAssemblyPropertyMetadata() const;

	/** Finds the metadata desc matching the input key, searching AssemblyMetadata and AssemblyPropertyMetadata */
	CINEASSEMBLYTOOLS_API const FAssemblyMetadataDesc* FindMetadataDesc(const FString& InMetadataKey) const;

	/** Iterates over each metadata desc in AssemblyMetadata and AssemblyPropertyMetadata */
	CINEASSEMBLYTOOLS_API void ForEachMetadataDesc(TFunctionRef<bool(const FAssemblyMetadataDesc&)> Visitor) const;

	/**
	 * Returns a unique Label of the form "{BaseName}{N}" for a new SubAssembly template section in this schema's TemplateSequence.
	 * Uniqueness is enforced against existing UMovieSceneSubAssemblySection labels in this schema's TemplateSequence.
	 */
	CINEASSEMBLYTOOLS_API FName MakeUniqueAssemblyLabel(const FString& BaseName) const;

	/**
	 * Returns a unique Label of the form "{BaseName}{N}" for a new Associated Asset in this schema's TemplateSequence.
	 * Uniqueness is enforced against existing Associated Asset labels in this schema's TemplateSequence.
	 */
	CINEASSEMBLYTOOLS_API FName MakeUniqueAssetLabel(const FString& BaseName) const;

public:
	/** The schema name, which will be used by assemblies made from this schema as their "assembly type" */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default")
	FString SchemaName;

	/** A user-facing text description of this schema */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default")
	FString Description;

	/** The default name to be use when creating assemblies from this schema */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default")
	FString DefaultAssemblyName;

	/** Restricts assemblies made from this schema to using this Schema when picking a Parent Assembly */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default", meta = (AllowedClasses = "/Script/CineAssemblyTools.CineAssemblySchema"))
	FSoftObjectPath ParentSchema;

	/** The thumbnail image to use for this schema and assemblies built from this schema */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default", meta = (DisplayName = "Thumbnail Image"))
	TSoftObjectPtr<UTexture2D> ThumbnailTexture;

	/**
	 * If checked, the selected override will be used to initialize the Level property for a new Assembly made from this Schema.
	 * If unchecked, the default Level for new Assemblies will be the currently open level when the Assembly is created.
	 */
	UPROPERTY(EditAnywhere, Category = "Assembly Defaults", meta = (AllowedClasses = "/Script/Engine.World", EditCondition = "bOverrideDefaultLevel"))
	FSoftObjectPath DefaultLevel;

	/** Whether assemblies created from this schema should use the DefaultLevel */
	UPROPERTY(EditAnywhere, Category = "Assembly Defaults", meta = (InlineEditConditionToggle))
	bool bOverrideDefaultLevel = false;

	/** List of metadata fields that should be automatically added to assemblies made from this schema */
	UPROPERTY(EditAnywhere, Category = "Metadata")
	TArray<FAssemblyMetadataDesc> AssemblyMetadata;

	/** Template sequence, owned and managed by the schema, to use when creating the assembly as a starting point.  */
	UPROPERTY(BlueprintReadOnly, Category = "Default")
	TObjectPtr<UCineAssembly> TemplateSequence;

	/** Mark this Schema as Data-Only. Assemblies made from this Schema will have their Data-Only defaulted to this value. */
	UPROPERTY(AssetRegistrySearchable, BlueprintReadWrite, EditAnywhere, Category = "Default", AdvancedDisplay, meta = (DisplayName = "Data-Only"))
	bool bIsDataOnly = false;

	/** Hidden schemas will not appear in the UI when creating new Cine Assemblies. */
	UPROPERTY(AssetRegistrySearchable, BlueprintReadWrite, EditAnywhere, Category = "Default", AdvancedDisplay, meta = (DisplayName = "Hidden"))
	bool bIsHidden = false;

	/** Metadata key identifying the schema's DefaultLevel as a linkable target. */
	static CINEASSEMBLYTOOLS_API const FString DefaultLevelMetadataKey;

	/** Asset registry tag holding the soft object path of the schema's thumbnail texture, so thumbnail display can resolve without loading the schema. */
	static CINEASSEMBLYTOOLS_API const FName AssetRegistryTag_ThumbnailTexture;

	static CINEASSEMBLYTOOLS_API const FName SchemaGuidPropertyName;

protected:
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "ThumbnailImage is deprecated. Use ThumbnailTexture instead (now a soft reference).")
	UPROPERTY()
	TObjectPtr<UTexture2D> ThumbnailImage_DEPRECATED;

	UE_DEPRECATED(5.8, "Subsequences to create are now represented by the SubAssembly sections of the TemplateSequence.")
	UPROPERTY()
	TArray<FString> SubsequencesToCreate_DEPRECATED;

	UE_DEPRECATED(5.8, "Use the TemplateSequence owned by the Schema to initialize a new Assembly.")
	UPROPERTY()
	FSoftObjectPath Template_DEPRECATED;

	UE_DEPRECATED(5.8, "Subsequence Template objects are now stored in the SubAssembly sections of the TemplateSequence.")
	UPROPERTY()
	TMap<FString, FSoftObjectPath> SubsequenceTemplates_DEPRECATED;

	UE_DEPRECATED(5.8, "This property is now managed directly by the TemplateSequence's PathRelativeToRoot property")
	UPROPERTY()
	FString DefaultAssemblyPath_DEPRECATED;

	UE_DEPRECATED(5.8, "This property is now managed directly by the TemplateSequence's DefaultFolderNames property")
	UPROPERTY()
	TArray<FString> FoldersToCreate_DEPRECATED;
#endif // WITH_EDITORONLY_DATA 

private:
	/** List of metadata fields that refer to specific Assembly properties */
	TArray<FAssemblyMetadataDesc> AssemblyPropertyMetadata;

	/** Unique ID for this schema, assigned at object creation */
	UPROPERTY(AssetRegistrySearchable, meta = (IgnoreForMemberInitializationTest))
	FGuid SchemaGuid;

	/** Whether the schema asset supports renaming */
	bool bSupportsRename = false;
};
