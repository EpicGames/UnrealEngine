// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sections/MovieSceneCinematicShotSection.h"

#include "MovieSceneSubAssemblySection.generated.h"

class UCineAssemblySchema;

#define UE_API CINEASSEMBLYTOOLS_API

/** Indicates how a SubAssemblySection will be used */
UENUM(BlueprintType)
enum class ESubAssemblySectionType : uint8
{
	Reference, /** Section contains a reference to an existing sequence */
	Template   /** Section contains a template object used to create a new sequence */
};

/**
 * The UMovieSceneSubAssemblySection is only supported in the Template Assembly of a CineAssemblySchema as a placeholder section.
 * The Track containing this section has a TrackType that indicates which section should be created when creating an Assembly from a Schema Template.
 */
UCLASS(BlueprintType, MinimalAPI)
class UMovieSceneSubAssemblySection : public UMovieSceneCinematicShotSection
{
	GENERATED_BODY()

public:
	UMovieSceneSubAssemblySection(const FObjectInitializer& ObjectInitializer);

	//~ Begin UObject interface
	virtual void PostInitProperties() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
	//~ End UObject interface

	/** Get the unique identifier for this section */
	UE_API FGuid GetSectionID() const;

	/** Returns the schema associated with this section's template. Returns nullptr if no schema can be determined. */
	UE_API const UCineAssemblySchema* GetTemplateSchema() const;

	/** Returns true if this section is used as a reference to an existing sequence */
	UE_API bool IsReferenceSection() const;

	/** Returns true if this section is used as a template to create a new sequence */
	UE_API bool IsTemplateSection() const;

	/** Returns true if this section is a placeholder for a UMovieSceneSubSection */
	UE_API bool IsSubsequenceSection() const;

	/** Returns true if this section is a placeholder for a UMovieSceneCinematicShotSection */
	UE_API bool IsCinematicShotSection() const;

	/** Returns the Assembly Template object */
	UFUNCTION(BlueprintPure, Category = "Cine Assembly Tools")
	UE_API UObject* GetAssemblyTemplate() const;

	/** Sets the Assembly Template and updates the SequenceName to match the new template */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	UE_API void SetAssemblyTemplate(UPARAM(meta = (AllowedClasses = "/Script/LevelSequence.LevelSequence, /Script/CineAssemblyTools.CineAssembly, /Script/CineAssemblyTools.CineAssemblySchema")) UObject* TemplateObject);

	/** Get the name to use when creating a new SubAssembly for this section */
	UE_API FText GetSequenceName() const;

	/** Set the name to use when creating a new SubAssembly for this section */
	UE_API void SetSequenceName(const FText& InName);

	/** Get the relative path to use when creating a new SubAssembly for this section */
	UE_API FString GetSequencePath() const;

	/** Set the relative path to use when creating a new SubAssembly for this section */
	UE_API void SetSequencePath(const FString& InPath);

	/** Sets a unique label based on the template assembly name to use when creating a new SubAssembly for this section */
	UE_API void SetDefaultLabel();

public:
	/** Indicates whether this section will be used as a reference to an existing sequence or as a template to create a new sequence */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sequence")
	ESubAssemblySectionType SectionType = ESubAssemblySectionType::Reference;

	/** The name that will be used when creating a new sequence for this section. Only applicable to Template sections */
	UPROPERTY(BlueprintReadWrite, Category = "Cine Assembly Tools")
	FText NewSequenceName;

	/** The path (relative to the root assembly's path) where the new sequence asset for this sections should be created */
	UPROPERTY(BlueprintReadWrite, Category = "Cine Assembly Tools")
	FString NewSequencePath;

	/**
	 * Metadata key-value pairs, used to override the default values of a SubAssembly created for this section.
	 * Keys should match metadata field keys defined by this section's template schema.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Cine Assembly Tools")
	TMap<FString, FString> MetadataOverrides;

	/** Semantic label for identifying the SubAssembly created from this section. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sequence", meta = (EditCondition = "SectionType == ESubAssemblySectionType::Template", EditConditionHides))
	FName Label;

	/** Name of the protected AssemblyTemplate property */
	static UE_API const FName AssemblyTemplatePropertyName;

	static const FColor ReferenceSectionColorTint;
	static const FColor TemplateSectionColorTint;

protected:
	/** The object that will be used as a template to create a new sequence in this section. Can be a Level Sequence, Cine Assembly, or Schema */
	UPROPERTY(EditAnywhere, Category = "Sequence")
	TObjectPtr<UObject> AssemblyTemplate;

	/** Unique identifier for this section */
	UPROPERTY()
	FGuid SectionID;
};

#undef UE_API
