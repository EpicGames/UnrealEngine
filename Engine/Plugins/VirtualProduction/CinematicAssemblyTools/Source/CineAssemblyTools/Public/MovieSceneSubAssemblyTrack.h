// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tracks/MovieSceneSubTrack.h"

#include "MovieSceneSubAssemblyTrack.generated.h"

#define UE_API CINEASSEMBLYTOOLS_API

/** 
 * Indicates the underlying track type for which this track is a placeholder
 */
UENUM(BlueprintType)
enum class ESubAssemblyTrackType : uint8
{
	SubsequenceTrack,  /** This track is a placeholder for a UMovieSceneSubTrack */
	CinematicShotTrack /** This track is a placeholder for a UMovieSceneCinematicShotTrack */
};

/**
 * The UMovieSceneSubAssemblyTrack is only supported in the Template Assembly of a CineAssemblySchema as a placeholder track.
 * The Track Type indicates which underlying track should be created in place of this track when creating an Assembly from a Schema Template.
 */
UCLASS(BlueprintType, MinimalAPI)
class UMovieSceneSubAssemblyTrack : public UMovieSceneSubTrack
{
	GENERATED_BODY()

public:
	UMovieSceneSubAssemblyTrack(const FObjectInitializer& ObjectInitializer);

	//~ Begin UMovieSceneTrack interface
	UE_API virtual UMovieSceneSection* CreateNewSection() override;
	//~ End UMovieSceneTrack interface

	//~ Begin UMovieSceneNameableTrack interface
#if WITH_EDITOR
	UE_API virtual FText GetDefaultDisplayName() const override;
#endif // WITH_EDITOR
	//~ End UMovieSceneNameableTrack interface

	/** Returns true if this track represents a UMovieSceneSubTrack */
	UE_API bool IsSubsequenceTrack() const;

	/** Returns true if this track represents a UMovieSceneCinematicShotTrack */
	UE_API bool IsCinematicShotTrack() const;

#if WITH_EDITOR
	/** Get the icon brush for this track, determined by the track type */
	UE_API FName GetTrackIconBrush() const;
#endif // WITH_EDITOR

	/** Indicates the underlying track type for which this track is a placeholder */
	UPROPERTY(BlueprintReadWrite, Category = "Cine Assembly Tools")
	ESubAssemblyTrackType TrackType = ESubAssemblyTrackType::CinematicShotTrack;
};

#undef UE_API
