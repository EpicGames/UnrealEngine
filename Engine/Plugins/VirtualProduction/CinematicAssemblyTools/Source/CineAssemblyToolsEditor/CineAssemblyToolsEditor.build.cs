// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class CineAssemblyToolsEditor : ModuleRules
	{
		public CineAssemblyToolsEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ApplicationCore",
					"AssetDefinition",
					"AssetRegistry",
					"AssetTools",
					"CineAssemblyTools",
					"CinematicCamera",
					"ClassViewer",
					"ContentBrowser",
					"ContentBrowserData",
					"Core",
					"CoreUObject",
					"DesktopPlatform",
					"DeveloperSettings",
					"DirectoryPlaceholder",
					"EditorSubsystem",
					"EditorWidgets",
					"Engine",
					"InputCore",
					"Json",
					"JsonUtilities",
					"LevelSequence",
					"LevelSequenceEditor",
					"MovieRenderPipelineCore",
					"MovieScene",
					"MovieSceneTools",
					"MovieSceneTracks",
					"NamingTokens",
					"NamingTokensUI",
					"Projects",
					"PropertyEditor",
					"Sequencer",
					"SequencerCore",
					"SharedSettingsWidgets",
					"Slate",
					"SlateCore",
					"SourceControl",
					"StructUtilsEditor",
					"TimeManagement",
					"TakeRecorder",
					"TakesCore",
					"ToolMenus",
					"ToolWidgets",
					"UniversalObjectLocator",
					"UnrealEd",
					"WorkspaceMenuStructure",
				}
			);
		}
	}
}
