// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_CineAssembly.h"
#include "AssetToolsModule.h"
#include "CineAssembly.h"
#include "CineAssemblyAssetTags.h"
#include "CineAssemblySchema.h"
#include "CineAssemblyToolsStyle.h"
#include "CineAssemblyEditorFunctionLibrary.h"
#include "CineAssemblyToolsEditorModule.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FText UAssetDefinition_CineAssembly::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_CineAssembly", "Cine Assembly");
}

FText UAssetDefinition_CineAssembly::GetAssetDisplayName(const FAssetData& AssetData) const
{
	const FAssetData SchemaAssetData = UE::CineAssemblyTools::Private::GetSchemaAssetData(AssetData);
	if (SchemaAssetData.IsValid())
	{
		return FText::FromName(SchemaAssetData.AssetName);
	}

	// Fallback to the older AssemblyType tag
	const FAssetDataTagMapSharedView::FFindTagResult AssemblyType = AssetData.TagsAndValues.FindTag(UCineAssembly::AssetRegistryTag_AssemblyType);
	if (AssemblyType.IsSet())
	{
		return FText::FromString(AssemblyType.GetValue());
	}

	return GetAssetDisplayName();
}

FText UAssetDefinition_CineAssembly::GetAssetDescription(const FAssetData& AssetData) const
{
	if (!AssetData.IsAssetLoaded())
	{
		return LOCTEXT("AssetDescription_NotLoaded", "Load this Cine Assembly to preview its notes");
	}

	if (const UCineAssembly* const CineAssembly = Cast<UCineAssembly>(AssetData.GetAsset()))
	{
		if (!CineAssembly->AssemblyNote.IsEmpty())
		{
			return FText::FromString(CineAssembly->AssemblyNote);
		}
	}

	return FText::GetEmpty();
}

TSoftClassPtr<> UAssetDefinition_CineAssembly::GetAssetClass() const
{
	return UCineAssembly::StaticClass();
}

FLinearColor UAssetDefinition_CineAssembly::GetAssetColor() const
{
	return FColor(229, 45, 113);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_CineAssembly::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { UE::CineAssemblyTools::CineAssemblyAssetCategory };
	return Categories;
}

const FSlateBrush* UAssetDefinition_CineAssembly::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	using namespace UE::CineAssemblyTools::Private;

	FCineAssemblyToolsStyle& Style = FCineAssemblyToolsStyle::Get();
	const FSlateBrush* DefaultBrush = Style.GetBrush("ClassThumbnail.CineAssembly");

	const FAssetData SchemaAssetData = GetSchemaAssetData(InAssetData);
	if (!SchemaAssetData.IsValid())
	{
		return DefaultBrush;
	}

	const FGuid SchemaGuid = GetSchemaID(SchemaAssetData, UCineAssemblySchema::SchemaGuidPropertyName);
	if (!SchemaGuid.IsValid())
	{
		return DefaultBrush;
	}

	const FName ThumbnailBrushName = Style.GetThumbnailBrushNameForSchema(SchemaGuid);
	const FSlateBrush* ThumbnailBrush = Style.GetOptionalBrush(ThumbnailBrushName, nullptr, nullptr);

	if (!ThumbnailBrush)
	{
		const FSoftObjectPath ThumbnailPath = GetSchemaThumbnailPath(SchemaAssetData);
		if (ThumbnailPath.IsNull())
		{
			return DefaultBrush;
		}

		UTexture2D* Texture = Cast<UTexture2D>(ThumbnailPath.TryLoad());
		Style.SetThumbnailBrushTextureForSchema(SchemaGuid, Texture);
		ThumbnailBrush = Style.GetBrush(ThumbnailBrushName);
	}

	return ThumbnailBrush;
}

EAssetCommandResult UAssetDefinition_CineAssembly::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<FAssetData> DataOnlyAssetData;
	TArray<FAssetData> FullAssetData;
	
	const FName IsDataOnlyAssetRegistrySearchablePropertyName = GET_MEMBER_NAME_CHECKED(UCineAssembly, bIsDataOnly);

	for (const FAssetData& AssetData : OpenArgs.Assets)
	{
		bool bAssemblyIsDataOnly;
		AssetData.GetTagValue(IsDataOnlyAssetRegistrySearchablePropertyName, bAssemblyIsDataOnly);

		if (bAssemblyIsDataOnly)
		{
			DataOnlyAssetData.Add(AssetData);
		}
		else
		{
			FullAssetData.Add(AssetData);
		}
	}

	FAssetOpenArgs DataOnlyOpenArgs(OpenArgs);
	DataOnlyOpenArgs.Assets = TConstArrayView<FAssetData>(DataOnlyAssetData);
	FAssetOpenArgs FullAssetOpenArgs(OpenArgs);
	FullAssetOpenArgs.Assets = TConstArrayView<FAssetData>(FullAssetData);

	if (!DataOnlyOpenArgs.Assets.IsEmpty())
	{
		UAssetDefinitionDefault::OpenAssets(DataOnlyOpenArgs);
	}

	return Super::OpenAssets(FullAssetOpenArgs);
}

TArray<FAssetData> UAssetDefinition_CineAssembly::PrepareToActivateAssets(const FAssetActivateArgs& ActivateArgs) const
{
	TArray<FAssetData> AssetsToOpen;

	// We only support opening one asset at a time
	if (ActivateArgs.Assets.Num() == 1)
	{
		const FAssetData& CineAssemblyData = ActivateArgs.Assets[0];

		UCineAssembly* CineAssembly = Cast<UCineAssembly>(CineAssemblyData.GetAsset());

		// Attempt to open the level associated with this assembly (if it has one). If this operation fails, then the assembly should not be opened. 
		if (UCineAssemblyEditorFunctionLibrary::OpenAssociatedLevel(CineAssembly))
		{
			AssetsToOpen.Add(CineAssemblyData);
		}
	}

	return AssetsToOpen;
}

#undef LOCTEXT_NAMESPACE
