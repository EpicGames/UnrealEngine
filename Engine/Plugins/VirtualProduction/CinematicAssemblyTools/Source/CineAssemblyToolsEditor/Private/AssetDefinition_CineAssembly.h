// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTools/AssetDefinition_LevelSequence.h"

#include "AssetDefinition_CineAssembly.generated.h"

/** Asset definition for a UCineAssembly asset */
UCLASS()
class UAssetDefinition_CineAssembly : public UAssetDefinition_LevelSequence
{
	GENERATED_BODY()

protected:
	// Begin UAssetDefinition Interface
	virtual FText GetAssetDisplayName() const override;
	virtual FText GetAssetDisplayName(const FAssetData& AssetData) const override;
	virtual FText GetAssetDescription(const FAssetData& AssetData) const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual TArray<FAssetData> PrepareToActivateAssets(const FAssetActivateArgs& ActivateArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// End UAssetDefinition Interface
};
