// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_CineAssemblySchema.h"

#include "CineAssemblyAssetTags.h"
#include "CineAssemblySchema.h"
#include "CineAssemblyToolsEditorModule.h"
#include "CineAssemblyToolsStyle.h"
#include "UI/CineAssembly/SCineAssemblySchemaWindow.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

TSoftClassPtr<> UAssetDefinition_CineAssemblySchema::GetAssetClass() const
{
	return UCineAssemblySchema::StaticClass();
}

FText UAssetDefinition_CineAssemblySchema::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_CineAssemblySchema", "Cine Assembly Schema");
}

FLinearColor UAssetDefinition_CineAssemblySchema::GetAssetColor() const
{
	return FColor(176, 58, 104);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_CineAssemblySchema::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { UE::CineAssemblyTools::CineAssemblyAssetCategory };
	return Categories;
}

const FSlateBrush* UAssetDefinition_CineAssemblySchema::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	using namespace UE::CineAssemblyTools::Private;

	FCineAssemblyToolsStyle& Style = FCineAssemblyToolsStyle::Get();
	const FSlateBrush* DefaultBrush = Style.GetBrush("ClassThumbnail.CineAssemblySchema");

	const FGuid SchemaID = GetSchemaID(InAssetData, UCineAssemblySchema::SchemaGuidPropertyName);
	if (!SchemaID.IsValid())
	{
		return DefaultBrush;
	}

	const FName ThumbnailBrushName = Style.GetThumbnailBrushNameForSchema(SchemaID);
	const FSlateBrush* ThumbnailBrush = Style.GetOptionalBrush(ThumbnailBrushName, nullptr, nullptr);

	if (!ThumbnailBrush)
	{
		const FSoftObjectPath ThumbnailPath = GetSchemaThumbnailPath(InAssetData);
		if (ThumbnailPath.IsNull())
		{
			return DefaultBrush;
		}

		UTexture2D* Texture = Cast<UTexture2D>(ThumbnailPath.TryLoad());
		Style.SetThumbnailBrushTextureForSchema(SchemaID, Texture);
		ThumbnailBrush = Style.GetBrush(ThumbnailBrushName);
	}

	return ThumbnailBrush;
}

FAssetSupportResponse UAssetDefinition_CineAssemblySchema::CanRename(const FAssetData& InAsset) const
{
	if (UCineAssemblySchema* CineAssemblySchema = Cast<UCineAssemblySchema>(InAsset.GetAsset()))
	{
		if (!CineAssemblySchema->SupportsRename())
		{
			return FAssetSupportResponse::NotSupported();
		}
	}

	return FAssetSupportResponse::Supported();
}

EAssetCommandResult UAssetDefinition_CineAssemblySchema::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (const FAssetData& SchemaAssetData : OpenArgs.Assets)
	{
		if (UCineAssemblySchema* CineAssemblySchema = Cast<UCineAssemblySchema>(SchemaAssetData.GetAsset()))
		{
			FCineAssemblyToolsEditorModule& CineAssemblyToolsEditorModule = FModuleManager::GetModuleChecked<FCineAssemblyToolsEditorModule>("CineAssemblyToolsEditor");
			CineAssemblyToolsEditorModule.OpenSchemaForEdit(CineAssemblySchema);
		}
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
