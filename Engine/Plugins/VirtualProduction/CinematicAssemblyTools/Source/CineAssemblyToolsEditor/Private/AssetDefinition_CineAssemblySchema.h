// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinition.h"

#include "AssetDefinition_CineAssemblySchema.generated.h"

/** Asset definition for a UCineAssemblySchema asset */
UCLASS()
class UAssetDefinition_CineAssemblySchema : public UAssetDefinition
{
	GENERATED_BODY()

protected:
	// Begin UAssetDefinition Interface
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual FAssetSupportResponse CanRename(const FAssetData& InAsset) const override;
	// End UAssetDefinition Interface
};
