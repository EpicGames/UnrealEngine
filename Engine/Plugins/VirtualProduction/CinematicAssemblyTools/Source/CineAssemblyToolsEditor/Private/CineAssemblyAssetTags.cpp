// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblyAssetTags.h"

#include "Algo/Find.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CineAssembly.h"
#include "CineAssemblySchema.h"
#include "Containers/Set.h"
#include "HAL/CriticalSection.h"
#include "Logging/LogMacros.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogCineAssemblyAssetTags, Log, All)

namespace UE::CineAssemblyTools::Private
{
	void LogMissingSchemaThumbnailTagOnce(FName SchemaPackageName)
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
		if (AssetRegistryModule.Get().IsLoadingAssets())
		{
			return;
		}

		static FCriticalSection Mutex;
		static TSet<FName> LoggedSchemas;

		FScopeLock Lock(&Mutex);
		if (!LoggedSchemas.Contains(SchemaPackageName))
		{
			LoggedSchemas.Add(SchemaPackageName);
			UE_LOGF(LogCineAssemblyAssetTags, Log, "Schema '%ls' needs to be resaved so that its thumbnail texture can be read without loading the asset.", *SchemaPackageName.ToString());
		}
	}

	FGuid GetSchemaID(const FAssetData& AssetData, FName TagName)
	{
		FString SchemaIDTagValue;
		AssetData.GetTagValue(TagName, SchemaIDTagValue);

		FGuid SchemaID;
		FGuid::Parse(SchemaIDTagValue, SchemaID);
		return SchemaID;
	}

	FSoftObjectPath GetSchemaThumbnailPath(const FAssetData& SchemaAssetData)
	{
		FString ThumbnailTagValue;
		const bool bTagPresent = SchemaAssetData.GetTagValue(UCineAssemblySchema::AssetRegistryTag_ThumbnailTexture, ThumbnailTagValue);
		if (!bTagPresent)
		{
			LogMissingSchemaThumbnailTagOnce(SchemaAssetData.PackageName);
		}
		return FSoftObjectPath(ThumbnailTagValue);
	}

	FAssetData GetSchemaAssetData(const FAssetData& AssemblyAssetData)
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");

		const FGuid SchemaID = GetSchemaID(AssemblyAssetData, UCineAssembly::AssetRegistryTag_SchemaGuid);
		if (SchemaID.IsValid())
		{
			const TMultiMap<FName, FString> TagValues = { { UCineAssemblySchema::SchemaGuidPropertyName, SchemaID.ToString() } };

			// Find the schema asset with the matching SchemaID
			TArray<FAssetData> SchemaAssets;
			AssetRegistryModule.Get().GetAssetsByTagValues(TagValues, SchemaAssets);
			if (!SchemaAssets.IsEmpty())
			{
				return SchemaAssets[0];
			}
		}

		// Fallback to finding the schema asset by class and then checking the asset name to find the matching one
		FString SchemaName;
		AssemblyAssetData.GetTagValue(UCineAssembly::AssetRegistryTag_AssemblyType, SchemaName);

		if (!SchemaName.IsEmpty())
		{
			TArray<FAssetData> AllSchemas;
			AssetRegistryModule.Get().GetAssetsByClass(UCineAssemblySchema::StaticClass()->GetClassPathName(), AllSchemas);

			if (const FAssetData* FoundSchema = Algo::FindBy(AllSchemas, FName(*SchemaName), &FAssetData::AssetName))
			{
				return *FoundSchema;
			}
		}

		return FAssetData();
	}
}
