// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Misc/Guid.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPath.h"

namespace UE::CineAssemblyTools::Private
{
	/** Logs (once per schema asset) a diagnostic when the schema's thumbnail registry tag does not exist. */
	void LogMissingSchemaThumbnailTagOnce(FName SchemaPackageName);

	/** Reads a schema GUID from the named registry tag on the input asset data. */
	FGuid GetSchemaID(const FAssetData& AssetData, FName TagName);

	/** Reads the thumbnail texture path from a schema's asset registry tags.*/
	FSoftObjectPath GetSchemaThumbnailPath(const FAssetData& SchemaAssetData);

	/** Returns the Schema FAssetData referenced by the input Assembly asset */
	FAssetData GetSchemaAssetData(const FAssetData& AssemblyAssetData);
}
