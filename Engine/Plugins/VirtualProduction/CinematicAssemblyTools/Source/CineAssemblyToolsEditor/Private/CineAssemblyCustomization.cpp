// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblyCustomization.h"

#include "Algo/Find.h"
#include "CineAssembly.h"
#include "CineAssemblyNamingTokens.h"
#include "CineAssemblySchema.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MovieSceneMetaData.h"
#include "ProductionSettings.h"
#include "PropertyCustomizationHelpers.h"
#include "SNamingTokensEditableTextBox.h"
#include "STemplateStringEditableTextBox.h"
#include "UI/CineAssembly/CineAssemblyMetadataWidgets.h"
#include "UI/SAssemblyMetadataLink.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CineAssemblyCustomization"

namespace UE::CineAssemblyTools::Private
{
	constexpr float AssetRowIndentWidth = 24.0f;
	constexpr float AssetRowLabelColumnWidth = 120.0f;

	/** Walk the ParentAssembly chain upward toward the RootAssembly (the one being customized). Return false if any parent will not be created, true otherwise */
	bool IsAncestryEnabled(UCineAssembly* ParentAssembly, UCineAssembly* RootAssembly)
	{
		UCineAssembly* CurrentAssembly = ParentAssembly;
		while (CurrentAssembly && CurrentAssembly != RootAssembly)
		{
			if (!CurrentAssembly->bShouldCreate)
			{
				return false;
			}
			CurrentAssembly = CurrentAssembly->GetParentAssembly().Get();
		}
		return true;
	}

	/** Pushes a context-menu popup containing a single Edit SubMenu wrapping the given panel widget at the mouse position. */
	void PushEditContextMenu(const FPointerEvent& MouseEvent, TSharedRef<SWidget> EditPanel)
	{
		if (EditPanel == SNullWidget::NullWidget)
		{
			return;
		}

		const FWidgetPath* EventPath = MouseEvent.GetEventPath();
		if (!EventPath)
		{
			return;
		}

		constexpr bool bCloseAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bCloseAfterMenuSelection, nullptr);

		MenuBuilder.AddSubMenu(
			LOCTEXT("EditMenuLabel", "Edit"),
			LOCTEXT("EditMenuTooltip", "Edit this row's Label and per-instance Metadata."),
			FNewMenuDelegate::CreateLambda([EditPanel](FMenuBuilder& SubMenuBuilder) { SubMenuBuilder.AddWidget(EditPanel, FText::GetEmpty(), true); }),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"));

		FSlateApplication::Get().PushMenu(
			EventPath->GetLastWidget(),
			*EventPath,
			MenuBuilder.MakeWidget(),
			MouseEvent.GetScreenSpacePosition(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
	}
}

TSharedRef<IDetailCustomization> FCineAssemblyCustomization::MakeInstance()
{
	return MakeShared<FCineAssemblyCustomization>();
}

void FCineAssemblyCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	DetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);

	// Ensure that we are only customizing one object
	if (CustomizedObjects.Num() != 1)
	{
		return;
	}

	CustomizedCineAssembly = Cast<UCineAssembly>(CustomizedObjects[0]);

	// Several widgets will display differently for newly configured assemblies compared to existing Assembly assets
	bIsBeingConfigured = CustomizedCineAssembly->GetPackage() == GetTransientPackage();

	LeadingZeroTypeInterface = MakeShared<FLeadingZeroNumericTypeInterface>();

	NamingTokenContext = TStrongObjectPtr<UCineAssemblyNamingTokensContext>(NewObject<UCineAssemblyNamingTokensContext>());
	NamingTokenContext->Assembly = CustomizedCineAssembly;

	FilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

	CustomizeDefaultCategory(DetailBuilder);
	CustomizeMetadataCategory(DetailBuilder);
	CustomizeManagedAssetsCategory(DetailBuilder);
}

void FCineAssemblyCustomization::CustomizeDefaultCategory(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& DefaultCategory = DetailBuilder.EditCategory("Default", LOCTEXT("DefaultCategoryName", "Default"));

	TSharedRef<IPropertyHandle> LevelPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCineAssembly, Level));
	IDetailPropertyRow& LevelPropertyRow = DefaultCategory.AddProperty(LevelPropertyHandle);

	TSharedRef<IPropertyHandle> ParentPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCineAssembly, ParentAssembly));
	IDetailPropertyRow& ParentPropertyRow = DefaultCategory.AddProperty(ParentPropertyHandle);

	TSharedRef<IPropertyHandle> ProdctionPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCineAssembly, Production));
	IDetailPropertyRow& ProductionPropertyRow = DefaultCategory.AddProperty(ProdctionPropertyHandle);

	UMovieSceneMetaData* MetaData = CustomizedCineAssembly->FindOrAddMetaData<UMovieSceneMetaData>();
	DefaultCategory.AddExternalObjectProperty({ MetaData }, FName("Author"));

	const UCineAssemblySchema* BaseSchema = CustomizedCineAssembly->GetSchema();
	const FSoftObjectPath ParentSchema = BaseSchema ? BaseSchema->ParentSchema : FSoftObjectPath();

	// Wrap the Level property in a metadata link widget during configuration
	if (bIsBeingConfigured)
	{
		const FString& LevelKey = UCineAssemblySchema::DefaultLevelMetadataKey;

		LevelPropertyRow.CustomWidget()
			.NameContent()
			[
				LevelPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.HAlign(HAlign_Fill)
			[
				SNew(SAssemblyMetadataLink, CustomizedCineAssembly, LevelKey)
					.EvaluateTokens(true)
					[
						SNew(SObjectPropertyEntryBox)
							.AllowedClass(UWorld::StaticClass())
							.ThumbnailPool(DetailBuilder.GetThumbnailPool())
							.AllowCreate(false)
							.ObjectPath_Lambda([this]()
								{
									return CustomizedCineAssembly->Level.ToString();
								})
							.OnObjectChanged_Lambda([this](const FAssetData& InAssetData)
								{
									CustomizedCineAssembly->Modify();
									CustomizedCineAssembly->Level = InAssetData.GetSoftObjectPath();
								})
					]
			];
	}

	ParentPropertyRow.CustomWidget()
		.NameContent()
		[
			ParentPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SObjectPropertyEntryBox)
				.AllowedClass(UCineAssembly::StaticClass())
				.ThumbnailPool(DetailBuilder.GetThumbnailPool())
				.AllowCreate(true)
				.OnShouldFilterAsset_Raw(this, &FCineAssemblyCustomization::ShouldFilterAssetBySchema, ParentSchema)
				.ObjectPath_Lambda([this]()
					{
						return CustomizedCineAssembly->ParentAssembly.ToString();
					})
				.OnObjectChanged_Lambda([this](const FAssetData& InAssetData)
					{
						CustomizedCineAssembly->Modify();
						CustomizedCineAssembly->ParentAssembly = InAssetData.GetObjectPathString();
					})
		];

	ProductionPropertyRow.CustomWidget()
		.NameContent()
		[
			ProdctionPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SComboButton)
				.VAlign(VAlign_Center)
				.OnGetMenuContent(this, &FCineAssemblyCustomization::BuildProductionNameMenu)
				.ButtonContent()
				[
					SNew(STextBlock)
						.Text_Lambda([this]()
							{
								const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
								TOptional<const FCinematicProduction> Production = ProductionSettings->GetProduction(CustomizedCineAssembly->Production);
								return Production.IsSet() ? FText::FromString(Production.GetValue().ProductionName) : FText::FromName(NAME_None);
							})
						.Font(IDetailLayoutBuilder::GetDetailFont())
				]
		];
}

void FCineAssemblyCustomization::CustomizeMetadataCategory(IDetailLayoutBuilder& DetailBuilder)
{
	const UCineAssemblySchema* BaseSchema = CustomizedCineAssembly->GetSchema();
	if (!BaseSchema)
	{
		return;
	}

	// Assemblies that are still being configured should always have editable metadata fields
	// Previously created assemblies should default to having read-only metadata
	bIsMetadataReadOnly = !bIsBeingConfigured;

	// Add a new category for Schema Metadata properties
	const FText MetadataCategoryName = FText::Format(LOCTEXT("SchemaMetadataCategoryName", "{0} Metadata"), FText::FromString(BaseSchema->SchemaName));
	IDetailCategoryBuilder& MetadataCategory = DetailBuilder.EditCategory("SchemaMetadata", MetadataCategoryName);

	// Add a lock/unlock checkbox to the category header to determine whether the metadata values are read-only or editable
	// Note: The checkbox will not be visible on Assemblies that are still being configured, so that their metadata is always editable
	MetadataCategory.HeaderContent(
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
				.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
				.Visibility_Lambda([this]() { return bIsBeingConfigured ? EVisibility::Collapsed : EVisibility::Visible; })
				.IsChecked_Lambda([this]() { return bIsMetadataReadOnly ? ECheckBoxState::Unchecked : ECheckBoxState::Checked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState) { bIsMetadataReadOnly = NewState != ECheckBoxState::Checked; })
				.ToolTipText_Lambda([this]() { return bIsMetadataReadOnly ? LOCTEXT("UnlockMetadataTooltip", "Unlock metadata for editing") : LOCTEXT("LockMetadataTooltip", "Lock metadata (make read-only)"); })
				[
					SNew(SImage)
						.DesiredSizeOverride(FVector2D(16, 16))
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image_Lambda([this]() { return bIsMetadataReadOnly ? FAppStyle::Get().GetBrush("PropertyWindow.Locked") : FAppStyle::Get().GetBrush("PropertyWindow.Unlocked"); })
				]
		]
	);

	// Add a row for each metadata field defined in the schema
	for (const FAssemblyMetadataDesc& MetadataDesc : BaseSchema->AssemblyMetadata)
	{
		if (MetadataDesc.Key.IsEmpty())
		{
			continue;
		}

		CustomizedCineAssembly->AddMetadataNamingToken(MetadataDesc.Key);

		MetadataCategory.AddCustomRow(FText::FromString(MetadataDesc.Key))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(FText::FromString(MetadataDesc.Key))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			.HAlign(HAlign_Fill)
			[
				SNew(SWidgetSwitcher)
					.WidgetIndex_Lambda([this]() 
						{
							constexpr int32 ReadOnlyWidgetIndex = 0;
							constexpr int32 ReadWriteWidgetIndex = 1;
							return bIsMetadataReadOnly ? ReadOnlyWidgetIndex : ReadWriteWidgetIndex;
						})

					+ SWidgetSwitcher::Slot()
					[
						SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
							.Text_Lambda([this, MetadataDesc]()
								{
									FString Value;
									CustomizedCineAssembly->GetMetadataAsString(MetadataDesc.Key, Value);
									return FText::FromString(Value);
								})
					]

					+ SWidgetSwitcher::Slot()
					[MakeMetadataWidget(DetailBuilder, MetadataDesc)]
			];
	}
}

TSharedRef<SWidget> FCineAssemblyCustomization::MakeMetadataWidget(IDetailLayoutBuilder& DetailBuilder, const FAssemblyMetadataDesc& MetadataDesc)
{
	auto WrapWidgetInMetadataLink = [this, &MetadataDesc](TSharedRef<SWidget> WidgetToWrap) -> TSharedRef<SWidget>
		{
			if (bIsBeingConfigured)
			{
				return SNew(SAssemblyMetadataLink, CustomizedCineAssembly, MetadataDesc.Key)
					.EvaluateTokens(true)
					[
						WidgetToWrap
					];
			}
			return WidgetToWrap;
		};

	switch (MetadataDesc.Type)
	{
		case ECineAssemblyMetadataType::String:
			return WrapWidgetInMetadataLink(
				SNew(SBox)
				.MaxDesiredHeight(120.0f)
				[
					MakeStringValueWidget(MetadataDesc)
				]
			);
		case ECineAssemblyMetadataType::Bool:
			return SNew(SCheckBox)
				.IsChecked_Lambda([this, MetadataDesc]()
					{
						bool Value;
						if (!CustomizedCineAssembly->GetMetadataAsBool(MetadataDesc.Key, Value))
						{
							Value = MetadataDesc.DefaultValue.Get<bool>();
						}
						return Value ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([this, MetadataDesc](ECheckBoxState CheckBoxState)
					{
						const bool Value = (CheckBoxState == ECheckBoxState::Checked);
						CustomizedCineAssembly->SetMetadataAsBool(MetadataDesc.Key, Value);
					});
		case ECineAssemblyMetadataType::Integer:
			return SNew(SNumericEntryBox<int32>)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.TypeInterface(LeadingZeroTypeInterface)
				.Value_Lambda([this, MetadataDesc]()
					{
						int32 Value;
						if (!CustomizedCineAssembly->GetMetadataAsInteger(MetadataDesc.Key, Value))
						{
							Value = MetadataDesc.DefaultValue.Get<int32>();
						}
						return Value;
					})
				.OnValueChanged_Lambda([this, MetadataDesc](int32 InValue)
					{
						// In order to preserve the desired number of leading zeroes, the integer value is converted to a string using the type interface
						// and then stored in the assembly as string metadata.
						const FString ValueString = LeadingZeroTypeInterface->ToString(InValue);
						CustomizedCineAssembly->SetMetadataAsString(MetadataDesc.Key, ValueString);
					});
		case ECineAssemblyMetadataType::Float:
			return SNew(SNumericEntryBox<float>)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Value_Lambda([this, MetadataDesc]()
					{
						float Value;
						if (!CustomizedCineAssembly->GetMetadataAsFloat(MetadataDesc.Key, Value))
						{
							Value = MetadataDesc.DefaultValue.Get<float>();
						}
						return Value;
					})
				.OnValueChanged_Lambda([this, MetadataDesc](float InValue)
					{
						CustomizedCineAssembly->SetMetadataAsFloat(MetadataDesc.Key, InValue);
					});
		case ECineAssemblyMetadataType::AssetPath:
			return WrapWidgetInMetadataLink(
				SNew(SObjectPropertyEntryBox)
				.AllowedClass(!MetadataDesc.AssetClass.IsNull() ? MetadataDesc.AssetClass.ResolveClass() : UObject::StaticClass())
				.ThumbnailPool(DetailBuilder.GetThumbnailPool())
				.AllowCreate(true)
				.ObjectPath_Lambda([this, MetadataDesc]()
					{
						FString Value;
						if (!CustomizedCineAssembly->GetMetadataAsString(MetadataDesc.Key, Value))
						{
							Value = MetadataDesc.DefaultValue.Get<FString>();
						}
						return Value;
					})
				.OnObjectChanged_Lambda([this, MetadataDesc](const FAssetData& InAssetData)
					{
						CustomizedCineAssembly->SetMetadataAsString(MetadataDesc.Key, InAssetData.GetObjectPathString());
					})
			);
		case ECineAssemblyMetadataType::CineAssembly:
			return WrapWidgetInMetadataLink(
				SNew(SObjectPropertyEntryBox)
				.AllowedClass(UCineAssembly::StaticClass())
				.ThumbnailPool(DetailBuilder.GetThumbnailPool())
				.AllowCreate(true)
				.OnShouldFilterAsset_Raw(this, &FCineAssemblyCustomization::ShouldFilterAssetBySchema, MetadataDesc.SchemaType)
				.ObjectPath_Lambda([this, MetadataDesc]()
					{
						FString Value;
						if (!CustomizedCineAssembly->GetMetadataAsString(MetadataDesc.Key, Value))
						{
							Value = MetadataDesc.DefaultValue.Get<FString>();
						}
						return Value;
					})
				.OnObjectChanged_Lambda([this, MetadataDesc](const FAssetData& InAssetData)
					{
						CustomizedCineAssembly->SetMetadataAsString(MetadataDesc.Key, InAssetData.GetObjectPathString());
					})
			);
		default:
			checkNoEntry();
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> FCineAssemblyCustomization::MakeStringValueWidget(const FAssemblyMetadataDesc& MetadataDesc)
{
	TSharedPtr<SWidget> StringValueWidget;
	if (!MetadataDesc.bEvaluateTokens)
	{
		StringValueWidget = SNew(SMultiLineEditableTextBox)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.AutoWrapText(true)
			.Text_Lambda([this, MetadataDesc]()
				{
					FString Value;
					if (!CustomizedCineAssembly->GetMetadataAsString(MetadataDesc.Key, Value))
					{
						Value = MetadataDesc.DefaultValue.Get<FString>();
					}
					return FText::FromString(Value);
				})
			.OnTextCommitted_Lambda([this, MetadataDesc](const FText& InText, ETextCommit::Type InCommitType)
				{
					CustomizedCineAssembly->SetMetadataAsString(MetadataDesc.Key, InText.ToString());
				});
	}
	else
	{
		StringValueWidget = SNew(SNamingTokensEditableTextBox)
			.AllowMultiLine(true)
			.Contexts({ NamingTokenContext.Get() })
			.FilterArgs(FilterArgs)
			.EvaluationFrequency(1.0f)
			.ShowUnsetTokenWarning(true)
			.Text_Lambda([this, &MetadataDesc]() -> FText
				{
					return GetMetadataTokenStringValue(MetadataDesc.Key);
				})
			.OnTextCommitted_Lambda([this, &MetadataDesc](const FText& InText, ETextCommit::Type InCommitType)
				{
					SetMetadataTokenStringValue(MetadataDesc.Key, InText, ETemplateStringType::Template);
				})
			.OnTokenizedTextEvaluated_Lambda([this, &MetadataDesc](const FText& InText)
				{
					SetMetadataTokenStringValue(MetadataDesc.Key, InText, ETemplateStringType::Resolved);
				});
	}

	return StringValueWidget.ToSharedRef();
}

bool FCineAssemblyCustomization::ShouldFilterAssetBySchema(const FAssetData& InAssetData, FSoftObjectPath Schema)
{
	if (Schema.IsValid())
	{
		const FAssetDataTagMapSharedView::FFindTagResult AssemblyType = InAssetData.TagsAndValues.FindTag(UCineAssembly::AssetRegistryTag_AssemblyType);
		if (AssemblyType.IsSet())
		{
			return !AssemblyType.GetValue().Equals(Schema.GetAssetName());
		}
		return true;
	}
	return false;
}

TSharedRef<SWidget> FCineAssemblyCustomization::BuildProductionNameMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	// Always add a "None" option
	FName NoActiveProductionName = NAME_None;
	MenuBuilder.AddMenuEntry(
		FText::FromName(NoActiveProductionName),
		FText::FromName(NoActiveProductionName),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([this]() 
			{ 
				CustomizedCineAssembly->Modify();
				CustomizedCineAssembly->Production = FGuid();
				CustomizedCineAssembly->ProductionName = TEXT("None");
			})),
		NAME_None,
		EUserInterfaceActionType::None
	);

	const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();

	// Add a menu option with the production name for each production available in this project
	const TArray<FCinematicProduction> Productions = ProductionSettings->GetProductions();
	for (const FCinematicProduction& Production : Productions)
	{
		MenuBuilder.AddMenuEntry(
			FText::FromString(Production.ProductionName),
			FText::FromString(Production.ProductionName),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this, Production]() 
				{
					CustomizedCineAssembly->Modify();
					CustomizedCineAssembly->Production = Production.ProductionID;
					CustomizedCineAssembly->ProductionName = Production.ProductionName;
				})),
			NAME_None,
			EUserInterfaceActionType::None
		);
	}

	return MenuBuilder.MakeWidget();
}

FText FCineAssemblyCustomization::GetMetadataTokenStringValue(const FString& InMetadataKey) const
{
	FTemplateString Value;
	if (CustomizedCineAssembly->GetMetadataAsTokenString(InMetadataKey, Value))
	{
		return FText::FromString(Value.Template);
	}
	return FText::GetEmpty();
}

void FCineAssemblyCustomization::SetMetadataTokenStringValue(const FString& InMetadataKey, const FText& InValue, ETemplateStringType InTemplateStringType)
{
	FTemplateString TemplateString;
	CustomizedCineAssembly->GetMetadataAsTokenString(InMetadataKey, TemplateString);

	if (InTemplateStringType == ETemplateStringType::Template)
	{
		TemplateString.Template = InValue.ToString();
	}
	else if (InTemplateStringType == ETemplateStringType::Resolved)
	{
		TemplateString.Resolved = InValue;
	}

	CustomizedCineAssembly->SetMetadataAsTokenString(InMetadataKey, TemplateString);
}

void FCineAssemblyCustomization::CustomizeManagedAssetsCategory(IDetailLayoutBuilder& DetailBuilder)
{
	if (!bIsBeingConfigured)
	{
		return;
	}

	if (CustomizedCineAssembly->SubAssemblies.IsEmpty() && CustomizedCineAssembly->AssociatedAssets.IsEmpty())
	{
		return;
	}

	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("ManagedAssets", LOCTEXT("ManagedAssetsCategory", "Managed Assets"));

	ManagedAssetRowTokenContexts.Reset();

	constexpr int32 StartingDepth = 1;
	AddAssetRowsRecursive(Category, CustomizedCineAssembly, StartingDepth);
}

void FCineAssemblyCustomization::AddAssetRowsRecursive(IDetailCategoryBuilder& Category, UCineAssembly* Assembly, int32 Depth)
{
	if (!Assembly)
	{
		return;
	}

	for (UCineAssembly* SubAssembly : Assembly->GetSubAssemblies())
	{
		AddSubAssemblyRow(Category, SubAssembly, Depth);
		AddAssetRowsRecursive(Category, SubAssembly, Depth + 1);
	}

	for (const FAssemblyAssociatedAssetDesc& AssetDesc : Assembly->AssociatedAssets)
	{
		AddAssociatedAssetRow(Category, Assembly, AssetDesc.AssetID, Depth);
	}
}

void FCineAssemblyCustomization::AddSubAssemblyRow(IDetailCategoryBuilder& Category, UCineAssembly* SubAssembly, int32 Depth)
{
	using namespace UE::CineAssemblyTools::Private;

	if (!SubAssembly)
	{
		return;
	}

	UCineAssembly* RootAssembly = CustomizedCineAssembly;

	// Naming Token Context used by this Row to evaluate the SubAssembly name
	TStrongObjectPtr<UCineAssemblyNamingTokensContext>& RowContext = ManagedAssetRowTokenContexts.Emplace_GetRef(NewObject<UCineAssemblyNamingTokensContext>());
	RowContext->Assembly = SubAssembly;

	const float IndentAmount = FMath::Max(0, Depth - 1) * AssetRowIndentWidth;

	TSharedRef<SHorizontalBox> SubAssemblyRow = SNew(SHorizontalBox)
		.IsEnabled_Lambda([SubAssembly, RootAssembly]() { return IsAncestryEnabled(SubAssembly->GetParentAssembly().Get(), RootAssembly); })

		// Checkbox to control the SubAssembly's bShouldCreate property
		+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(IndentAmount, 0.0f, 6.0f, 0.0f))
			[
				SNew(SCheckBox)
					.IsChecked_Lambda([SubAssembly]() { return SubAssembly->bShouldCreate ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					.OnCheckStateChanged_Lambda([SubAssembly](ECheckBoxState CheckBoxState) { SubAssembly->bShouldCreate = (CheckBoxState == ECheckBoxState::Checked); })
			]

		// Naming Tokens TextBox to display the SubAssembly's name
		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SNamingTokensEditableTextBox)
					.Contexts({ RowContext.Get() })
					.FilterArgs(FilterArgs)
					.EvaluationFrequency(1.0f)
					.ShowUnsetTokenWarning(true)
					.Text_Lambda([SubAssembly]() { return FText::FromString(SubAssembly->AssemblyName.Template); })
					.OnTextCommitted_Lambda([SubAssembly](const FText& InText, ETextCommit::Type) { SubAssembly->AssemblyName.Template = InText.ToString(); })
					.OnTokenizedTextEvaluated_Lambda([SubAssembly](const FText& InText) { SubAssembly->AssemblyName.Resolved = InText; })
			]

		// TextBlock to display the SubAssembly's label
		+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
			[
				// Fixed-width box so every row's label starts at the same x-position, giving a clean column appearance
				SNew(SBox)
					.MinDesiredWidth(AssetRowLabelColumnWidth)
					.HAlign(HAlign_Left)
					[
						SNew(STextBlock)
							.Font(FCoreStyle::GetDefaultFontStyle("Italic", 10))
							.ColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.3f))
							.Text_Lambda([SubAssembly]() -> FText
								{
									const FName Label = SubAssembly->GetLabel();
									return Label.IsNone() ? FText::GetEmpty() : FText::Format(LOCTEXT("LabelOnly", "[{0}]"), FText::FromName(Label));
								})
					]
			];

	Category.AddCustomRow(FText::GetEmpty())
		.WholeRowContent()
		.HAlign(HAlign_Fill)
		[
			// Wrap row content in an SBorder to handle right-click to summon the metadata edit menu
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(0.0f)
				.OnMouseButtonUp(this, &FCineAssemblyCustomization::OnSubAssemblyRowClicked, SubAssembly)
				[
					SubAssemblyRow
				]
		];
}

void FCineAssemblyCustomization::AddAssociatedAssetRow(IDetailCategoryBuilder& Category, UCineAssembly* OwnerAssembly, const FGuid& AssetID, int32 Depth)
{
	using namespace UE::CineAssemblyTools::Private;

	if (!OwnerAssembly)
	{
		return;
	}

	UCineAssembly* RootAssembly = CustomizedCineAssembly;

	// Naming Token Context used by this Row to evaluate the associated asset name
	TStrongObjectPtr<UCineAssemblyNamingTokensContext>& RowContext = ManagedAssetRowTokenContexts.Emplace_GetRef(NewObject<UCineAssemblyNamingTokensContext>());
	RowContext->Assembly = OwnerAssembly;

	const float IndentAmount = FMath::Max(0, Depth - 1) * AssetRowIndentWidth;

	/** Finds the mutable AssetDesc for the given AssetID on the owner, or nullptr if missing. */
	auto FindAssetDesc = [OwnerAssembly, AssetID]()
		{
			return Algo::FindBy(OwnerAssembly->AssociatedAssets, AssetID, &FAssemblyAssociatedAssetDesc::AssetID);
		};

	TSharedRef<SHorizontalBox> AssociatedAssetRow = SNew(SHorizontalBox)
		.IsEnabled_Lambda([OwnerAssembly, RootAssembly]() { return IsAncestryEnabled(OwnerAssembly, RootAssembly); })

		// Checkbox to control the associated asset's bShouldCreate property
		+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(IndentAmount, 0.0f, 6.0f, 0.0f))
			[
				SNew(SCheckBox)
					.IsChecked_Lambda([FindAssetDesc]()
						{
							const FAssemblyAssociatedAssetDesc* Desc = FindAssetDesc();
							return (Desc && Desc->bShouldCreate) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})
					.OnCheckStateChanged_Lambda([FindAssetDesc](ECheckBoxState CheckBoxState)
						{
							if (FAssemblyAssociatedAssetDesc* Desc = FindAssetDesc())
							{
								Desc->bShouldCreate = (CheckBoxState == ECheckBoxState::Checked);
							}
						})
			]

		// Naming Tokens TextBox to display the associated asset's name
		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SNamingTokensEditableTextBox)
					.Contexts({ RowContext.Get() })
					.FilterArgs(FilterArgs)
					.EvaluationFrequency(1.0f)
					.ShowUnsetTokenWarning(true)
					.Text_Lambda([FindAssetDesc]()
						{
							const FAssemblyAssociatedAssetDesc* Desc = FindAssetDesc();
							return Desc ? FText::FromString(Desc->AssetName.Template) : FText::GetEmpty();
						})
					.OnTextCommitted_Lambda([FindAssetDesc](const FText& InText, ETextCommit::Type)
						{
							if (FAssemblyAssociatedAssetDesc* Desc = FindAssetDesc())
							{
								Desc->AssetName.Template = InText.ToString();
							}
						})
					.OnTokenizedTextEvaluated_Lambda([FindAssetDesc](const FText& InText)
						{
							if (FAssemblyAssociatedAssetDesc* Desc = FindAssetDesc())
							{
								Desc->AssetName.Resolved = InText;
							}
						})
			]

		// TextBlock to display the associated asset's label
		+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
			[
				// Fixed-width box so every row's label starts at the same x-position, giving a clean column appearance
				SNew(SBox)
					.MinDesiredWidth(AssetRowLabelColumnWidth)
					.HAlign(HAlign_Left)
					[
						SNew(STextBlock)
							.Font(FCoreStyle::GetDefaultFontStyle("Italic", 10))
							.ColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.3f))
							.Text_Lambda([FindAssetDesc]() -> FText
								{
									const FAssemblyAssociatedAssetDesc* Desc = FindAssetDesc();
									const FName Label = Desc ? Desc->Label : NAME_None;
									return Label.IsNone() ? FText::GetEmpty() : FText::Format(LOCTEXT("LabelOnly", "[{0}]"), FText::FromName(Label));
								})
					]
			];

	Category.AddCustomRow(FText::GetEmpty())
		.WholeRowContent()
		.HAlign(HAlign_Fill)
		[
			// Wrap row content in an SBorder to handle right-click to summon the Edit Label menu
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(0.0f)
				.OnMouseButtonUp(this, &FCineAssemblyCustomization::OnAssociatedAssetRowClicked, OwnerAssembly, AssetID)
				[
					AssociatedAssetRow
				]
		];
}

FReply FCineAssemblyCustomization::OnSubAssemblyRowClicked(const FGeometry&, const FPointerEvent& MouseEvent, UCineAssembly* SubAssembly)
{
	using namespace UE::CineAssemblyTools::Private;

	if (MouseEvent.GetEffectingButton() != EKeys::RightMouseButton || !SubAssembly)
	{
		return FReply::Unhandled();
	}

	TSharedRef<SWidget> EditPanel = FCineAssemblyMetadataWidgets::MakeEditMenuForAssembly(SubAssembly);
	PushEditContextMenu(MouseEvent, EditPanel);

	return FReply::Handled();
}

FReply FCineAssemblyCustomization::OnAssociatedAssetRowClicked(const FGeometry&, const FPointerEvent& MouseEvent, UCineAssembly* OwnerAssembly, FGuid AssetID)
{
	using namespace UE::CineAssemblyTools::Private;

	if (MouseEvent.GetEffectingButton() != EKeys::RightMouseButton || !OwnerAssembly)
	{
		return FReply::Unhandled();
	}

	TSharedRef<SWidget> EditPanel = FCineAssemblyMetadataWidgets::MakeEditMenuForAssociatedAsset(OwnerAssembly, AssetID);
	PushEditContextMenu(MouseEvent, EditPanel);

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
