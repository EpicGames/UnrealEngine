// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

#include "CineAssemblyNamingTokens.h"
#include "Input/Reply.h"
#include "LeadingZeroNumericTypeInterface.h"
#include "NamingTokensEngineSubsystem.h"

class IDetailCategoryBuilder;
class UCineAssembly;
struct FAssemblyMetadataDesc;
struct FGeometry;
struct FPointerEvent;

/**
 * Detail customization for UCineAssembly
 */
class FCineAssemblyCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface */

private:
	/** Utility functions to customize each of the categories */
	void CustomizeDefaultCategory(IDetailLayoutBuilder& DetailBuilder);
	void CustomizeMetadataCategory(IDetailLayoutBuilder& DetailBuilder);
	void CustomizeManagedAssetsCategory(IDetailLayoutBuilder& DetailBuilder);

	/** Creates the appropriate widget for a metadata field based on its type */
	TSharedRef<SWidget> MakeMetadataWidget(IDetailLayoutBuilder& DetailBuilder, const FAssemblyMetadataDesc& MetadataDesc);

	/**
	* Given a FAssemblyMetadataDesc, makes a STemplateStringEditableTextBox for a string value that has bEvaluateTokens set to true,
	* and a SMultiLineEditableTextBox for a string value that has bEvaluateTokens set to false.
	*/
	TSharedRef<SWidget> MakeStringValueWidget(const FAssemblyMetadataDesc& MetadataDesc);

	/** Builds the drop-down menu list of productions */
	TSharedRef<SWidget> BuildProductionNameMenu();

	/** Determines whether the input asset should be filtered out of an object picker widget, based on whether it is of the input schema type */
	bool ShouldFilterAssetBySchema(const FAssetData& InAssetData, FSoftObjectPath Schema);

	/** Used to specify which member of an FTemplateString is desired */
	enum class ETemplateStringType
	{
		Template,
		Resolved
	};

	/** Returns the template text for a token string metadata property */
	FText GetMetadataTokenStringValue(const FString& InMetadataKey) const;

	/** Updates either the template text or resolved text for a token string metadata property */
	void SetMetadataTokenStringValue(const FString& InMetadataKey, const FText& InValue, ETemplateStringType InTemplateStringType);

	/** Recursively adds rows to the input category for the input Assembly's SubAssemblies and AssociatedAssets, using the recursion depth to indent properly */
	void AddAssetRowsRecursive(IDetailCategoryBuilder& Category, UCineAssembly* Assembly, int32 Depth);

	/** Adds an asset row for a managed SubAssembly */
	void AddSubAssemblyRow(IDetailCategoryBuilder& Category, UCineAssembly* SubAssembly, int32 Depth);

	/** Adds an asset row for a managed Associated Asset */
	void AddAssociatedAssetRow(IDetailCategoryBuilder& Category, UCineAssembly* OwnerAssembly, const FGuid& AssetID, int32 Depth);

	/** Callback when a SubAssembly asset row is clicked. Right-click shows a menu to edit the SubAssembly's label and metadata */
	FReply OnSubAssemblyRowClicked(const FGeometry&, const FPointerEvent& MouseEvent, UCineAssembly* SubAssembly);

	/** Callback when an Associated Asset row is clicked. Right-click shows a menu to edit the descriptor's Label */
	FReply OnAssociatedAssetRowClicked(const FGeometry&, const FPointerEvent& MouseEvent, UCineAssembly* OwnerAssembly, FGuid AssetID);

private:
	/** The assembly being customized */
	UCineAssembly* CustomizedCineAssembly;

	/** Naming Token Contexts for each asset row, referencing the correct SubAssembly for token resolution */
	TArray<TStrongObjectPtr<UCineAssemblyNamingTokensContext>> ManagedAssetRowTokenContexts;

	/** Type interface for a numeric entry box that supports leading zeroes */
	TSharedPtr<FLeadingZeroNumericTypeInterface> LeadingZeroTypeInterface;

	/** Context object allowing cine assembly tokens to evaluate */
	TStrongObjectPtr<UCineAssemblyNamingTokensContext> NamingTokenContext;

	/** Filter allowing cine assembly tokens to evaluate without the namespace prefix */
	FNamingTokenFilterArgs FilterArgs;

	/** True if the metadata properties are read-only, false if they are editable */
	bool bIsMetadataReadOnly = true;

	/** True if the assembly is being configured (transient), false if viewing an existing assembly */
	bool bIsBeingConfigured = false;
};
