// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblyEditorFunctionLibrary.h"

#include "Algo/Find.h"
#include "AssetDefinitionRegistry.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetToolsModule.h"
#include "CineAssembly.h"
#include "CineAssemblyEditorSubsystem.h"
#include "CineAssemblyFactory.h"
#include "CineAssemblyNamingTokens.h"
#include "CineAssemblySchema.h"
#include "CineAssemblySchemaFactory.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "IAssetTools.h"
#include "Misc/PackageName.h"
#include "MovieScene.h"
#include "MovieSceneSubAssemblySection.h"
#include "MovieSceneSubAssemblyTrack.h"
#include "NamingTokensEngineSubsystem.h"
#include "ProductionSettings.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UObject/Package.h"

DEFINE_LOG_CATEGORY_STATIC(LogCineAssemblyEditorFunctionLibrary, Log, All)

namespace UE::CineAssemblyTools::Private
{
	/** Adds a new SubAssemblyTrack and Section to the template sequence of the input Schema */
	UMovieSceneSubAssemblySection* AddSubAssemblyTrackAndSection(UCineAssemblySchema* Schema, ESubAssemblyTrackType TrackType)
	{
		if (!Schema)
		{
			UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not add SubAssembly because the Schema is null");
			return nullptr;
		}

		UMovieScene* MovieScene = Schema->TemplateSequence ? Schema->TemplateSequence->GetMovieScene() : nullptr;
		if (!MovieScene)
		{
			UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not add SubAssembly because the Schema's TemplateSequence is invalid");
			return nullptr;
		}

		UMovieSceneSubAssemblyTrack* NewTrack = Cast<UMovieSceneSubAssemblyTrack>(MovieScene->AddTrack(UMovieSceneSubAssemblyTrack::StaticClass()));
		if (!NewTrack)
		{
			return nullptr;
		}

		NewTrack->TrackType = TrackType;

		const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
		const FFrameNumber StartFrame = PlaybackRange.GetLowerBoundValue();
		const int32 Duration = PlaybackRange.Size<FFrameNumber>().Value;

		UMovieSceneSubAssemblySection* NewSection = Cast<UMovieSceneSubAssemblySection>(NewTrack->AddSequence(nullptr, StartFrame, Duration));
		if (!NewSection)
		{
			MovieScene->RemoveTrack(*NewTrack);
			return nullptr;
		}

		return NewSection;
	}

	/** Strips /All/ prefix and validates the path for content-browser use. Returns true on success. */
	bool NormalizeAndValidateContentPath(const FString& InPath, FString& OutPath)
	{
		OutPath = InPath;
		if (OutPath.StartsWith(TEXT("/All/")))
		{
			OutPath.RemoveFromStart(TEXT("/All"));
		}
		return FPackageName::IsValidPath(OutPath);
	}

	/** Returns Schema->TemplateSequence after null-checking both, logging an error if either is null. */
	UCineAssembly* GetTemplateSequence(const UCineAssemblySchema* Schema)
	{
		if (!Schema)
		{
			UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Schema is null");
			return nullptr;
		}
		if (!Schema->TemplateSequence)
		{
			UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Schema '%ls' has no valid TemplateSequence", *Schema->GetName());
			return nullptr;
		}
		return Schema->TemplateSequence;
	}

	/** Finds the AssociatedAsset descriptor with the given AssetID on the given Assembly. Logs and returns null on failure. Caller is responsible for calling Modify() before mutating the returned descriptor. */
	FAssemblyAssociatedAssetDesc* FindAssociatedAsset(UCineAssembly* Assembly, const FGuid& AssetID)
	{
		if (!Assembly)
		{
			UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Associated Asset lookup failed because the Assembly is null");
			return nullptr;
		}

		FAssemblyAssociatedAssetDesc* AssetDesc = Algo::FindBy(Assembly->AssociatedAssets, AssetID, &FAssemblyAssociatedAssetDesc::AssetID);
		if (!AssetDesc)
		{
			UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "No Associated Asset with ID '%ls' was found on '%ls'", *AssetID.ToString(), *Assembly->GetName());
			return nullptr;
		}

		return AssetDesc;
	}
}

void UCineAssemblyEditorFunctionLibrary::OpenAssembly(UCineAssembly* CineAssembly, bool bOpenAssociatedLevel, FCineAssemblyLevelSaveOptions SaveOptions)
{
	if (!CineAssembly)
	{
		return;
	}

	if (bOpenAssociatedLevel)
	{
		OpenAssociatedLevel(CineAssembly, SaveOptions);
	}

	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets({ CineAssembly });
}

bool UCineAssemblyEditorFunctionLibrary::OpenAssociatedLevel(UCineAssembly* CineAssembly, FCineAssemblyLevelSaveOptions SaveOptions)
{
	if (!CineAssembly)
	{
		return false;
	}

	if (CineAssembly->Level.IsValid() && !CineAssembly->bIsDataOnly)
	{
		// Load the level associated with the input assembly
		if (UWorld* WorldToOpen = Cast<UWorld>(CineAssembly->Level.TryLoad()))
		{
			UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();

			// Early-out if the associated level is already open
			if (WorldToOpen == CurrentWorld)
			{
				return true;
			}

			// Save the currently open level before opening the associated level (if they differ)
			if (SaveOptions.bSaveCurrentLevel)
			{
				constexpr bool bSaveMapPackages = true;
				constexpr bool bSaveContentPackages = false;
				if (!FEditorFileUtils::SaveDirtyPackages(SaveOptions.bPromptForUnsavedChanges, bSaveMapPackages, bSaveContentPackages))
				{
					// If the user was prompted to save but cancelled, then do not proceed with opening the level
					return false;
				}
			}

			// Open the associated level
			if (!WorldToOpen->GetPackage()->HasAnyPackageFlags(PKG_NewlyCreated))
			{
				const FString FileToOpen = FPackageName::LongPackageNameToFilename(WorldToOpen->GetOutermost()->GetName(), FPackageName::GetMapPackageExtension());

				const bool bLoadAsTemplate = false;
				const bool bShowProgress = true;
				FEditorFileUtils::LoadMap(FileToOpen, bLoadAsTemplate, bShowProgress);
			}
		}
	}

	return true;
}

UCineAssembly* UCineAssemblyEditorFunctionLibrary::CreateAssembly(UCineAssemblySchema* Schema, TSoftObjectPtr<UWorld> Level, TSoftObjectPtr<UCineAssembly> ParentAssembly, const TMap<FString, FString>& Metadata, const FString& Path, const FString& NameOverride)
{
	UCineAssembly* NewAssembly = CreateAssemblyToConfigure(Schema, Level, ParentAssembly, Metadata);
	if (!NewAssembly)
	{
		return nullptr;
	}

	return FinalizeConfiguredAssembly(NewAssembly, Path, NameOverride);
}

UCineAssembly* UCineAssemblyEditorFunctionLibrary::CreateAssemblyToConfigure(UCineAssemblySchema* Schema, TSoftObjectPtr<UWorld> Level, TSoftObjectPtr<UCineAssembly> ParentAssembly, const TMap<FString, FString>& Metadata)
{
	// Create a new transient assembly to configure. Call FinalizeConfiguredAssembly() to persist it as an asset.
	UCineAssembly* NewAssembly = NewObject<UCineAssembly>(GetTransientPackage(), NAME_None, RF_Transient);
	NewAssembly->InitializeFromSchema(Schema);

	NewAssembly->SetLevel(Level);
	NewAssembly->SetParentAssembly(ParentAssembly);

	// Associate the current active production with this assembly
	const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
	TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
	if (ActiveProduction.IsSet())
	{
		NewAssembly->Production = ActiveProduction->ProductionID;
		NewAssembly->ProductionName = ActiveProduction->ProductionName;
	}

	NewAssembly->ApplyMetadata(Metadata);

	return NewAssembly;
}

UCineAssembly* UCineAssemblyEditorFunctionLibrary::FinalizeConfiguredAssembly(UCineAssembly* ConfiguredAssembly, const FString& Path, const FString& NameOverride)
{
	if (!ConfiguredAssembly)
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not persist Cine Assembly because the input assembly is null");
		return nullptr;
	}

	if (ConfiguredAssembly->GetPackage() != GetTransientPackage())
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not persist Cine Assembly '%ls' because it is not a transient assembly", *ConfiguredAssembly->GetName());
		return nullptr;
	}

	FString CreatePath;
	bool bPathIsValid = UE::CineAssemblyTools::Private::NormalizeAndValidateContentPath(Path, CreatePath);
	if (!bPathIsValid)
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not create Cine Assembly asset because the path '%ls' is not valid", *Path);
		return nullptr;
	}

	if (!NameOverride.IsEmpty())
	{
		ConfiguredAssembly->AssemblyName.Template = NameOverride;
		ConfiguredAssembly->AssemblyName.Resolved = FText::FromString(NameOverride);
	}

	UCineAssemblyFactory::CreateConfiguredAssembly(ConfiguredAssembly, CreatePath);

	// Check to see if the factory failed to create an asset for the configured assembly
	if (ConfiguredAssembly->GetPackage() == GetTransientPackage())
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Factory failed to create a new CineAssembly asset for '%ls' at '%ls'", *ConfiguredAssembly->GetName(), *CreatePath);
		return nullptr;
	}

	// The new top-level assembly has nothing to duplicate, but its SubAssemblies might have been created as duplicates of existing assets.
	// These steps will handle duplicating an subsequences and associated assets of such SubAssemblies (if any exist)
	UCineAssemblyFactory::DuplicateSubsequences(ConfiguredAssembly);
	UCineAssemblyFactory::DuplicateAssociatedAssets(ConfiguredAssembly);

	return ConfiguredAssembly;
}

UCineAssembly* UCineAssemblyEditorFunctionLibrary::DuplicateAssembly(UCineAssembly* SourceAssembly, const FString& DuplicatePath, const TMap<FString, FString>& Metadata, bool bDuplicateSubsequences, bool bDuplicateAssociatedAssets, const FString& NameOverride, EDuplicateExternalAssetPreference ExternalAssetPreference)
{
	UCineAssembly* DuplicatedAssembly = DuplicateAssemblyToConfigure(SourceAssembly, Metadata, bDuplicateSubsequences);
	if (!DuplicatedAssembly)
	{
		return nullptr;
	}

	return FinalizeDuplicateAssembly(DuplicatedAssembly, DuplicatePath, NameOverride, bDuplicateSubsequences, bDuplicateAssociatedAssets, ExternalAssetPreference);
}

UCineAssembly* UCineAssemblyEditorFunctionLibrary::DuplicateAssemblyToConfigure(UCineAssembly* SourceAssembly, const TMap<FString, FString>& Metadata, bool bDuplicateSubsequences)
{
	if (!SourceAssembly)
	{
		return nullptr;
	}

	// Create a transient duplicate of the source assembly to configure. Call FinalizeDuplicateAssembly() to persist it as an asset.
	UCineAssembly* DuplicatedAssembly = Cast<UCineAssembly>(StaticDuplicateObject(SourceAssembly, GetTransientPackage(), NAME_None, RF_Transient));
	if (!DuplicatedAssembly)
	{
		return nullptr;
	}

	DuplicatedAssembly->SourceAssembly = SourceAssembly;

	DuplicatedAssembly->ApplyMetadata(Metadata);

	if (bDuplicateSubsequences)
	{
		// Set the original path so DuplicateSubsequences can compute correct relative positions during persist
		DuplicatedAssembly->SourceAssemblyPath = FPaths::GetPath(SourceAssembly->GetPathName());

		// Recursively create transient duplicates of each managed SubAssembly, which will also be given valid packages during persist
		DuplicatedAssembly->DuplicateManagedSubAssemblies();
	}

	return DuplicatedAssembly;
}

UCineAssembly* UCineAssemblyEditorFunctionLibrary::FinalizeDuplicateAssembly(UCineAssembly* ConfiguredDuplicateAssembly, const FString& DuplicatePath, const FString& NameOverride, bool bDuplicateSubsequences, bool bDuplicateAssociatedAssets, EDuplicateExternalAssetPreference ExternalAssetPreference)
{
	if (!ConfiguredDuplicateAssembly)
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not persist Cine Assembly because the input assembly is null");
		return nullptr;
	}

	if (ConfiguredDuplicateAssembly->GetPackage() != GetTransientPackage())
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not persist Cine Assembly '%ls' because it is not a transient assembly", *ConfiguredDuplicateAssembly->GetName());
		return nullptr;
	}

	if (ConfiguredDuplicateAssembly->SourceAssembly.IsNull())
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not persist Cine Assembly '%ls' as a duplicate because its SourceAssembly is not set. If this is a fresh assembly rather than a duplicate, call FinalizeConfiguredAssembly instead.", *ConfiguredDuplicateAssembly->GetName());
		return nullptr;
	}

	FString CreatePath;
	bool bPathIsValid = UE::CineAssemblyTools::Private::NormalizeAndValidateContentPath(DuplicatePath, CreatePath);
	if (!bPathIsValid)
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not create Cine Assembly asset because the path '%ls' is not valid", *DuplicatePath);
		return nullptr;
	}

	if (!NameOverride.IsEmpty())
	{
		ConfiguredDuplicateAssembly->AssemblyName.Template = NameOverride;
		ConfiguredDuplicateAssembly->AssemblyName.Resolved = FText::FromString(NameOverride);
	}

	UCineAssemblyFactory::CreateConfiguredAssembly(ConfiguredDuplicateAssembly, CreatePath);

	// Check to see if the factory failed to create an asset for the configured assembly
	if (ConfiguredDuplicateAssembly->GetPackage() == GetTransientPackage())
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Factory failed to create a new CineAssembly asset for '%ls' at '%ls'", *ConfiguredDuplicateAssembly->GetName(), *CreatePath);
		return nullptr;
	}

	if (bDuplicateSubsequences)
	{
		UCineAssemblyFactory::DuplicateSubsequences(ConfiguredDuplicateAssembly, ExternalAssetPreference);
	}

	if (bDuplicateAssociatedAssets)
	{
		UCineAssemblyFactory::DuplicateAssociatedAssets(ConfiguredDuplicateAssembly, ExternalAssetPreference);
	}

	// Broadcast the duplication event.
	if (UCineAssemblyEditorSubsystem* EditorSubsystem = GEditor ? GEditor->GetEditorSubsystem<UCineAssemblyEditorSubsystem>() : nullptr)
	{
		UCineAssembly* Source = ConfiguredDuplicateAssembly->SourceAssembly.LoadSynchronous();
		EditorSubsystem->OnAssemblyDuplicated.Broadcast(ConfiguredDuplicateAssembly, Source);
	}

	return ConfiguredDuplicateAssembly;
}

UCineAssemblySchema* UCineAssemblyEditorFunctionLibrary::CreateSchema(const FString& SchemaName, const FString& AssetPath, const FString& DefaultAssemblyName, const FString& RelativeAssemblyPath, const FString& Description, UCineAssemblySchema* ParentSchema, UTexture2D* ThumbnailImage, bool bIsDataOnly, bool bIsHidden)
{
	if (SchemaName.IsEmpty())
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not create Schema because the SchemaName is empty");
		return nullptr;
	}

	FString CreatePath;
	bool bPathIsValid = UE::CineAssemblyTools::Private::NormalizeAndValidateContentPath(AssetPath, CreatePath);
	if (!bPathIsValid)
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not create Schema because the path '%ls' is not valid", *AssetPath);
		return nullptr;
	}

	// Create a transient schema and configure its properties
	UCineAssemblySchema* NewSchema = NewObject<UCineAssemblySchema>(GetTransientPackage(), NAME_None, RF_Transient);
	NewSchema->SchemaName = SchemaName;
	NewSchema->DefaultAssemblyName = DefaultAssemblyName;
	NewSchema->Description = Description;
	NewSchema->ThumbnailTexture = ThumbnailImage;
	NewSchema->bIsDataOnly = bIsDataOnly;
	NewSchema->bIsHidden = bIsHidden;
	NewSchema->ParentSchema = ParentSchema ? FSoftObjectPath(ParentSchema) : FSoftObjectPath();
	NewSchema->SetDefaultAssemblyPath(RelativeAssemblyPath);

	// Persist the transient schema to a valid package
	UCineAssemblySchemaFactory::CreateConfiguredSchema(NewSchema, CreatePath);

	// Verify the schema was successfully created (i.e. it is no longer in the transient package)
	if (NewSchema->GetPackage() == GetTransientPackage())
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Failed to create Schema '%ls' at path '%ls'", *SchemaName, *AssetPath);
		return nullptr;
	}

	return NewSchema;
}

UMovieSceneSubAssemblySection* UCineAssemblyEditorFunctionLibrary::AddSubAssemblyTemplate(UCineAssemblySchema* Schema, ESubAssemblyTrackType TrackType, const FString& SubAssemblyName, FName Label, const FString& RelativePath, UObject* TemplateObject)
{
	if (TemplateObject == Schema)
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not add SubAssembly template because the TemplateObject references the owning Schema, which would cause infinite recursion when creating an assembly");
		return nullptr;
	}

	// Only LevelSequences, CineAssemblies, and CineAssemblySchemas are valid template objects (null is also allowed for an empty template)
	if (TemplateObject
		&& !TemplateObject->IsA<ULevelSequence>()
		&& !TemplateObject->IsA<UCineAssembly>()
		&& !TemplateObject->IsA<UCineAssemblySchema>())
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not add SubAssembly template because the TemplateObject '%ls' is not a LevelSequence, CineAssembly, or CineAssemblySchema", *TemplateObject->GetPathName());
		return nullptr;
	}

	UMovieSceneSubAssemblySection* NewSection = UE::CineAssemblyTools::Private::AddSubAssemblyTrackAndSection(Schema, TrackType);
	if (NewSection)
	{
		NewSection->SectionType = ESubAssemblySectionType::Template;
		NewSection->SetAssemblyTemplate(TemplateObject);
		NewSection->SetSequenceName(FText::AsCultureInvariant(SubAssemblyName));
		NewSection->SetSequencePath(RelativePath);

		if (Label.IsNone())
		{
			NewSection->SetDefaultLabel();
		}
		else
		{
			NewSection->Label = Label;
		}
	}
	return NewSection;
}

UMovieSceneSubAssemblySection* UCineAssemblyEditorFunctionLibrary::AddSubAssemblyReference(UCineAssemblySchema* Schema, ESubAssemblyTrackType TrackType, UMovieSceneSequence* Sequence)
{
	UMovieSceneSubAssemblySection* NewSection = UE::CineAssemblyTools::Private::AddSubAssemblyTrackAndSection(Schema, TrackType);
	if (NewSection)
	{
		NewSection->SectionType = ESubAssemblySectionType::Reference;
		NewSection->SetSequence(Sequence);
	}
	return NewSection;
}

FString UCineAssemblyEditorFunctionLibrary::GetDefaultAssetNameForClass(const UClass* AssetClass)
{
	if (!AssetClass)
	{
		return FString();
	}

	const IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	for (UFactory* Factory : AssetTools.GetNewAssetFactories())
	{
		if (Factory->GetSupportedClass() == AssetClass)
		{
			const FString FactoryDefault = Factory->GetDefaultNewAssetName();
			if (!FactoryDefault.IsEmpty())
			{
				return FactoryDefault;
			}
			break;
		}
	}

	return FString::Printf(TEXT("New%s"), *AssetClass->GetName());
}

FName UCineAssemblyEditorFunctionLibrary::MakeDefaultAssociatedAssetLabel(const UCineAssemblySchema* Schema, const UClass* AssetClass)
{
	if (!Schema || !AssetClass)
	{
		return NAME_None;
	}

	const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(AssetClass);
	const FText DisplayName = AssetDefinition ? AssetDefinition->GetAssetDisplayName() : AssetClass->GetDisplayNameText();
	return Schema->MakeUniqueAssetLabel(DisplayName.ToString());
}

FGuid UCineAssemblyEditorFunctionLibrary::AddAssociatedAsset(UCineAssemblySchema* Schema, TSubclassOf<UObject> AssetClass, const FString& AssetName, FName Label, const FString& RelativePath, UObject* TemplateAsset)
{
	UCineAssembly* TemplateSequence = UE::CineAssemblyTools::Private::GetTemplateSequence(Schema);
	if (!TemplateSequence)
	{
		return {};
	}
	if (!AssetClass)
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not add Associated Asset because the AssetClass is null");
		return {};
	}
	if (TemplateAsset && !TemplateAsset->IsA(AssetClass))
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Warning, "TemplateAsset '%ls' is not a '%ls' and will be ignored for the new associated asset.", *TemplateAsset->GetPathName(), *AssetClass->GetName());
		TemplateAsset = nullptr;
	}

	// CineAssemblies belong in SubAssembly tracks rather than the AssociatedAssets list. Route the call to AddSubAssemblyTemplate.
	if (AssetClass->IsChildOf(UCineAssembly::StaticClass()))
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Log, "Attempted associated asset of type CineAssembly was routed to AddSubAssemblyTemplate instead. Consider calling AddSubAssemblyTemplate directly to suppress this message.");
		AddSubAssemblyTemplate(Schema, ESubAssemblyTrackType::SubsequenceTrack, AssetName, Label, RelativePath, TemplateAsset);
		return {};
	}

	// Check the value of AssetClass to ensure that it is a valid asset class with a valid factory that can create it.
	const IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	const bool bHasFactory = AssetTools.GetNewAssetFactories().ContainsByPredicate([&AssetClass](const UFactory* Factory) 
		{ 
			return Factory && AssetClass->IsChildOf(Factory->GetSupportedClass()); 
		});

	if (!bHasFactory)
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Could not add Associated Asset because '%ls' is not a valid asset class (it has no registered asset factory).", *AssetClass->GetName());
		return {};
	}

	FAssemblyAssociatedAssetDesc NewAssetDesc;
	NewAssetDesc.AssetClass = AssetClass.Get();
	NewAssetDesc.AssetID = FGuid::NewGuid();
	NewAssetDesc.RelativePath.Template = RelativePath;
	NewAssetDesc.TemplateAsset = TemplateAsset;
	NewAssetDesc.AssetName.Template = AssetName.IsEmpty() ? GetDefaultAssetNameForClass(AssetClass) : AssetName;
	NewAssetDesc.Label = Label.IsNone() ? MakeDefaultAssociatedAssetLabel(Schema, AssetClass) : Label;

	TemplateSequence->Modify();
	TemplateSequence->AssociatedAssets.Add(NewAssetDesc);

	return NewAssetDesc.AssetID;
}

void UCineAssemblyEditorFunctionLibrary::RemoveAssociatedAsset(UCineAssemblySchema* Schema, FGuid AssetID)
{
	UCineAssembly* TemplateSequence = UE::CineAssemblyTools::Private::GetTemplateSequence(Schema);
	if (!TemplateSequence)
	{
		return;
	}

	const int32 RemoveIndex = TemplateSequence->AssociatedAssets.IndexOfByPredicate([&AssetID](const FAssemblyAssociatedAssetDesc& Desc) { return Desc.AssetID == AssetID; });
	if (RemoveIndex != INDEX_NONE)
	{
		TemplateSequence->Modify();
		TemplateSequence->AssociatedAssets.RemoveAt(RemoveIndex);
		TemplateSequence->ValidateMetadataLinks();
	}
}

FAssemblyAssociatedAssetDesc UCineAssemblyEditorFunctionLibrary::GetSchemaAssociatedAssetDesc(UCineAssemblySchema* Schema, FGuid AssetID)
{
	return GetAssemblyAssociatedAssetDesc(Schema ? Schema->TemplateSequence : nullptr, AssetID);
}

FAssemblyAssociatedAssetDesc UCineAssemblyEditorFunctionLibrary::GetAssemblyAssociatedAssetDesc(UCineAssembly* Assembly, FGuid AssetID)
{
	const FAssemblyAssociatedAssetDesc* AssetDesc = UE::CineAssemblyTools::Private::FindAssociatedAsset(Assembly, AssetID);
	return AssetDesc ? *AssetDesc : FAssemblyAssociatedAssetDesc{};
}

TArray<FAssemblyAssociatedAssetDesc> UCineAssemblyEditorFunctionLibrary::GetSchemaAssociatedAssetDescs(UCineAssemblySchema* Schema)
{
	return GetAssemblyAssociatedAssetDescs(Schema ? Schema->TemplateSequence : nullptr);
}

TArray<FAssemblyAssociatedAssetDesc> UCineAssemblyEditorFunctionLibrary::GetAssemblyAssociatedAssetDescs(UCineAssembly* Assembly)
{
	return Assembly ? Assembly->AssociatedAssets : TArray<FAssemblyAssociatedAssetDesc>{};
}

void UCineAssemblyEditorFunctionLibrary::SetAssociatedAssetClass(UCineAssemblySchema* Schema, FGuid AssetID, TSubclassOf<UObject> NewAssetClass)
{
	using namespace UE::CineAssemblyTools::Private;
	UCineAssembly* TemplateSequence = GetTemplateSequence(Schema);
	if (!TemplateSequence)
	{
		return;
	}
	if (!NewAssetClass)
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Set Associated Asset Class failed because the NewAssetClass is null");
		return;
	}

	FAssemblyAssociatedAssetDesc* AssetDesc = FindAssociatedAsset(TemplateSequence, AssetID);
	if (!AssetDesc)
	{
		return;
	}

	TemplateSequence->Modify();
	AssetDesc->AssetClass = NewAssetClass.Get();

	// Clear the template asset if it is no longer compatible with the new asset class.
	if (!AssetDesc->TemplateAsset.IsNull())
	{
		const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		const FAssetData TemplateAssetData = AssetRegistry.GetAssetByObjectPath(AssetDesc->TemplateAsset.ToSoftObjectPath());
		if (UClass* TemplateClass = TemplateAssetData.GetClass())
		{
			if (!TemplateClass->IsChildOf(NewAssetClass))
			{
				UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Warning, "Set Associated Asset Class cleared the TemplateAsset '%ls' on descriptor '%ls' because it is not a '%ls'", *AssetDesc->TemplateAsset.ToString(), *AssetID.ToString(), *NewAssetClass->GetName());
				AssetDesc->TemplateAsset.Reset();
			}
		}
	}
}

void UCineAssemblyEditorFunctionLibrary::SetAssociatedAssetRelativePath(UCineAssemblySchema* Schema, FGuid AssetID, const FString& NewRelativePath)
{
	UCineAssembly* TemplateSequence = UE::CineAssemblyTools::Private::GetTemplateSequence(Schema);
	if (!TemplateSequence)
	{
		return;
	}

	FAssemblyAssociatedAssetDesc* AssetDesc = UE::CineAssemblyTools::Private::FindAssociatedAsset(TemplateSequence, AssetID);
	if (!AssetDesc)
	{
		return;
	}

	TemplateSequence->Modify();
	AssetDesc->RelativePath.Template = NewRelativePath;
	AssetDesc->RelativePath.Resolved = FText();
}

void UCineAssemblyEditorFunctionLibrary::SetAssociatedAssetTemplate(UCineAssemblySchema* Schema, FGuid AssetID, UObject* NewTemplateAsset)
{
	UCineAssembly* TemplateSequence = UE::CineAssemblyTools::Private::GetTemplateSequence(Schema);
	if (!TemplateSequence)
	{
		return;
	}

	FAssemblyAssociatedAssetDesc* AssetDesc = UE::CineAssemblyTools::Private::FindAssociatedAsset(TemplateSequence, AssetID);
	if (!AssetDesc)
	{
		return;
	}

	if (NewTemplateAsset)
	{
		UClass* RequiredClass = AssetDesc->AssetClass.LoadSynchronous();
		if (RequiredClass && !NewTemplateAsset->IsA(RequiredClass))
		{
			UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Set Associated Asset Template failed because the TemplateAsset '%ls' is not a '%ls'", *NewTemplateAsset->GetPathName(), *RequiredClass->GetName());
			return;
		}
	}

	TemplateSequence->Modify();
	AssetDesc->TemplateAsset = NewTemplateAsset;
}

void UCineAssemblyEditorFunctionLibrary::SetSchemaAssociatedAssetName(UCineAssemblySchema* Schema, FGuid AssetID, const FString& NewAssetName)
{
	if (UCineAssembly* TemplateSequence = UE::CineAssemblyTools::Private::GetTemplateSequence(Schema))
	{
		SetAssemblyAssociatedAssetName(TemplateSequence, AssetID, NewAssetName);
	}
}

void UCineAssemblyEditorFunctionLibrary::SetSchemaAssociatedAssetLabel(UCineAssemblySchema* Schema, FGuid AssetID, FName NewLabel)
{
	if (UCineAssembly* TemplateSequence = UE::CineAssemblyTools::Private::GetTemplateSequence(Schema))
	{
		SetAssemblyAssociatedAssetLabel(TemplateSequence, AssetID, NewLabel);
	}
}

void UCineAssemblyEditorFunctionLibrary::SetAssemblyAssociatedAssetName(UCineAssembly* Assembly, FGuid AssetID, const FString& NewAssetName)
{
	FAssemblyAssociatedAssetDesc* AssetDesc = UE::CineAssemblyTools::Private::FindAssociatedAsset(Assembly, AssetID);
	if (!AssetDesc)
	{
		return;
	}

	if (!AssetDesc->CreatedAsset.IsNull())
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Set Associated Asset Name failed because the asset for descriptor '%ls' has already been created at '%ls'; changing the name template would diverge from the on-disk asset", *AssetID.ToString(), *AssetDesc->CreatedAsset.ToString());
		return;
	}

	Assembly->Modify();
	AssetDesc->AssetName.Template = NewAssetName;
	AssetDesc->AssetName.Resolved = FText();
}

void UCineAssemblyEditorFunctionLibrary::SetAssemblyAssociatedAssetLabel(UCineAssembly* Assembly, FGuid AssetID, FName NewLabel)
{
	FAssemblyAssociatedAssetDesc* AssetDesc = UE::CineAssemblyTools::Private::FindAssociatedAsset(Assembly, AssetID);
	if (!AssetDesc)
	{
		return;
	}

	Assembly->Modify();
	AssetDesc->Label = NewLabel;
}

void UCineAssemblyEditorFunctionLibrary::SetAssemblyAssociatedAssetShouldCreate(UCineAssembly* Assembly, FGuid AssetID, bool bShouldCreate)
{
	FAssemblyAssociatedAssetDesc* AssetDesc = UE::CineAssemblyTools::Private::FindAssociatedAsset(Assembly, AssetID);
	if (!AssetDesc)
	{
		return;
	}

	if (!AssetDesc->CreatedAsset.IsNull())
	{
		UE_LOGF(LogCineAssemblyEditorFunctionLibrary, Error, "Set Associated Asset Should Create failed because the asset for descriptor '%ls' has already been created at '%ls'; the flag is meaningless after creation", *AssetID.ToString(), *AssetDesc->CreatedAsset.ToString());
		return;
	}

	Assembly->Modify();
	AssetDesc->bShouldCreate = bShouldCreate;
}

FNamingTokenResultData UCineAssemblyEditorFunctionLibrary::EvaluateTokenString(const FString& TokenString, UCineAssembly* Assembly)
{
	UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();
	if (!NamingTokensSubsystem)
	{
		return FNamingTokenResultData();
	}

	FNamingTokenFilterArgs FilterArgs;
	FilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

	return NamingTokensSubsystem->EvaluateTokenString(TokenString, FilterArgs, { BuildTokenContext(Assembly) });
}

UObject* UCineAssemblyEditorFunctionLibrary::BuildTokenContext(UCineAssembly* Assembly)
{
	UCineAssemblyNamingTokensContext* Context = NewObject<UCineAssemblyNamingTokensContext>();
	Context->Assembly = Assembly;
	return Context;
}

TArray<FAssetData> UCineAssemblyEditorFunctionLibrary::FindAssembliesBySchema(const UCineAssemblySchema* Schema)
{
	TArray<FAssetData> Results;

	FARFilter Filter;
	Filter.ClassPaths.Add(UCineAssembly::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;

	// Assemblies with no schema still have a tag that equals an invalid FGuid, so passing FGuid() here filters for assemblies that have no schema set.
	const FGuid SchemaGuid = Schema ? Schema->GetSchemaGuid() : FGuid();
	Filter.TagsAndValues.Add(UCineAssembly::AssetRegistryTag_SchemaGuid, SchemaGuid.ToString());

	const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	AssetRegistry.GetAssets(Filter, Results);

	return Results;
}
