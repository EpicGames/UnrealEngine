// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblyEditorSubsystem.h"

#include "CineAssembly.h"

void UCineAssemblyEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	// OnAssemblyMetadataChanged is broadcast from the runtime module (UCineAssembly::SetMetadataAs*), which cannot depend on this editor-only subsystem.
	// The runtime module fires a native multicast, which we subscribe to and re-broadcast as a dynamic delegate
	OnAssemblyMetadataChangedHandle = UCineAssembly::OnAssemblyMetadataChanged().AddLambda([this](UCineAssembly* Assembly, const FString& Key)
		{
			OnAssemblyMetadataChanged.Broadcast(Assembly, Key);
		});
}

void UCineAssemblyEditorSubsystem::Deinitialize()
{
	UCineAssembly::OnAssemblyMetadataChanged().Remove(OnAssemblyMetadataChangedHandle);

	Super::Deinitialize();
}
