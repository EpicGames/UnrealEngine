// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblyFactory.h"

#include "Algo/Find.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetViewUtils.h"
#include "AssetToolsModule.h"
#include "CineAssemblyEditorSubsystem.h"
#include "CineAssemblyNamingTokens.h"
#include "CineAssemblySchema.h"
#include "CineAssemblyToolsAnalytics.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Engine/LevelStreaming.h"
#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "IContentBrowserSingleton.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneToolsProjectSettings.h"
#include "Sections/MovieSceneSubSection.h"
#include "Sequencer/CineAssemblySequencerUtilities.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "UI/CineAssembly/SCineAssemblyConfigWindow.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHandle.h"

#define LOCTEXT_NAMESPACE "CineAssemblyFactory"

DEFINE_LOG_CATEGORY_STATIC(LogCineAssemblyFactory, Log, All);

namespace UE::CineAssemblyTools::Private
{
	/** Finds and instantiates the factory that creates new assets of the given class, or nullptr if none exists */
	UFactory* CreateFactoryForClass(UClass* AssetClass)
	{
		const IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		for (UFactory* FactoryCDO : AssetTools.GetNewAssetFactories())
		{
			if (FactoryCDO->GetSupportedClass() == AssetClass)
			{
				return NewObject<UFactory>(GetTransientPackage(), FactoryCDO->GetClass());
			}
		}
		return nullptr;
	}

	/** Find the AssociatedAsset or SubAssembly linked to each metadata field, and use them to set the matching CineAssembly property or metadata values */
	void PopulateLinkedMetadata(UCineAssembly* Assembly)
	{
		const UCineAssemblySchema* Schema = Assembly->GetSchema();

		for (const TPair<FString, FGuid>& MetadataLink : Assembly->MetadataLinks)
		{
			if (!MetadataLink.Value.IsValid())
			{
				continue;
			}

			// Find the linked AssociatedAsset or SubAssembly
			UObject* LinkedAsset = nullptr;
			if (const FAssemblyAssociatedAssetDesc* AssetDesc = Algo::FindBy(Assembly->AssociatedAssets, MetadataLink.Value, &FAssemblyAssociatedAssetDesc::AssetID))
			{
				LinkedAsset = AssetDesc->CreatedAsset.Get();
			}
			else
			{
				LinkedAsset = Assembly->FindSubAssembly(MetadataLink.Value);
			}

			if (!LinkedAsset)
			{
				continue;
			}

			// If the metadata key refers to a CineAssembly property, set that property directly. Otherwise, add this as metadata on the assembly
			if (Schema && Algo::FindBy(Schema->GetAssemblyPropertyMetadata(), MetadataLink.Key, &FAssemblyMetadataDesc::Key))
			{
				if (MetadataLink.Key == UCineAssemblySchema::DefaultLevelMetadataKey)
				{
					Assembly->Level = FSoftObjectPath(LinkedAsset);
				}
				else
				{
					ensureMsgf(false, TEXT("Unhandled linkable CineAssembly property: %s"), *MetadataLink.Key);
				}
			}
			else
			{
				Assembly->SetMetadataAsString(MetadataLink.Key, LinkedAsset->GetPathName());
			}
		}
	}

	/** Returns NewParentPath / <OriginalAssetPath's folder relative to OriginalParentPath> / <OriginalAssetPath's name> */
	FString GetDuplicatePath(const FString& OriginalAssetPath, const FString& OriginalParentPath, const FString& NewParentPath)
	{
		// Append '/' so MakePathRelativeTo treats OriginalParentPath as a directory
		FString BaseParent = OriginalParentPath;
		BaseParent.AppendChar('/');

		FString RelativePath = FPaths::GetPath(OriginalAssetPath);
		FPaths::MakePathRelativeTo(RelativePath, *BaseParent);

		FString Result = NewParentPath / RelativePath / FPaths::GetBaseFilename(OriginalAssetPath);
		FPaths::CollapseRelativeDirectories(Result);
		return Result;
	}

	/** Returns the desired asset name and path for a duplicate of SequenceToDuplicate, placed relative to ParentSequence */
	FString GetDuplicateSequenceNameAndPath(UMovieSceneSequence* SequenceToDuplicate, UMovieSceneSequence* ParentSequence, const TMap<UMovieSceneSequence*, FString>& OriginalPaths)
	{
		// Find the original path of the parent sequence (in case the parent sequence itself is already a duplicate)
		const FString* OriginalPathPtr = OriginalPaths.Find(ParentSequence);
		const FString OriginalParentPath = OriginalPathPtr ? *OriginalPathPtr : FPaths::GetPath(ParentSequence->GetPathName());

		return GetDuplicatePath(SequenceToDuplicate->GetPathName(), OriginalParentPath, FPaths::GetPath(ParentSequence->GetPathName()));
	}

	/** Returns the desired asset name and path for a duplicate of OriginalSubLevel, mirroring its position relative to OriginalPersistent under NewPersistent's folder */
	FString GetDuplicateSubLevelNameAndPath(const UWorld* OriginalSubLevel, const UWorld* OriginalPersistent, const UWorld* NewPersistent)
	{
		return GetDuplicatePath(OriginalSubLevel->GetPathName(), FPaths::GetPath(OriginalPersistent->GetPathName()), FPaths::GetPath(NewPersistent->GetPathName()));
	}

	/** Applies the external asset placement policy */
	bool ApplyExternalAssetPreference(FString& DesiredNameAndPath, const FString& TopLevelAssemblyPath, EDuplicateExternalAssetPreference ExternalAssetPreference)
	{
		switch (ExternalAssetPreference)
		{
		case EDuplicateExternalAssetPreference::DoNotDuplicate:
			return false;

		case EDuplicateExternalAssetPreference::DuplicateIntoAssemblyFolder:
			DesiredNameAndPath = TopLevelAssemblyPath / FPaths::GetBaseFilename(DesiredNameAndPath);
			return true;

		case EDuplicateExternalAssetPreference::DuplicateIntoOriginalFolder:
			return true;

		default:
			checkNoEntry();
		}

		return true;
	}

	/** Recursively duplicates all of the unmanaged subsequences of the input Sequence, replacing the sequence referenced in the MovieScene with the duplicate */
	void DuplicateSubsequencesImpl(UMovieSceneSequence* ParentSequence, TMap<UMovieSceneSequence*, FString>& OriginalPaths, const FString& TopLevelAssemblyPath, EDuplicateExternalAssetPreference ExternalAssetPreference)
	{
		// If this parent is a UCineAssembly, ensure its original path is in the map so original path lookups work for its subsequences
		if (const UCineAssembly* ParentAssembly = Cast<UCineAssembly>(ParentSequence))
		{
			if (!ParentAssembly->SourceAssemblyPath.IsEmpty() && !OriginalPaths.Contains(ParentSequence))
			{
				OriginalPaths.Add(ParentSequence, ParentAssembly->SourceAssemblyPath);
			}
		}

		UMovieScene* MovieScene = ParentSequence->GetMovieScene();
		if (!MovieScene)
		{
			return;
		}

		const bool bIsReadOnly = MovieScene->IsReadOnly();
		MovieScene->SetReadOnly(false);

		for (UMovieSceneSection* Section : MovieScene->GetAllSections())
		{
			if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
			{
				if (UMovieSceneSequence* Subsequence = SubSection->GetSequence())
				{
					// Managed SubAssemblies are already handled by CreateConfiguredSubAssemblies. Skip duplication but recurse into them.
					UCineAssembly* ParentAssembly = Cast<UCineAssembly>(ParentSequence);
					if (ParentAssembly && ParentAssembly->SubAssemblies.Contains(SubSection))
					{
						DuplicateSubsequencesImpl(Subsequence, OriginalPaths, TopLevelAssemblyPath, ExternalAssetPreference);
						continue;
					}

					FString DesiredNameAndPath = GetDuplicateSequenceNameAndPath(Subsequence, ParentSequence, OriginalPaths);

					if (!FPaths::IsUnderDirectory(DesiredNameAndPath, TopLevelAssemblyPath))
					{
						const bool bShouldDuplicate = ApplyExternalAssetPreference(DesiredNameAndPath, TopLevelAssemblyPath, ExternalAssetPreference);
						if (!bShouldDuplicate)
						{
							// Recurse into the unduplicated subsequence so its own children still get walked
							DuplicateSubsequencesImpl(Subsequence, OriginalPaths, TopLevelAssemblyPath, ExternalAssetPreference);
							continue;
						}
					}

					IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();

					FString UniquePackageName;
					FString UniqueAssetName;
					AssetTools.CreateUniqueAssetName(DesiredNameAndPath, TEXT(""), UniquePackageName, UniqueAssetName);

					const FString DestinationFolder = FPaths::GetPath(UniquePackageName);
					UMovieSceneSequence* DuplicateSubsequence = Cast<UMovieSceneSequence>(AssetTools.DuplicateAsset(UniqueAssetName, DestinationFolder, Subsequence));
					if (!DuplicateSubsequence)
					{
						UE_LOGF(LogCineAssemblyFactory, Warning, "Failed to duplicate subsequence '%ls' under parent '%ls'", *Subsequence->GetPathName(), *ParentSequence->GetPathName());
						continue;
					}

					FCineAssemblySequencerUtilities::ReplaceSubSequence(SubSection, DuplicateSubsequence);

					// Record the original's path so the recursive call can place its children correctly
					OriginalPaths.Add(DuplicateSubsequence, FPaths::GetPath(Subsequence->GetPathName()));
					DuplicateSubsequencesImpl(DuplicateSubsequence, OriginalPaths, TopLevelAssemblyPath, ExternalAssetPreference);
				}
			}
		}

		MovieScene->SetReadOnly(bIsReadOnly);
	}

	/** Duplicates each streaming sublevel of DuplicatedWorld and rewires the corresponding ULevelStreaming to point at the duplicate */
	void DuplicateSubLevels(const UWorld* OriginalWorld, UWorld* DuplicatedWorld, const FString& TopLevelAssemblyPath, EDuplicateExternalAssetPreference ExternalAssetPreference)
	{
		if (!OriginalWorld || !DuplicatedWorld)
		{
			return;
		}

		DuplicatedWorld->Modify();

		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		for (ULevelStreaming* StreamingLevel : DuplicatedWorld->GetStreamingLevels())
		{
			if (!StreamingLevel)
			{
				continue;
			}

			UWorld* OriginalSubLevel = StreamingLevel->GetWorldAsset().LoadSynchronous();
			if (!OriginalSubLevel)
			{
				continue;
			}

			// Compute the desired name and path for the duplicated sublevel
			FString DesiredNameAndPath = GetDuplicateSubLevelNameAndPath(OriginalSubLevel, OriginalWorld, DuplicatedWorld);

			if (!FPaths::IsUnderDirectory(DesiredNameAndPath, TopLevelAssemblyPath))
			{
				const bool bShouldDuplicate = ApplyExternalAssetPreference(DesiredNameAndPath, TopLevelAssemblyPath, ExternalAssetPreference);
				if (!bShouldDuplicate)
				{
					continue;
				}
			}

			FString UniquePackageName;
			FString UniqueAssetName;
			AssetTools.CreateUniqueAssetName(DesiredNameAndPath, FString(), UniquePackageName, UniqueAssetName);

			UWorld* NewSubLevel = Cast<UWorld>(AssetTools.DuplicateAsset(UniqueAssetName, FPaths::GetPath(UniquePackageName), OriginalSubLevel));
			if (!NewSubLevel)
			{
				UE_LOGF(LogCineAssemblyFactory, Warning, "Failed to duplicate sublevel '%ls' for assembly's persistent level '%ls'", *OriginalSubLevel->GetPathName(), *DuplicatedWorld->GetPathName());
				continue;
			}

			StreamingLevel->Modify();
			StreamingLevel->SetWorldAsset(TSoftObjectPtr<UWorld>(NewSubLevel));
		}
	}

	/** Creates the asset for AssetDesc inside Assembly's content tree. If SourceAsset is valid, it will be duplicated. Otherwise, a new asset will be created. */
	UObject* CreateAssetFromSource(UObject* SourceAsset, UCineAssembly* Assembly, FAssemblyAssociatedAssetDesc& AssetDesc, EDuplicateExternalAssetPreference ExternalAssetPreference)
	{
		if (!Assembly)
		{
			return nullptr;
		}

		UClass* AssetClass = AssetDesc.AssetClass.LoadSynchronous();
		if (!AssetClass)
		{
			return nullptr;
		}

		// Guard against a mismatched source: refuse to duplicate if SourceAsset isn't an instance of the descriptor's declared AssetClass.
		if (SourceAsset && !SourceAsset->IsA(AssetClass))
		{
			UE_LOGF(LogCineAssemblyFactory, Warning, "Source asset '%ls' (class %ls) does not match the associated asset class '%ls' for assembly '%ls'; skipping",
				*SourceAsset->GetPathName(),
				*SourceAsset->GetClass()->GetName(),
				*AssetClass->GetName(),
				*Assembly->GetPathName());
			return nullptr;
		}

		FString AssetPath;
		FString RootFolder;
		Assembly->GetAssetPathAndRootFolder(AssetPath, RootFolder);

		const FText ResolvedName = UCineAssemblyNamingTokens::GetResolvedText(AssetDesc.AssetName.Template, Assembly);
		const FText ResolvedPath = UCineAssemblyNamingTokens::GetResolvedText(AssetDesc.RelativePath.Template, Assembly);
		const FString DestinationPath = RootFolder / ResolvedPath.ToString();

		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		FString UniquePackageName;
		FString UniqueAssetName;
		AssetTools.CreateUniqueAssetName(DestinationPath / ResolvedName.ToString(), FString(), UniquePackageName, UniqueAssetName);

		// Validate the final package path length before attempting to create a new asset
		FText InvalidPackageError;
		if (!AssetViewUtils::IsValidPackageForCooking(UniquePackageName, InvalidPackageError))
		{
			UE_LOGF(LogCineAssemblyFactory, Warning, "Cannot create associated asset at '%ls' for assembly '%ls': %ls", *UniquePackageName, *Assembly->GetPathName(), *InvalidPackageError.ToString());
			return nullptr;
		}

		if (!SourceAsset)
		{
			UFactory* Factory = CreateFactoryForClass(AssetClass);
			UObject* NewAsset = AssetTools.CreateAsset(UniqueAssetName, FPaths::GetPath(UniquePackageName), AssetClass, Factory);

			if (!NewAsset)
			{
				UE_LOGF(LogCineAssemblyFactory, Warning, "Failed to create associated asset at '%ls' for assembly '%ls'", *UniquePackageName, *Assembly->GetPathName());
			}

			return NewAsset;
		}

		// Special case handling for World Partition levels. Load all external actor packages into memory so that they are properly duplicated.
		// The references go out of scope when this function returns, releasing the loaded actors.
		TArray<FWorldPartitionReference> ActorReferences;
		if (UWorld* SourceWorld = Cast<UWorld>(SourceAsset))
		{
			if (UWorldPartition* Partition = SourceWorld->GetWorldPartition())
			{
				if (!Partition->IsInitialized())
				{
					Partition->Initialize(SourceWorld, FTransform::Identity);
				}
				Partition->LoadAllActors(ActorReferences);
			}
		}

		UObject* DuplicateAsset = AssetTools.DuplicateAsset(UniqueAssetName, FPaths::GetPath(UniquePackageName), SourceAsset);
		if (!DuplicateAsset)
		{
			UE_LOGF(LogCineAssemblyFactory, Warning, "Failed to duplicate associated asset '%ls' for assembly '%ls'", *SourceAsset->GetPathName(), *Assembly->GetPathName());
			return nullptr;
		}

		// Special case handling for Level assets to handle duplication of sublevels
		if (UWorld* DuplicatedWorld = Cast<UWorld>(DuplicateAsset))
		{
			const UWorld* OriginalWorld = Cast<UWorld>(SourceAsset);
			DuplicateSubLevels(OriginalWorld, DuplicatedWorld, RootFolder, ExternalAssetPreference);
		}

		return DuplicateAsset;
	}
}

UCineAssemblyFactory::UCineAssemblyFactory()
{
	SupportedClass = UCineAssembly::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

bool UCineAssemblyFactory::CanCreateNew() const
{
	return true;
}

bool UCineAssemblyFactory::ConfigureProperties()
{
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	const FString CurrentPath = ContentBrowser.GetCurrentPath().GetInternalPathString();

	TSharedRef<SCineAssemblyConfigWindow> CineAssemblyConfigWindow = SNew(SCineAssemblyConfigWindow, CurrentPath);

	const IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	const TSharedPtr<SWindow> ParentWindow = MainFrameModule.GetParentWindow();

	if (ParentWindow.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(CineAssemblyConfigWindow, ParentWindow.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(CineAssemblyConfigWindow);
	}

	return false;
}

UObject* UCineAssemblyFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	// Procedural assembly creation that does not use the configuration window will hit this path
	UCineAssembly* NewAssembly = NewObject<UCineAssembly>(InParent, Name, Flags);
	NewAssembly->Initialize();

	return NewAssembly;
}

FString UCineAssemblyFactory::MakeAssemblyPackageName(UCineAssembly* ConfiguredAssembly, const FString& CreateAssetPath)
{
	// Resolve the assembly's path relative to its content root
	ConfiguredAssembly->PathRelativeToRoot.Resolved = UCineAssemblyNamingTokens::GetResolvedText(ConfiguredAssembly->PathRelativeToRoot.Template, ConfiguredAssembly);

	return CreateAssetPath / ConfiguredAssembly->PathRelativeToRoot.Resolved.ToString() / ConfiguredAssembly->AssemblyName.Resolved.ToString();
}

bool UCineAssemblyFactory::MakeUniqueNameAndPath(UCineAssembly* ConfiguredAssembly, const FString& CreateAssetPath, FString& UniquePackageName, FString& UniqueAssetName)
{
	const FString DesiredPackageName = MakeAssemblyPackageName(ConfiguredAssembly, CreateAssetPath);

	// Ensure the package name length does not exceed the maximum cook path length as this may cause issues later on.
	FText InvalidPackageError;
	if (!AssetViewUtils::IsValidPackageForCooking(DesiredPackageName, InvalidPackageError))
	{
		UE_LOGF(LogCineAssemblyFactory, Warning, "Cannot create assembly at '%ls': %ls", *DesiredPackageName, *InvalidPackageError.ToString());
		return false;
	}

	// Ensure that the resolved assembly name is actually unique
	const FString AssemblyName = ConfiguredAssembly->AssemblyName.Resolved.ToString();
	const FString DesiredSuffix = TEXT("");

	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
	AssetTools.CreateUniqueAssetName(DesiredPackageName, DesiredSuffix, UniquePackageName, UniqueAssetName);

	// If the assembly name was not unique, update the assembly template string
	if (UniqueAssetName != AssemblyName)
	{
		ConfiguredAssembly->AssemblyName.Template = UniqueAssetName;
		ConfiguredAssembly->AssemblyName.Resolved = FText::FromString(UniqueAssetName);

		// It is possible that the default assembly path was dependent on the assembly name (through tokens).
		// By updating the assembly name, we may also need to reevaluate the assembly path.
		// However, once we change the assembly path, we need to verify uniqueness again (in the new path).
		// Therefore, we recursively call this function until we end up with a combination of a unique assembly path and name.
		return MakeUniqueNameAndPath(ConfiguredAssembly, CreateAssetPath, UniquePackageName, UniqueAssetName);
	}

	return true;
}

bool UCineAssemblyFactory::CreateConfiguredAssembly(UCineAssembly* ConfiguredAssembly, const FString& CreateAssetPath)
{
	// The intended use of this function is to take a pre-configured, transient assembly, create a valid package for it, and initialize it.
	// If the input Assembly has already been persisted, early-out but treat as a success so callers know the ConfiguredAssembly is already valid.
	if (ConfiguredAssembly->GetPackage() != GetTransientPackage())
	{
		return true;
	}

	// Evaluate the name of the assembly
	ConfiguredAssembly->AssemblyName.Resolved = UCineAssemblyNamingTokens::GetResolvedText(ConfiguredAssembly->AssemblyName.Template, ConfiguredAssembly);

	// If the assembly name is empty or invalid, assign it a valid default name
	if (ConfiguredAssembly->AssemblyName.Resolved.IsEmpty() || FName(*ConfiguredAssembly->AssemblyName.Resolved.ToString()).IsNone())
	{
		ConfiguredAssembly->AssemblyName.Resolved = LOCTEXT("NewCineAssemblyName", "NewCineAssembly");
		ConfiguredAssembly->AssemblyName.Template = ConfiguredAssembly->AssemblyName.Resolved.ToString();
	}

	FString UniquePackageName;
	FString UniqueAssetName;
	if (!MakeUniqueNameAndPath(ConfiguredAssembly, CreateAssetPath, UniquePackageName, UniqueAssetName))
	{
		return false;
	}

	// The input assembly object was created in the transient package while its properties were configured.
	// Now, we can create a real package for it, rename it, and update its object flags.
	UPackage* Package = CreatePackage(*UniquePackageName);
	ConfiguredAssembly->Rename(*UniqueAssetName, Package);

	const EObjectFlags Flags = RF_Public | RF_Standalone | RF_Transactional;
	ConfiguredAssembly->SetFlags(Flags);
	ConfiguredAssembly->ClearFlags(RF_Transient);

	// Re-evaluate the assembly's metadata tokens
	if (const UCineAssemblySchema* Schema = ConfiguredAssembly->GetSchema())
	{
		for (const FAssemblyMetadataDesc& MetadataDesc : Schema->AssemblyMetadata)
		{
			if (MetadataDesc.Type == ECineAssemblyMetadataType::String && MetadataDesc.bEvaluateTokens)
			{
				FTemplateString TemplateString;
				if (ConfiguredAssembly->GetMetadataAsTokenString(MetadataDesc.Key, TemplateString))
				{
					TemplateString.Resolved = UCineAssemblyNamingTokens::GetResolvedText(TemplateString.Template, ConfiguredAssembly);
					ConfiguredAssembly->SetMetadataAsTokenString(MetadataDesc.Key, TemplateString);
				}
			}
		}
	}

	CreateSubFolders(ConfiguredAssembly);
	ConfiguredAssembly->ResolveMovieSceneTokens();

	// Create packages for each of the transient SubAssembly
	CreateConfiguredSubAssemblies(ConfiguredAssembly, CreateAssetPath);

	// Create any associated assets defined by the schema
	CreateAssociatedAssets(ConfiguredAssembly);

	// Analytics
	UE::CineAssemblyToolsAnalytics::RecordEvent_CreateAssembly(ConfiguredAssembly);

	// Notify the asset registry about the new assembly
	FAssetRegistryModule::AssetCreated(ConfiguredAssembly);

	// Mark the package dirty
	Package->MarkPackageDirty();

	// Refresh the content browser to make any new assets and folders are immediately visible
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	ContentBrowser.SetSelectedPaths({ CreateAssetPath }, true);

	if (UCineAssemblyEditorSubsystem* EditorSubsystem = GEditor ? GEditor->GetEditorSubsystem<UCineAssemblyEditorSubsystem>() : nullptr)
	{
		EditorSubsystem->OnAssemblyCreated.Broadcast(ConfiguredAssembly);
	}

	return true;
}

void UCineAssemblyFactory::CreateConfiguredSubAssemblies(UCineAssembly* ConfiguredAssembly, const FString& CreateAssetPath)
{
	// Any SubAssemblies that are not created will be removed from the Assembly's SubAssemblies list and MovieScene at the end
	TArray<UMovieSceneSubSection*> SectionsToRemove;

	for (UMovieSceneTrack* Track : ConfiguredAssembly->GetMovieScene()->GetTracks())
	{
		if (UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
		{
			for (UMovieSceneSection* Section : SubTrack->GetAllSections())
			{
				if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
				{
					// If the SubSequence is a transient CineAssembly, create a new package for it
					UCineAssembly* SubAssembly = Cast<UCineAssembly>(SubSection->GetSequence());
					if (SubAssembly && SubAssembly->GetPackage() == GetTransientPackage())
					{
						if (!SubAssembly->bShouldCreate)
						{
							SectionsToRemove.Add(SubSection);
							continue;
						}

						SubAssembly->ParentAssembly = ConfiguredAssembly;
						SubAssembly->Level = ConfiguredAssembly->Level;
						SubAssembly->Production = ConfiguredAssembly->Production;
						SubAssembly->ProductionName = ConfiguredAssembly->ProductionName;

						SubAssembly->PathRelativeToParent.Resolved = UCineAssemblyNamingTokens::GetResolvedText(SubAssembly->PathRelativeToParent.Template, ConfiguredAssembly);

						// If SubAssembly creation fails, drop the dangling section so the parent isn't left pointing at a transient SubAssembly
						const bool bResult = CreateConfiguredAssembly(SubAssembly, CreateAssetPath / SubAssembly->PathRelativeToParent.Resolved.ToString());
						if (!bResult)
						{
							UE_LOGF(LogCineAssemblyFactory, Warning, "Failed to create SubAssembly '%ls'. Removing it from parent '%ls'.",
								*SubAssembly->AssemblyName.Resolved.ToString(),
								*ConfiguredAssembly->GetPathName());

							SectionsToRemove.Add(SubSection);
						}
					}
				}
			}
		}
	}

	// Remove skipped sections from their tracks and from the SubAssemblies array.
	for (UMovieSceneSubSection* SectionToRemove : SectionsToRemove)
	{
		if (UMovieSceneTrack* OwningTrack = SectionToRemove->GetTypedOuter<UMovieSceneTrack>())
		{
			OwningTrack->RemoveSection(*SectionToRemove);
			if (OwningTrack->GetAllSections().IsEmpty())
			{
				ConfiguredAssembly->GetMovieScene()->RemoveTrack(*OwningTrack);
			}
		}
		ConfiguredAssembly->SubAssemblies.Remove(SectionToRemove);
	}

	// Clean up any MetadataLinks entries that referenced the removed SubAssemblies' GUIDs.
	if (!SectionsToRemove.IsEmpty())
	{
		ConfiguredAssembly->ValidateMetadataLinks();
	}
}

void UCineAssemblyFactory::CreateSubFolders(UCineAssembly* Assembly)
{
	if (!Assembly)
	{
		return;
	}

	// Get the path where the top-level assembly will be created so we can create other assets relative to it
	FString AssemblyPath;
	FString AssemblyRootFolder;
	Assembly->GetAssetPathAndRootFolder(AssemblyPath, AssemblyRootFolder);

	// Create the default folders for this assembly, based on the schema
	for (FTemplateString& FolderPath : Assembly->DefaultFolderNames)
	{
		// Resolve any tokens found in the folder template before attempting to create it
		FolderPath.Resolved = UCineAssemblyNamingTokens::GetResolvedText(FolderPath.Template, Assembly);

		if (FolderPath.Resolved.IsEmpty())
		{
			continue;
		}

		const FString PathToCreate = AssemblyRootFolder / FolderPath.Resolved.ToString();

		// Validate before touching the filesystem. AssetViewUtils::IsValidPackageForCooking checks the projected cooked path length,
		// which is the right gate here because the schema's per-folder paths aren't covered by the assembly's own MakeUniqueNameAndPath validation.
		FText InvalidPackageError;
		if (!AssetViewUtils::IsValidPackageForCooking(PathToCreate, InvalidPackageError))
		{
			UE_LOGF(LogCineAssemblyFactory, Warning, "Cannot create folder '%ls' for assembly '%ls': %ls", *PathToCreate, *Assembly->GetPathName(), *InvalidPackageError.ToString());
			continue;
		}

		const FString RelativeFilePath = FPackageName::LongPackageNameToFilename(PathToCreate);
		const FString AbsoluteFilePath = FPaths::ConvertRelativePathToFull(RelativeFilePath);

		// Create the directory on disk, then add its path to the asset registry so it appears in Content Browser
		if (IFileManager::Get().DirectoryExists(*AbsoluteFilePath))
		{
			continue;
		}

		constexpr bool bCreateParentFoldersIfMissing = true;
		if (IFileManager::Get().MakeDirectory(*AbsoluteFilePath, bCreateParentFoldersIfMissing))
		{
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
			AssetRegistryModule.Get().AddPath(PathToCreate);
		}
		else
		{
			UE_LOGF(LogCineAssemblyFactory, Warning, "Failed to create folder '%ls' for assembly '%ls'", *AbsoluteFilePath, *Assembly->GetPathName());
		}
	}
}

void UCineAssemblyFactory::DuplicateSubsequences(UCineAssembly* Assembly, EDuplicateExternalAssetPreference ExternalAssetPreference)
{
	const FString AssemblyPath = FPaths::GetPath(Assembly->GetPathName());

	TMap<UMovieSceneSequence*, FString> OriginalPaths;
	UE::CineAssemblyTools::Private::DuplicateSubsequencesImpl(Assembly, OriginalPaths, AssemblyPath, ExternalAssetPreference);
}

void UCineAssemblyFactory::CreateAssociatedAssets(UCineAssembly* ConfiguredAssembly)
{
	using namespace UE::CineAssemblyTools::Private;

	if (!ConfiguredAssembly)
	{
		return;
	}

	// Remove any associated asset descs that should not be created, and clean up any stale metadata links that might be referencing them
	const int32 NumAssociatedAssetsRemoved = ConfiguredAssembly->AssociatedAssets.RemoveAll([](const FAssemblyAssociatedAssetDesc& AssetDesc) { return !AssetDesc.bShouldCreate; });
	if (NumAssociatedAssetsRemoved > 0)
	{
		ConfiguredAssembly->ValidateMetadataLinks();
	}

	for (FAssemblyAssociatedAssetDesc& AssetDesc : ConfiguredAssembly->AssociatedAssets)
	{
		// Skip definitions that already have a created asset (e.g., from a duplicated assembly)
		if (!AssetDesc.CreatedAsset.IsNull())
		{
			continue;
		}

		// If a template is configured, use it as the duplication source.
		UObject* SourceAsset = nullptr;
		if (!AssetDesc.TemplateAsset.IsNull())
		{
			SourceAsset = AssetDesc.TemplateAsset.LoadSynchronous();
			if (!SourceAsset)
			{
				UE_LOGF(LogCineAssemblyFactory, Warning, "Template asset '%ls' could not be loaded for assembly '%ls'. A new asset will be created instead.", *AssetDesc.TemplateAsset.ToString(), *ConfiguredAssembly->GetPathName());
			}
		}

		if (UObject* NewAsset = CreateAssetFromSource(SourceAsset, ConfiguredAssembly, AssetDesc, EDuplicateExternalAssetPreference::DuplicateIntoAssemblyFolder))
		{
			AssetDesc.CreatedAsset = NewAsset;
		}
		else
		{
			UE_LOGF(LogCineAssemblyFactory, Warning, "Failed to create Associated Asset '%ls'. Removing it from assembly '%ls'.", *AssetDesc.AssetName.Template, *ConfiguredAssembly->GetPathName());
		}
	}

	// Remove any asset descriptors that failed to be created so the assembly doesn't persist with dangling entries that have no valid asset.
	const int32 NumRemoved = ConfiguredAssembly->AssociatedAssets.RemoveAll([](const FAssemblyAssociatedAssetDesc& AssetDesc) { return AssetDesc.CreatedAsset.IsNull(); });
	if (NumRemoved > 0)
	{
		ConfiguredAssembly->ValidateMetadataLinks();
	}

	// Auto-populate metadata fields that are linked to associated assets
	PopulateLinkedMetadata(ConfiguredAssembly);
}

void UCineAssemblyFactory::DuplicateAssociatedAsset(UCineAssembly* DuplicatedAssembly, FAssemblyAssociatedAssetDesc& AssetDesc, EDuplicateExternalAssetPreference ExternalAssetPreference)
{
	// The source asset that should be duplicated is the Created Asset from the original asset descriptor.
	UObject* OriginalAsset = AssetDesc.CreatedAsset.LoadSynchronous();
	if (!OriginalAsset)
	{
		return;
	}

	if (UObject* DuplicateAsset = UE::CineAssemblyTools::Private::CreateAssetFromSource(OriginalAsset, DuplicatedAssembly, AssetDesc, ExternalAssetPreference))
	{
		AssetDesc.CreatedAsset = DuplicateAsset;
	}
}

void UCineAssemblyFactory::DuplicateAssociatedAssets(UCineAssembly* DuplicatedAssembly, EDuplicateExternalAssetPreference ExternalAssetPreference)
{
	if (!DuplicatedAssembly)
	{
		return;
	}

	// Only duplicate assets for assemblies that were duplicated from an existing assembly.
	// Assemblies created fresh from a Schema have their assets created by CreateAssociatedAssets instead.
	if (!DuplicatedAssembly->SourceAssemblyPath.IsEmpty())
	{
		for (FAssemblyAssociatedAssetDesc& AssetDesc : DuplicatedAssembly->AssociatedAssets)
		{
			DuplicateAssociatedAsset(DuplicatedAssembly, AssetDesc, ExternalAssetPreference);
		}
	}

	// Auto-populate metadata fields that are linked to associated assets
	UE::CineAssemblyTools::Private::PopulateLinkedMetadata(DuplicatedAssembly);

	// Recurse into managed SubAssemblies
	for (UMovieSceneSubSection* SubSection : DuplicatedAssembly->SubAssemblies)
	{
		UCineAssembly* SubAssembly = SubSection ? Cast<UCineAssembly>(SubSection->GetSequence()) : nullptr;
		if (SubAssembly)
		{
			DuplicateAssociatedAssets(SubAssembly, ExternalAssetPreference);
		}
	}
}

void UCineAssemblyFactory::PopulateLinkedMetadataRecursive(UCineAssembly* Assembly)
{
	if (!Assembly)
	{
		return;
	}

	UE::CineAssemblyTools::Private::PopulateLinkedMetadata(Assembly);

	for (UMovieSceneSubSection* SubSection : Assembly->SubAssemblies)
	{
		UCineAssembly* SubAssembly = SubSection ? Cast<UCineAssembly>(SubSection->GetSequence()) : nullptr;

		// Only recurse into SubAssemblies that this Assembly actually owns
		if (SubAssembly && SubAssembly->ParentAssembly.ResolveObject() == Assembly)
		{
			PopulateLinkedMetadataRecursive(SubAssembly);
		}
	}
}

#undef LOCTEXT_NAMESPACE
