// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "CineAssembly.h"

#include "CineAssemblyFactory.generated.h"

/**
 * Options for handling the duplication of assets (subsequences, sublevels, etc.) that live outside of the source assembly's content tree
 */
UENUM(BlueprintType)
enum class EDuplicateExternalAssetPreference : uint8
{
	DuplicateIntoAssemblyFolder UMETA(ToolTip = "Duplicate external assets into the same folder as the duplicate assembly"),
	DuplicateIntoOriginalFolder UMETA(ToolTip = "Duplicate external assets into the same folder as the original asset"),
	DoNotDuplicate              UMETA(ToolTip = "Do not duplicate external assets. Maintain the reference to the original asset.")
};

/** 
 * Factory class used to create new UCineAssembly objects
 * Before creating a new Cine Assembly, the factory will spawn a new window to configure the properties of the asset that is being created.
 */
UCLASS(hidecategories=Object)
class UCineAssemblyFactory : public UFactory
{
	GENERATED_BODY()

public:
	/**
	 * Takes a pre-configured, transient assembly, creates a valid package for it, and initializes it.
	 * Returns true on success. Returns false if the assembly could not be persisted (e.g. resulting package path was too long).
	 * When called recursively for a SubAssembly, a false return is the caller's signal to remove the dangling section from the parent.
	 */
	static bool CreateConfiguredAssembly(UCineAssembly* ConfiguredAssembly, const FString& CreateAssetPath);

	/** Create a valid package for each SubAssembly found in a SubSequence or Shot Track of the input Assembly's MovieScene */
	static void CreateConfiguredSubAssemblies(UCineAssembly* ConfiguredAssembly, const FString& CreateAssetPath);

	/** Create the content folders specified by the input Assembly's DefaultFolderNames. */
	static void CreateSubFolders(UCineAssembly* Assembly);

	/**
	 * Recursively duplicate all subsequences of the input Assembly. This will skip over managed SubAssemblies which are handled by CreateConfiguredSubAssemblies.
	 * Note: The input assembly should have its SourceAssemblyPath set before calling this, so that relative paths can be computed correctly for the duplicated subsequences.
	 */
	static void DuplicateSubsequences(UCineAssembly* Assembly, EDuplicateExternalAssetPreference ExternalAssetPreference = EDuplicateExternalAssetPreference::DuplicateIntoAssemblyFolder);

	/** Create new assets for each associated asset definition on the input Assembly */
	static void CreateAssociatedAssets(UCineAssembly* ConfiguredAssembly);

	/** Duplicate the input asset associated with the duplicated assembly */
	static void DuplicateAssociatedAsset(UCineAssembly* DuplicatedAssembly, FAssemblyAssociatedAssetDesc& AssetDesc, EDuplicateExternalAssetPreference ExternalAssetPreference = EDuplicateExternalAssetPreference::DuplicateIntoAssemblyFolder);

	/** Duplicate the associated assets of a duplicated assembly, updating each definition's CreatedAsset to point to the duplicate */
	static void DuplicateAssociatedAssets(UCineAssembly* DuplicatedAssembly, EDuplicateExternalAssetPreference ExternalAssetPreference = EDuplicateExternalAssetPreference::DuplicateIntoAssemblyFolder);

	/** Populate metadata fields of the input assembly (and managed SubAssemblies) that are linked to associated assets */
	static void PopulateLinkedMetadataRecursive(UCineAssembly* Assembly);

	/** Takes a CineAssembly and a creation path and makes a formatted CineAssembly package name. */
	static FString MakeAssemblyPackageName(UCineAssembly* ConfiguredAssembly, const FString& CreateAssetPath);

	/** Recursively evaluates the default assembly name and path until a unique combination is found */
	static bool MakeUniqueNameAndPath(UCineAssembly* ConfiguredAssembly, const FString& CreateAssetPath, FString& UniquePackageName, FString& UniqueAssetName);

protected:
	UCineAssemblyFactory();

	// Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool CanCreateNew() const override;
	virtual bool ConfigureProperties() override;
	// End UFactory Interface
};
