// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblyMetadataCustomization.h"

#include "Algo/Contains.h"
#include "Algo/Find.h"
#include "Algo/IndexOf.h"
#include "CineAssembly.h"
#include "CineAssemblyNamingTokens.h"
#include "CineAssemblyToolsStyle.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "MovieScene.h"
#include "MovieSceneSubAssemblySection.h"
#include "NamingTokensEngineSubsystem.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "SNamingTokensEditableTextBox.h"
#include "UI/SAssemblyMetadataLink.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CineAssemblyMetadataCustomization"

TSharedRef<IPropertyTypeCustomization> FCineAssemblyMetadataCustomization::MakeInstance()
{
	return MakeShareable(new FCineAssemblyMetadataCustomization);
}

void FCineAssemblyMetadataCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			InPropertyHandle->CreatePropertyValueWidget()
		];
}

TSharedRef<SWidget> FCineAssemblyMetadataCustomization::MakeStringDefaultValueWidget(FAssemblyMetadataDesc& MetadataDesc, TSharedRef<IPropertyHandle> PropertyHandle)
{
	TSharedPtr<IPropertyHandle> MetadataEvaluateTokensHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAssemblyMetadataDesc, bEvaluateTokens));

	TSharedPtr<SWidgetSwitcher> StringValueWidget = SNew(SWidgetSwitcher)
		.WidgetIndex_Lambda([MetadataEvaluateTokensHandle]()
			{
				bool bValue = false;
				MetadataEvaluateTokensHandle->GetValue(bValue);
				return static_cast<int32>(bValue);
			});

	const int32 EvalFalseIndex = 0;
	const int32 EvalTrueIndex = 1;

	FNamingTokenFilterArgs FilterArgs;
	FilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

	StringValueWidget->AddSlot(EvalFalseIndex)
		[
			SNew(SMultiLineEditableTextBox)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.AutoWrapText(true)
				.Text_Lambda([&MetadataDesc]() -> FText
					{
						if (const FString* Value = MetadataDesc.DefaultValue.TryGet<FString>())
						{
							return FText::FromString(*Value);
						}
						return FText::GetEmpty();
					})
				.OnTextCommitted_Lambda([&MetadataDesc](const FText& InText, ETextCommit::Type InCommitType)
					{
						MetadataDesc.DefaultValue.Set<FString>(InText.ToString());
					})
		];

	StringValueWidget->AddSlot(EvalTrueIndex)
		[
			SNew(SNamingTokensEditableTextBox)
				.AllowMultiLine(true)
				.Style(FCineAssemblyToolsStyle::Get(), "DetailsEditableTextBox")
				.ArgumentStyle(FCineAssemblyToolsStyle::Get(), "DetailsEditableTextBoxArguments")
				.ShouldEvaluateTokens(false)
				.FilterArgs(FilterArgs)
				.Text_Lambda([&MetadataDesc]() -> FText
					{
						if (const FString* Value = MetadataDesc.DefaultValue.TryGet<FString>())
						{
							return FText::FromString(*Value);
						}
						return FText::GetEmpty();
					})
				.OnTextCommitted_Lambda([&MetadataDesc](const FText& InText, ETextCommit::Type InCommitType)
					{
						MetadataDesc.DefaultValue.Set<FString>(InText.ToString());
					})
		];

	return SNew(SBox)
		.MaxDesiredHeight(120.0f)
		.MinDesiredWidth(400)
		[
			StringValueWidget.ToSharedRef()
		];
}

void FCineAssemblyMetadataCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Get the schema object that owns the metadata struct being customized
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects = CustomizationUtils.GetPropertyUtilities()->GetSelectedObjects();
	if (CustomizedObjects.Num() != 1)
	{
		return;
	}

	Schema = Cast<UCineAssemblySchema>(CustomizedObjects[0].Get());

	LeadingZeroTypeInterface = MakeShared<FLeadingZeroNumericTypeInterface>();

	if (!PropertyHandle->IsExpanded())
	{
		PropertyHandle->SetExpanded(true);
	}

	ArrayIndex = PropertyHandle->GetArrayIndex();
	if (Schema->AssemblyMetadata.IsValidIndex(ArrayIndex))
	{
		FAssemblyMetadataDesc& MetadataDesc = Schema->AssemblyMetadata[ArrayIndex];

		// Add all of the existing reflected properties of the metadata struct
		uint32 NumChildren;
		PropertyHandle->GetNumChildren(NumChildren);

		for (uint32 Index = 0; Index < NumChildren; ++Index)
		{
			TSharedRef<IPropertyHandle> ChildPropertyHandle = PropertyHandle->GetChildHandle(Index).ToSharedRef();

			if (ChildPropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FAssemblyMetadataDesc, Key))
			{
				CustomizeKeyProperty(ChildPropertyHandle, ChildBuilder, MetadataDesc);
			}
			else
			{
				ChildBuilder.AddProperty(ChildPropertyHandle).ShowPropertyButtons(false);
			}
		}

		TSharedPtr<IPropertyHandle> MetadataAssetClassHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAssemblyMetadataDesc, AssetClass));
		MetadataAssetClassHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &FCineAssemblyMetadataCustomization::ValidateMetadataLink));

		TSharedPtr<IPropertyHandle> MetadataSchemaTypeHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAssemblyMetadataDesc, SchemaType));
		MetadataSchemaTypeHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &FCineAssemblyMetadataCustomization::ValidateMetadataLink));

		// Create a widget switcher that can display the appropriate widget based on the metadata type
		TSharedPtr<IPropertyHandle> MetadataTypeHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAssemblyMetadataDesc, Type));

		MetadataTypeHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this, &MetadataDesc]()
			{
				ValidateMetadataLink();

				if ((MetadataDesc.Type == ECineAssemblyMetadataType::String) ||
					(MetadataDesc.Type == ECineAssemblyMetadataType::AssetPath) ||
					(MetadataDesc.Type == ECineAssemblyMetadataType::CineAssembly))
				{
					MetadataDesc.DefaultValue.Set<FString>(TEXT(""));
				}
				else if (MetadataDesc.Type == ECineAssemblyMetadataType::Bool)
				{
					MetadataDesc.DefaultValue.Set<bool>(false);
				}
				else if (MetadataDesc.Type == ECineAssemblyMetadataType::Integer)
				{
					MetadataDesc.DefaultValue.Set<int32>(0);
				}
				else if (MetadataDesc.Type == ECineAssemblyMetadataType::Float)
				{
					MetadataDesc.DefaultValue.Set<float>(0);
				}
				else
				{
					checkNoEntry();
				}
			}));

		// Add a "Default Value" property, based on the metadata type
		TSharedPtr<SWidgetSwitcher> DefaultValueWidget = SNew(SWidgetSwitcher)
			.WidgetIndex_Lambda([MetadataTypeHandle]()
				{
					uint8 Value = 0;
					MetadataTypeHandle->GetValue(Value);
					return Value;
				});

		// Resolves to the current Key of this metadata entry on every read, so the widget tracks user renames instead of caching a stale string.
		const TAttribute<FString> MetadataKeyAttribute = TAttribute<FString>::CreateLambda([WeakSchema = TWeakObjectPtr<UCineAssemblySchema>(Schema), Index = ArrayIndex]() -> FString
			{
				UCineAssemblySchema* SchemaPtr = WeakSchema.Get();
				return (SchemaPtr && SchemaPtr->AssemblyMetadata.IsValidIndex(Index)) ? SchemaPtr->AssemblyMetadata[Index].Key : FString();
			});

		DefaultValueWidget->AddSlot((uint8)ECineAssemblyMetadataType::String)
			[
				SNew(SAssemblyMetadataLink, Schema->TemplateSequence, MetadataKeyAttribute)
					[
						MakeStringDefaultValueWidget(MetadataDesc, PropertyHandle)
					]
			];

		DefaultValueWidget->AddSlot((uint8)ECineAssemblyMetadataType::Bool)
			[
				SNew(SCheckBox)
					.IsChecked_Lambda([&MetadataDesc]() -> ECheckBoxState
						{
							if (const bool* Value = MetadataDesc.DefaultValue.TryGet<bool>())
							{
								return *Value ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
							}
							return ECheckBoxState::Unchecked;
						})
					.OnCheckStateChanged_Lambda([&MetadataDesc](ECheckBoxState CheckBoxState)
						{
							const bool Value = (CheckBoxState == ECheckBoxState::Checked);
							MetadataDesc.DefaultValue.Set<bool>(Value);
						})
			];

		DefaultValueWidget->AddSlot((uint8)ECineAssemblyMetadataType::Integer)
			[
				SNew(SNumericEntryBox<int32>)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.TypeInterface(LeadingZeroTypeInterface)
					.Value_Lambda([&MetadataDesc]() -> int32
						{
							if (const int32* Value = MetadataDesc.DefaultValue.TryGet<int32>())
							{
								return *Value;
							}
							return 0;
						})
					.OnValueChanged_Lambda([&MetadataDesc](int32 InValue)
						{
							MetadataDesc.DefaultValue.Set<int32>(InValue);
						})
			];

		DefaultValueWidget->AddSlot((uint8)ECineAssemblyMetadataType::Float)
			[
				SNew(SNumericEntryBox<float>)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.Value_Lambda([&MetadataDesc]() -> float
						{
							if (const float* Value = MetadataDesc.DefaultValue.TryGet<float>())
							{
								return *Value;
							}
							return 0;
						})
					.OnValueChanged_Lambda([&MetadataDesc](float InValue)
						{
							MetadataDesc.DefaultValue.Set<float>(InValue);
						})
			];

		DefaultValueWidget->AddSlot((uint8)ECineAssemblyMetadataType::AssetPath)
			[
				SNew(SAssemblyMetadataLink, Schema->TemplateSequence, MetadataKeyAttribute)
					[
						SNew(SObjectPropertyEntryBox)
							.ThumbnailPool(CustomizationUtils.GetThumbnailPool())
							.AllowCreate(true)
							.OnShouldFilterAsset_Lambda([&MetadataDesc](const FAssetData& InAssetData) -> bool
								{
									if (MetadataDesc.AssetClass.IsNull())
									{
										return false;
									}
									return (InAssetData.AssetClassPath != MetadataDesc.AssetClass.GetAssetPath());
								})
							.ObjectPath_Lambda([&MetadataDesc]() -> FString
								{
									if (const FString* Value = MetadataDesc.DefaultValue.TryGet<FString>())
									{
										return *Value;
									}
									return FString();
								})
							.OnObjectChanged_Lambda([&MetadataDesc](const FAssetData& InAssetData)
								{
									MetadataDesc.DefaultValue.Set<FString>(InAssetData.GetObjectPathString());
								})
					]
			];

		DefaultValueWidget->AddSlot((uint8)ECineAssemblyMetadataType::CineAssembly)
			[
				SNew(SAssemblyMetadataLink, Schema->TemplateSequence, MetadataKeyAttribute)
					[
						SNew(SObjectPropertyEntryBox)
							.AllowedClass(UCineAssembly::StaticClass())
							.ThumbnailPool(CustomizationUtils.GetThumbnailPool())
							.AllowCreate(true)
							.OnShouldFilterAsset_Lambda([&MetadataDesc](const FAssetData& InAssetData) -> bool
								{
									if (!MetadataDesc.SchemaType.IsValid())
									{
										return false;
									}

									// Filter out all Cine Assembly assets that do not match the selected Schema Type for this metadata struct
									const FAssetDataTagMapSharedView::FFindTagResult AssemblyType = InAssetData.TagsAndValues.FindTag(UCineAssembly::AssetRegistryTag_AssemblyType);
									if (AssemblyType.IsSet())
									{
										return !AssemblyType.GetValue().Equals(MetadataDesc.SchemaType.GetAssetName());
									}
									return true;
								})
							.ObjectPath_Lambda([&MetadataDesc]() -> FString
								{
									if (const FString* Value = MetadataDesc.DefaultValue.TryGet<FString>())
									{
										return *Value;
									}
									return FString();
								})
							.OnObjectChanged_Lambda([&MetadataDesc](const FAssetData& InAssetData)
								{
									MetadataDesc.DefaultValue.Set<FString>(InAssetData.GetObjectPathString());
								})
					]
			];

		ChildBuilder.AddCustomRow(FText::GetEmpty())
			.NameContent()
			[
				SNew(STextBlock)
					.Text(NSLOCTEXT("CineAssemblyMetadataCustomization", "DefaultValueText", "Default Value"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
			.ValueContent()
			.HAlign(HAlign_Fill)
			[
				DefaultValueWidget.ToSharedRef()
			];
	}
}

void FCineAssemblyMetadataCustomization::ValidateMetadataLink()
{
	if (!Schema || !Schema->TemplateSequence || !Schema->AssemblyMetadata.IsValidIndex(ArrayIndex))
	{
		return;
	}

	const FAssemblyMetadataDesc& MetadataDesc = Schema->AssemblyMetadata[ArrayIndex];

	// Get the ID of the currently linked Asset. Early-out if no asset is currently linked.
	const FGuid* LinkedAssetID = Schema->TemplateSequence->MetadataLinks.Find(MetadataDesc.Key);
	if (!LinkedAssetID)
	{
		return;
	}

	// If the currently linked asset is no longer compatible with this metadata field, unlink it.
	if (!IsLinkedAssetCompatible(*LinkedAssetID, MetadataDesc))
	{
		Schema->TemplateSequence->Modify();
		Schema->TemplateSequence->MetadataLinks.Remove(MetadataDesc.Key);
	}
}

bool FCineAssemblyMetadataCustomization::IsLinkedAssetCompatible(const FGuid& InLinkedAssetID, const FAssemblyMetadataDesc& InMetadataDesc) const
{
	if (!Schema || !Schema->TemplateSequence)
	{
		return false;
	}

	switch (InMetadataDesc.Type)
	{
	case ECineAssemblyMetadataType::String:
		return true;

	case ECineAssemblyMetadataType::AssetPath:
		{
			// If there is no class filter, then the link is valid
			const UClass* AllowedClass = InMetadataDesc.AssetClass.ResolveClass();
			if (!AllowedClass)
			{
				return true;
			}

			// If the linked asset is one of the TemplateSequence's associated assets, check that its class passes the current asset class filter
			if (const FAssemblyAssociatedAssetDesc* AssetDesc = Algo::FindBy(Schema->TemplateSequence->AssociatedAssets, InLinkedAssetID, &FAssemblyAssociatedAssetDesc::AssetID))
			{
				UClass* AssetClass = AssetDesc->AssetClass.Get();
				return AssetClass && AssetClass->IsChildOf(AllowedClass);
			}

			// If the linked asset is not an associated asset, we can assume it must be a SubAssembly, so check that it passes the class filter
			return UCineAssembly::StaticClass()->IsChildOf(AllowedClass);
		}

	case ECineAssemblyMetadataType::CineAssembly:
		{
			// CineAssembly metadata can only link to SubAssemblies, not associated assets
			if (Algo::FindBy(Schema->TemplateSequence->AssociatedAssets, InLinkedAssetID, &FAssemblyAssociatedAssetDesc::AssetID))
			{
				return false;
			}

			// If there is no Schema filter, then any SubAssembly is valid
			if (InMetadataDesc.SchemaType.IsNull())
			{
				return true;
			}

			// Find a SubAssemblySection that matches the linked asset ID
			UMovieSceneSubAssemblySection* LinkedSubAssemblySection = nullptr;
			if (UMovieScene* MovieScene = Schema->TemplateSequence->GetMovieScene())
			{
				for (UMovieSceneSection* Section : MovieScene->GetAllSections())
				{
					if (UMovieSceneSubAssemblySection* SubAssemblySection = Cast<UMovieSceneSubAssemblySection>(Section))
					{
						if (SubAssemblySection->GetSectionID() == InLinkedAssetID)
						{
							LinkedSubAssemblySection = SubAssemblySection;
							break;
						}
					}
				}
			}
			
			// Validate that the Schema of this SubAssembly passes the Schema filter
			if (LinkedSubAssemblySection)
			{
				const UCineAssemblySchema* TemplateSchema = LinkedSubAssemblySection->GetTemplateSchema();
				return TemplateSchema && InMetadataDesc.SchemaType == FSoftObjectPath(TemplateSchema);
			}

			return false;
		}

	case ECineAssemblyMetadataType::Bool:
	case ECineAssemblyMetadataType::Integer:
	case ECineAssemblyMetadataType::Float:
		return false;

	default:
		checkNoEntry();
	}

	return false;
}

void FCineAssemblyMetadataCustomization::CustomizeKeyProperty(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, FAssemblyMetadataDesc& MetadataDesc)
{
	FString ExistingKeyName;
	PropertyHandle->GetValue(ExistingKeyName);

	// If the key name is not yet set, assign it a unique default key name
	if (ExistingKeyName.IsEmpty())
	{
		PropertyHandle->SetValue(MakeUniqueKeyName());
	}

	IDetailPropertyRow& KeyRow = ChildBuilder.AddProperty(PropertyHandle);
	KeyRow.CustomWidget()
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SEditableTextBox)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text_Lambda([this, PropertyHandle]() -> FText
					{
						FString KeyName;
						PropertyHandle->GetValue(KeyName);
						return FText::FromString(KeyName);
					})
				.OnTextCommitted_Lambda([this, PropertyHandle](const FText& InText, ETextCommit::Type InCommitType)
					{
						RenameMetadataKey(PropertyHandle, InText.ToString());
					})
				.OnVerifyTextChanged_Raw(this, &FCineAssemblyMetadataCustomization::ValidateKeyName)
		];
}

void FCineAssemblyMetadataCustomization::RenameMetadataKey(TSharedRef<IPropertyHandle> KeyPropertyHandle, const FString& NewKeyName)
{
	FString OldKeyName;
	KeyPropertyHandle->GetValue(OldKeyName);

	if (OldKeyName == NewKeyName)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("RenameMetadataKey", "Rename Metadata Key"));

	// Capture any existing link for the old key before SetValue, in case downstream validation clears it
	FGuid LinkedAssetID;
	if (Schema && Schema->TemplateSequence)
	{
		if (const FGuid* Found = Schema->TemplateSequence->MetadataLinks.Find(OldKeyName))
		{
			LinkedAssetID = *Found;
		}
	}

	KeyPropertyHandle->SetValue(NewKeyName);

	// Migrate the TemplateSequence's MetadataLinks entry from the old key to the new one, preserving the linked asset
	if (LinkedAssetID.IsValid() && Schema && Schema->TemplateSequence)
	{
		Schema->TemplateSequence->Modify();
		Schema->TemplateSequence->MetadataLinks.Remove(OldKeyName);
		Schema->TemplateSequence->MetadataLinks.Add(NewKeyName, LinkedAssetID);
	}
}

bool FCineAssemblyMetadataCustomization::ValidateKeyName(const FText& InText, FText& OutErrorMessage) const
{
	// An empty name is invalid
	if (InText.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("EmptyKeyNameError", "Please provide a key name");
		return false;
	}

	// Check for duplicate keys in this schema
	const int32 MetadataIndex = Algo::IndexOfBy(Schema->AssemblyMetadata, InText.ToString(), &FAssemblyMetadataDesc::Key);
	if ((MetadataIndex != INDEX_NONE) && (MetadataIndex != ArrayIndex))
	{
		OutErrorMessage = LOCTEXT("DuplicateKeyNameError", "A metadata key with this name already exists in this schema");
		return false;
	}

	// Check that the proposed key name does not match one of the default CAT tokens
	UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();
	UCineAssemblyNamingTokens* CineAssemblyNamingTokens = Cast<UCineAssemblyNamingTokens>(NamingTokensSubsystem->GetNamingTokens(UCineAssemblyNamingTokens::TokenNamespace));

	TArray<FNamingTokenData> Tokens = CineAssemblyNamingTokens->GetDefaultTokens();
	if (Algo::ContainsBy(Tokens, InText.ToString(), &FNamingTokenData::TokenKey))
	{
		OutErrorMessage = LOCTEXT("ExistingTokenKeyError", "A CAT token key with this name already exists");
		return false;
	}

	return true;
}

FString FCineAssemblyMetadataCustomization::MakeUniqueKeyName()
{
	const FString BaseName = TEXT("NewKey");
	FString WorkingName = BaseName;

	int32 IntSuffix = 1;
	while (Algo::ContainsBy(Schema->AssemblyMetadata, WorkingName, &FAssemblyMetadataDesc::Key))
	{
		WorkingName = FString::Printf(TEXT("%s%d"), *BaseName, IntSuffix);
		IntSuffix++;
	}

	return WorkingName;
}

#undef LOCTEXT_NAMESPACE
