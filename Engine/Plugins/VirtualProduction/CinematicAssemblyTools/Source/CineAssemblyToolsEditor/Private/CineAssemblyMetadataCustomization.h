// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

#include "CineAssemblySchema.h"
#include "LeadingZeroNumericTypeInterface.h"

class SWidget;

class FCineAssemblyMetadataCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// Begin IDetailCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IDetailCustomization interface

private:
	/**
	* Given a FAssemblyMetadataDesc, makes a SNamingTokensEditableTextBox for a string default value that has bEvaluateTokens set to true,
	* and a SMultiLineEditableTextBox for a string default value that has bEvaluateTokens set to false.
	*/
	TSharedRef<SWidget> MakeStringDefaultValueWidget(FAssemblyMetadataDesc& MetadataDesc, TSharedRef<IPropertyHandle> PropertyHandle);

	/** Customizes the metadata desc Key property */
	void CustomizeKeyProperty(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, FAssemblyMetadataDesc& MetadataDesc);

	/** Renames the metadata key via its PropertyHandle and migrates the TemplateSequence's MetadataLinks entry (if any) from the old key to the new one */
	void RenameMetadataKey(TSharedRef<IPropertyHandle> KeyPropertyHandle, const FString& NewKeyName);

	/** Checks whether the proposed key name is valid (not empty, not a duplicate) */
	bool ValidateKeyName(const FText& InText, FText& OutErrorMessage) const;

	/** Returns a unique key name within the context of the schema */
	FString MakeUniqueKeyName();

	/** Validates the active metadata link against the current metadata field properties and removes the link if no longer valid */
	void ValidateMetadataLink();

	/** Whether the currently linked asset is compatible with the input metadata field properties */
	bool IsLinkedAssetCompatible(const FGuid& InLinkedAssetID, const FAssemblyMetadataDesc& InMetadataDesc) const;

private:
	/** The schema that owns the metadata desc struct being customized */
	UCineAssemblySchema* Schema;

	/** The array index of the metadata struct being customized */
	int32 ArrayIndex;

	/** Type interface for a numeric entry box that supports leading zeroes */
	TSharedPtr<FLeadingZeroNumericTypeInterface> LeadingZeroTypeInterface;
};
