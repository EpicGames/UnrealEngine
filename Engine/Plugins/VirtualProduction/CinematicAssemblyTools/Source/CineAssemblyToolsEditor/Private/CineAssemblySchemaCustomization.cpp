// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblySchemaCustomization.h"

#include "Algo/Contains.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetViewUtils.h"
#include "CineAssembly.h"
#include "CineAssemblyNamingTokens.h"
#include "CineAssemblySchema.h"
#include "CineAssemblyToolsStyle.h"
#include "DetailLayoutBuilder.h"
#include "Engine/World.h"
#include "PropertyCustomizationHelpers.h"
#include "SNamingTokensEditableTextBox.h"
#include "UI/SAssemblyMetadataLink.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "CineAssemblySchemaCustomization"

TSharedRef<IDetailCustomization> FCineAssemblySchemaCustomization::MakeInstance()
{
	return MakeShared<FCineAssemblySchemaCustomization>();
}

void FCineAssemblySchemaCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	DetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);

	// Ensure that we are only customizing one object
	if (CustomizedObjects.Num() != 1)
	{
		return;
	}

	CustomizedSchema = Cast<UCineAssemblySchema>(CustomizedObjects[0]);

	IDetailCategoryBuilder& DefaultCategory = DetailBuilder.EditCategory("Default");
	TSharedRef<IPropertyHandle> SchemaNamePropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCineAssemblySchema, SchemaName));
	TSharedRef<IPropertyHandle> DescriptionPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCineAssemblySchema, Description));
	TSharedRef<IPropertyHandle> DefaultAssemblyNamePropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCineAssemblySchema, DefaultAssemblyName));

	// Add the properties back into the category in the correct order
	IDetailPropertyRow& SchemaNamePropertyRow = DefaultCategory.AddProperty(SchemaNamePropertyHandle);
	DefaultCategory.AddProperty(DescriptionPropertyHandle);
	IDetailPropertyRow& DefaultAssemblyNamePropertyRow = DefaultCategory.AddProperty(DefaultAssemblyNamePropertyHandle);

	// Customize the widget for the SchemaName property to add additional validation on the user input text
	SchemaNamePropertyHandle->GetValue(OriginalSchemaName);

	// If the key name is not yet set, assign it a unique default key name
	if (OriginalSchemaName.IsEmpty())
	{
		OriginalSchemaName = MakeUniqueSchemaName();
		SchemaNamePropertyHandle->SetValue(OriginalSchemaName);
	}

	SchemaNamePropertyRow.CustomWidget()
		.NameContent()
		[
			SchemaNamePropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SEditableTextBox)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda([this]() -> FText
					{
						return FText::FromString(CustomizedSchema->SchemaName);
					})
				.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type InCommitType)
					{
						if (InCommitType != ETextCommit::Default)
						{
							if (!CustomizedSchema->SchemaName.Equals(InText.ToString()))
							{
								CustomizedSchema->RenameAsset(InText.ToString());
							}
						}
					})
				.OnVerifyTextChanged_Raw(this, &FCineAssemblySchemaCustomization::ValidateSchemaName)
		];

	SchemaNamePropertyRow.OverrideResetToDefault(
		FResetToDefaultOverride::Create(
			FIsResetToDefaultVisible::CreateRaw(this, &FCineAssemblySchemaCustomization::IsSchemaNameResetToDefaultVisible),
			FResetToDefaultHandler::CreateRaw(this, &FCineAssemblySchemaCustomization::OnSchemaNameResetToDefault)));

	// Customize the widget for the DefaultAssemblyName property to add additional validation on the user input text
	FNamingTokenFilterArgs FilterArgs;
	FilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

	DefaultAssemblyNamePropertyRow.CustomWidget()
		.NameContent()
		[
			DefaultAssemblyNamePropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SNamingTokensEditableTextBox)
				.Style(FCineAssemblyToolsStyle::Get(), "DetailsEditableTextBox")
				.ArgumentStyle(FCineAssemblyToolsStyle::Get(), "DetailsEditableTextBoxArguments")
				.DisplayTokenIcon(false)
				.ShouldEvaluateTokens(false)
				.FilterArgs(FilterArgs)
				.Text_Lambda([this]() -> FText
					{
						return FText::FromString(CustomizedSchema->DefaultAssemblyName);
					})
				.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type InCommitType)
					{
						CustomizedSchema->DefaultAssemblyName = InText.ToString();
					})
				.OnValidateTokenizedText_Raw(this, &FCineAssemblySchemaCustomization::ValidateDefaultAssemblyName)
		];

	// Customize the DefaultLevel property to wrap it in a metadata link widget
	IDetailCategoryBuilder& AssemblyDefaultsCategory = DetailBuilder.EditCategory("Assembly Defaults");
	TSharedRef<IPropertyHandle> DefaultLevelHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCineAssemblySchema, DefaultLevel));
	IDetailPropertyRow& DefaultLevelRow = AssemblyDefaultsCategory.AddProperty(DefaultLevelHandle);

	const FString& LevelKey = UCineAssemblySchema::DefaultLevelMetadataKey;

	DefaultLevelRow.CustomWidget()
		.NameContent()
		[
			DefaultLevelHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SAssemblyMetadataLink, CustomizedSchema->TemplateSequence, LevelKey)
			[
				SNew(SObjectPropertyEntryBox)
					.AllowedClass(UWorld::StaticClass())
					.ThumbnailPool(DetailBuilder.GetThumbnailPool())
					.AllowCreate(false)
					.ObjectPath_Lambda([this]()
						{
							return CustomizedSchema->DefaultLevel.ToString();
						})
					.OnObjectChanged_Lambda([this](const FAssetData& InAssetData)
						{
							CustomizedSchema->Modify();
							CustomizedSchema->SetDefaultLevel(TSoftObjectPtr<UWorld>(InAssetData.GetSoftObjectPath()));
						})
			]
		];
}

FString FCineAssemblySchemaCustomization::MakeUniqueSchemaName() const
{
	const FString BaseName = TEXT("NewCineAssemblySchema");
	FString WorkingName = BaseName;

	int32 IntSuffix = 1;
	while (DoesSchemaExistWithName(WorkingName))
	{
		WorkingName = FString::Printf(TEXT("%s%d"), *BaseName, IntSuffix);
		IntSuffix++;
	}

	return WorkingName;
}

bool FCineAssemblySchemaCustomization::DoesSchemaExistWithName(const FString& SchemaName) const
{
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
	TArray<FAssetData> SchemaAssets;
	AssetRegistryModule.Get().GetAssetsByClass(UCineAssemblySchema::StaticClass()->GetClassPathName(), SchemaAssets);

	return Algo::ContainsBy(SchemaAssets, *SchemaName, &FAssetData::AssetName);
}

bool FCineAssemblySchemaCustomization::ValidateSchemaName(const FText& InText, FText& OutErrorMessage) const
{
	if (InText.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("EmptyNameErrorMessage", "Please provide a name for the schema");
		return false;
	}

	// It is valid if the input text matches the schema's current name
	if (CustomizedSchema->SchemaName == InText.ToString())
	{
		return true;
	}

	if (DoesSchemaExistWithName(InText.ToString()))
	{
		OutErrorMessage = LOCTEXT("DuplicateNameErrorMessage", "A schema with that name already exists");
		return false;
	}

	/* Check the resulting package name would not fail validation if we have enough information. If not transient,
	*  check the new name with the existing path name.
	*  This is typically transient for schemas being created and non-transient for ones being edited.
	*/
	if (CustomizedSchema->GetPackage() != GetTransientPackage())
	{
		const FString PackageParentName = FPackageName::GetLongPackagePath(CustomizedSchema->GetPackage()->GetPathName());
		if (!AssetViewUtils::IsValidPackageForCooking(PackageParentName / InText.ToString(), OutErrorMessage))
		{
			return false;
		}
	}

	return FName::IsValidXName(InText.ToString(), INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, &OutErrorMessage);
}

bool FCineAssemblySchemaCustomization::ValidateDefaultAssemblyName(const FText& InText, FText& OutErrorMessage) const
{
	// Ensure that the name does not contain any characters that would be invalid for an asset name
	// This matches the validation that would happen if the user was renaming an asset in the content browser
	FString InvalidCharacters = INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS;

	// These characters are actually valid, because we want to support naming tokens
	InvalidCharacters = InvalidCharacters.Replace(TEXT("{}"), TEXT(""));
	InvalidCharacters = InvalidCharacters.Replace(TEXT(":"), TEXT(""));

	const FString PotentialName = InText.ToString();

	if (!FName::IsValidXName(PotentialName, InvalidCharacters, &OutErrorMessage))
	{
		return false;
	}

	if (PotentialName.Contains(TEXT("{assembly}")) || PotentialName.Contains(TEXT("{cat:assembly}")))
	{
		OutErrorMessage = LOCTEXT("RecursiveAssemblyTokenError", "The {assembly} token cannot be used in an assembly name because it is self-referential.");
		return false;
	}

	return true;
}

bool FCineAssemblySchemaCustomization::IsSchemaNameResetToDefaultVisible(TSharedPtr<IPropertyHandle> SchemaNamePropertyHandle) const
{
	// The "default" name of the schema is defined as follows:
	// For new schemas, this will be the default (unique) name that was assigned to the schema
	// For existing schemas, this will be the name of the schema when its asset editor window was opened
	FString CurrentSchemaName;
	SchemaNamePropertyHandle->GetValue(CurrentSchemaName);

	return !CurrentSchemaName.Equals(OriginalSchemaName);
};

void FCineAssemblySchemaCustomization::OnSchemaNameResetToDefault(TSharedPtr<IPropertyHandle> SchemaNamePropertyHandle)
{
	CustomizedSchema->RenameAsset(OriginalSchemaName);
}

#undef LOCTEXT_NAMESPACE
