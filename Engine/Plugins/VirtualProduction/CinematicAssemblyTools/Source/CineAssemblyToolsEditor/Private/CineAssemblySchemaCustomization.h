// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class UCineAssemblySchema;
class IPropertyHandle;

/**
 * Detail customization for UCineAssemblySchema
 */
class FCineAssemblySchemaCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface */

private:
	/** Generate a unique schema name */
	FString MakeUniqueSchemaName() const;

	/** Returns true if a schema already exists with the input name */
	bool DoesSchemaExistWithName(const FString& SchemaName) const;

	/** Validates the user input text for the associated properties */
	bool ValidateSchemaName(const FText& InText, FText& OutErrorMessage) const;
	bool ValidateDefaultAssemblyName(const FText& InText, FText& OutErrorMessage) const;

	/** Implements "reset to default" behavior for the schema name property */
	bool IsSchemaNameResetToDefaultVisible(TSharedPtr<IPropertyHandle> SchemaNamePropertyHandle) const;
	void OnSchemaNameResetToDefault(TSharedPtr<IPropertyHandle> SchemaNamePropertyHandle);

private:
	/** The assembly schema being customized */
	UCineAssemblySchema* CustomizedSchema;

	/** Original name of the schema being customized */
	FString OriginalSchemaName;
};
