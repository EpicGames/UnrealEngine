// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblySchemaFactory.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetViewUtils.h"
#include "AssetToolsModule.h"
#include "CineAssemblyEditorSubsystem.h"
#include "CineAssemblySchema.h"
#include "CineAssemblyToolsAnalytics.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "IContentBrowserSingleton.h"
#include "Interfaces/IMainFrameModule.h"
#include "Modules/ModuleManager.h"
#include "UI/CineAssembly/SCineAssemblySchemaWindow.h"

UCineAssemblySchemaFactory::UCineAssemblySchemaFactory()
{
	SupportedClass = UCineAssemblySchema::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

bool UCineAssemblySchemaFactory::CanCreateNew() const
{
	return true;
}

bool UCineAssemblySchemaFactory::ConfigureProperties()
{
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	const FString CurrentPath = ContentBrowser.GetCurrentPath().GetInternalPathString();

	TSharedRef<SCineAssemblySchemaWindow> SchemaWidget = SNew(SCineAssemblySchemaWindow, CurrentPath);

	const FVector2D DefaultWindowSize = FVector2D(1400, 750);

	TSharedRef<SWindow> NewSchemaWindow = SNew(SWindow)
		.Title(NSLOCTEXT("CineAssemblySchemaFactory", "WindowTitleCreateNew", "Create Assembly Schema"))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(DefaultWindowSize)
		.SupportsMaximize(false)
		.SupportsMinimize(false);

	NewSchemaWindow->SetContent(SchemaWidget);
	NewSchemaWindow->SetOnWindowClosed(FOnWindowClosed::CreateSP(SchemaWidget, &SCineAssemblySchemaWindow::OnWindowClosed));

	const IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	const TSharedPtr<SWindow> ParentWindow = MainFrameModule.GetParentWindow();

	if (ParentWindow.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(NewSchemaWindow, ParentWindow.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(NewSchemaWindow);
	}

	return false;
}

UObject* UCineAssemblySchemaFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UCineAssemblySchema* NewSchema = NewObject<UCineAssemblySchema>(InParent, Name, Flags);
	return NewSchema;
}

void UCineAssemblySchemaFactory::CreateConfiguredSchema(UCineAssemblySchema* ConfiguredSchema, const FString& CreateAssetPath)
{
	// The intended use of this function is to take a pre-configured, transient assembly, create a valid package for it, and initialize it.
	if (ConfiguredSchema->GetPackage() != GetTransientPackage())
	{
		return;
	}

	// If the assembly name is empty, assign it a valid default name
	if (ConfiguredSchema->SchemaName.IsEmpty())
	{
		ConfiguredSchema->SchemaName = TEXT("NewCineAssemblySchema");
	}

	const FString DesiredPackageName = CreateAssetPath / ConfiguredSchema->SchemaName;

	// Ensure the package name length does not exceed the maximum cook path length as this may cause issues later on.
	FText OutErrorMessage;
	if (!AssetViewUtils::IsValidPackageForCooking(DesiredPackageName, OutErrorMessage))
	{
		return;
	}

	const FString DesiredSuffix = TEXT("");

	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();

	FString UniquePackageName;
	FString UniqueAssetName;
	AssetTools.CreateUniqueAssetName(DesiredPackageName, DesiredSuffix, UniquePackageName, UniqueAssetName);

	// Double check again now that we may have created a new unique name
	if (!AssetViewUtils::IsValidPackageForCooking(UniquePackageName, OutErrorMessage))
	{
		return;
	}

	// The input schema object was created in the transient package while its properties were configured.
	// Now, we can create a real package for it, rename it, and update its object flags.
	UPackage* Package = CreatePackage(*UniquePackageName);
	ConfiguredSchema->Rename(*UniqueAssetName, Package);

	const EObjectFlags Flags = RF_Public | RF_Standalone | RF_Transactional;
	ConfiguredSchema->SetFlags(Flags);
	ConfiguredSchema->ClearFlags(RF_Transient);

	UE::CineAssemblyToolsAnalytics::RecordEvent_CreateAssemblySchema();

	// Notify the asset registry about the new schema
	FAssetRegistryModule::AssetCreated(ConfiguredSchema);

	// Mark the package dirty
	Package->MarkPackageDirty();

	// Refresh the content browser to make any new assets and folders are immediately visible
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	ContentBrowser.SetSelectedPaths({ CreateAssetPath }, true);

	if (UCineAssemblyEditorSubsystem* EditorSubsystem = GEditor ? GEditor->GetEditorSubsystem<UCineAssemblyEditorSubsystem>() : nullptr)
	{
		EditorSubsystem->OnSchemaCreated.Broadcast(ConfiguredSchema);
	}
}
