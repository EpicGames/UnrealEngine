// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "CineAssemblySchemaFactory.generated.h"

class UCineAssemblySchema;

/**
 * Factory class used to create new UCineAssemblySchema objects
 * Before creating a new assembly schema, the factory will spawn a window to configure the properties of the asset that is being created
 */
UCLASS(hidecategories = Object)
class UCineAssemblySchemaFactory : public UFactory
{
	GENERATED_BODY()

public:
	/** Takes a pre-configured, transient schema, creates a valid package for it, and initializes it */
	static void CreateConfiguredSchema(UCineAssemblySchema* ConfiguredSchema, const FString& CreateAssetPath);

protected:
	UCineAssemblySchemaFactory();

	// Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool CanCreateNew() const override;
	virtual bool ConfigureProperties() override;
	// End UFactory Interface
};
