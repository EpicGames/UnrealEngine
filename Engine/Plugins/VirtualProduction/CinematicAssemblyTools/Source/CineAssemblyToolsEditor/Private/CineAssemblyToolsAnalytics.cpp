// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblyToolsAnalytics.h"

#include "CineAssembly.h"
#include "CineAssemblySchema.h"
#include "EngineAnalytics.h"
#include "Interfaces/IPluginManager.h"
#include "UObject/Package.h"

namespace UE::CineAssemblyToolsAnalytics
{
	void RecordEvent_CreateAssembly(const UCineAssembly* const Assembly)
	{
		if (FEngineAnalytics::IsAvailable() && Assembly)
		{
			const UCineAssemblySchema* const Schema = Assembly->GetSchema();
			const bool bHasSchema = (Schema != nullptr);
			bool bHasUserSchema = false;

			if (bHasSchema && Schema->GetPackage())
			{
				// Check if the schema's package comes from the Cinematic Assembly Tools plugin
				const FString Name = Schema->GetPackage()->GetName();
				const FString PluginPackageName = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetMountedAssetPath();

				bHasUserSchema = !(Name.StartsWith(PluginPackageName, ESearchCase::CaseSensitive));
			}

			TArray<FAnalyticsEventAttribute> EventAttributes;
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("HasSchema"), bHasSchema));
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("HasUserSchema"), bHasUserSchema));

			FEngineAnalytics::GetProvider().RecordEvent(TEXT("CinematicAssemblyTools.CreateCineAssembly"), EventAttributes);
		}
	}

	void RecordEvent_CreateAssemblySchema()
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("CinematicAssemblyTools.CreateCineAssemblySchema"));
		}
	}

	void RecordEvent_CreateProduction()
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("CinematicAssemblyTools.CreateProduction"));
		}
	}

	void RecordEvent_ProductionAddAssetNaming()
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("CinematicAssemblyTools.Productions.AddAssetNaming"));
		}
	}

	void RecordEvent_ProductionCreateTemplateFolders()
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("CinematicAssemblyTools.Productions.CreateTemplateFolders"));
		}
	}

	void RecordEvent_RecordAssembly()
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("CinematicAssemblyTools.TakeRecorder.RecordAssembly"));
		}
	}
}
