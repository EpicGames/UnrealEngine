// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UCineAssembly;

namespace UE::CineAssemblyToolsAnalytics
{
	void RecordEvent_CreateAssembly(const UCineAssembly* const Assembly);
	void RecordEvent_CreateAssemblySchema();
	void RecordEvent_CreateProduction();
	void RecordEvent_ProductionAddAssetNaming();
	void RecordEvent_ProductionCreateTemplateFolders();
	void RecordEvent_RecordAssembly();
};