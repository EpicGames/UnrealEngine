// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblyToolsEditorModule.h"

#include "Algo/AllOf.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "CineAssembly.h"
#include "CineAssemblyCustomization.h"
#include "CineAssemblyMetadataCustomization.h"
#include "CineAssemblyNamingTokens.h"
#include "CineAssemblySchema.h"
#include "CineAssemblySchemaCustomization.h"
#include "CineAssemblyToolsStyle.h"
#include "CinematicProductionCustomization.h"
#include "ContentBrowserMenuContexts.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "ISequencerModule.h"
#include "LevelEditor.h"
#include "Misc/ConfigCacheIni.h"
#include "MoviePipelineQueue.h"
#include "NamingTokensEngineSubsystem.h"
#include "ProductionSettings.h"
#include "ProductionSettingsCustomization.h"
#include "PropertyEditorModule.h"
#include "Sequencer/CineAssemblySchemaSequencerCustomization.h"
#include "Sequencer/SubAssemblyTrackEditor.h"
#include "SequencerCustomizationManager.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ToolMenus.h"
#include "UI/CineAssembly/SCineAssemblyPropertyEditor.h"
#include "UI/CineAssembly/SCineAssemblySchemaWindow.h"
#include "UI/CineAssembly/SDuplicateAssemblyWindow.h"
#include "UI/SProductionWizard.h"
#include "Widgets/Input/SComboButton.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "CineAssemblyToolsEditorModule"

DEFINE_LOG_CATEGORY_STATIC(LogCineAssemblyToolsEditorModule, Log, All)

namespace UE::CineAssemblyTools
{
	FAssetCategoryPath CineAssemblyAssetCategory = FAssetCategoryPath(EAssetCategoryPaths::Cinematics, LOCTEXT("CineAssemblyCategorySection", "Cinematic Assembly Tools"), ECategoryMenuType::Section);
}

static const FName ProductionWizardTabName("ProductionWizard");

const FString FCineAssemblyToolsEditorModule::OpenTabSection = TEXT("CinematicAssemblyTools_OpenTabs");

void FCineAssemblyToolsEditorModule::StartupModule()
{
	FCineAssemblyToolsStyle::Get();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyEditorModule.RegisterCustomClassLayout(
		UProductionSettings::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FProductionSettingsCustomization::MakeInstance)
	);

	PropertyEditorModule.RegisterCustomClassLayout(
		UCineAssembly::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCineAssemblyCustomization::MakeInstance)
	);

	PropertyEditorModule.RegisterCustomClassLayout(
		UCineAssemblySchema::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCineAssemblySchemaCustomization::MakeInstance)
	);

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
		FCinematicProduction::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCinematicProductionCustomization::MakeInstance)
	);

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
		FAssemblyMetadataDesc::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCineAssemblyMetadataCustomization::MakeInstance)
	);

	// Add a new entry to the Tools->Cinematics menu to spawn the Production Wizard
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ProductionWizardTabName, FOnSpawnTab::CreateRaw(this, &FCineAssemblyToolsEditorModule::SpawnProductionWizard))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCinematicsCategory())
		.SetTooltipText(LOCTEXT("ProductionWizardTabTooltip", "Open the Production Wizard"))
		.SetDisplayName(LOCTEXT("ProductionWizardTabTitle", "Production Wizard"))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ProjectSettings.TabIcon"));

	RegisterMenus();

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FCineAssemblyToolsEditorModule::OnPostEngineInit);

	OnSequenceSetHandle = UMoviePipelineExecutorJob::OnSequenceSetEvent().AddRaw(this, &FCineAssemblyToolsEditorModule::OnSequenceSet);

	// Initialize the maps of managed tabs
	constexpr int32 MaxAssetTabs = 16;
	for (int32 TabIndex = 0; TabIndex < MaxAssetTabs; ++TabIndex)
	{
		const FString AssemblyTabName = FString::Printf(TEXT("CineAssemblyTab%d"), TabIndex);
		ManagedAssemblyTabs.Add(FTabId(*AssemblyTabName), FGuid());

		const FString SchemaTabName = FString::Printf(TEXT("CineAssemblySchemaTab%d"), TabIndex);
		ManagedSchemaTabs.Add(FTabId(*SchemaTabName), FGuid());
	}

	// Register the SubAssembly Track Editor used by the Schema Sequencer tool
	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	RegisterSubTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FSubAssemblyTrackEditor::CreateTrackEditor));

	// Register a Sequencer customization for UCineAssembly assets being edited in a schema template
	if (TSharedPtr<FSequencerCustomizationManager> CustomizationManager = SequencerModule.GetSequencerCustomizationManager())
	{
		CustomizationManager->RegisterInstancedSequencerCustomization(
			UCineAssembly::StaticClass(),
			FOnGetSequencerCustomizationInstance::CreateLambda([]() -> ISequencerCustomization*
			{
				return new FCineAssemblySchemaSequencerCustomization();
			}));
	}

	// Listen for property changes on UCineAssemblySchema so that the Style brush cache stays in sync
	SchemaPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FCineAssemblyToolsEditorModule::OnSchemaPropertyChanged);
}

void FCineAssemblyToolsEditorModule::RegisterProductionExtension(const UScriptStruct& DataScriptStruct)
{
	ProductionExtensions.RegisterProductionExtension(DataScriptStruct);
}

void FCineAssemblyToolsEditorModule::UnregisterProductionExtension(const UScriptStruct& DataScriptStruct)
{
	ProductionExtensions.UnregisterProductionExtension(DataScriptStruct);
}

void FCineAssemblyToolsEditorModule::RegisterProductionWizardCustomization(const UScriptStruct& ForDataScriptStruct, FGetWidget MakeCustomWidget, 
	TAttribute<FText> Label, TAttribute<FSlateIcon> Icon, bool bShowProductionSelection, bool bHideInWizard)
{
	ProductionExtensions.RegisterProductionWizardCustomization(ForDataScriptStruct, MakeCustomWidget, Label, Icon, bShowProductionSelection, bHideInWizard);
}

void FCineAssemblyToolsEditorModule::UnregisterProductionWizardCustomization(const UScriptStruct& ForDataScriptStruct)
{
	ProductionExtensions.UnregisterProductionWizardCustomization(ForDataScriptStruct);
}

void FCineAssemblyToolsEditorModule::RegisterProductionWizardUserSettings(const FName Name, FGetWidget MakeCustomWidget, TAttribute<FText> Label, TAttribute<FSlateIcon> Icon)
{
	ProductionExtensions.RegisterProductionWizardUserSettings(Name, MakeCustomWidget, Label, Icon);
}

void FCineAssemblyToolsEditorModule::UnregisterProductionWizardUserSettings(const FName Name)
{
	ProductionExtensions.UnregisterProductionWizardUserSettings(Name);
}

void FCineAssemblyToolsEditorModule::OnPostEngineInit()
{
	TakeRecorderIntegration = MakeUnique<FCineAssemblyTakeRecorderIntegration>();

	// Register with the asset tools module so that default asset names can be evaluated for tokens before being used to create new assets
	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
	AssetTools.RegisterSanitizeNameDelegate("CineAssemblyToolsEditorModule", FSanitizeName::CreateLambda([](FString& NameToSanitize)
		{
			if (UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>())
			{
				FNamingTokenFilterArgs FilterArgs;
				FilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

				FNamingTokenResultData Result = NamingTokensSubsystem->EvaluateTokenString(NameToSanitize, FilterArgs);
				NameToSanitize = Result.EvaluatedText.ToString();
			}
		}));

	// Load any previously opened assembly tabs and register new nomad tab spawners for them so they can be properly restored in the layout.
	for (TPair<FTabId, FGuid>& TabPair : ManagedAssemblyTabs)
	{
		const FName TabName = TabPair.Key.TabType;
		const FGuid AssetID = FindTabAssetInConfig(TabName);

		if (AssetID.IsValid())
		{
			TabPair.Value = AssetID;

			FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TabName, FOnSpawnTab::CreateRaw(this, &FCineAssemblyToolsEditorModule::SpawnAssemblyTab, AssetID))
				.SetMenuType(ETabSpawnerMenuType::Hidden);
		}
	}

	// Load any previously opened schema tabs and register new nomad tab spawners for them so they can be properly restored in the layout.
	for (TPair<FTabId, FGuid>& TabPair : ManagedSchemaTabs)
	{
		const FName TabName = TabPair.Key.TabType;
		const FGuid AssetID = FindTabAssetInConfig(TabName);

		if (AssetID.IsValid())
		{
			TabPair.Value = AssetID;
			FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TabName, FOnSpawnTab::CreateRaw(this, &FCineAssemblyToolsEditorModule::SpawnSchemaTab, AssetID))
				.SetMenuType(ETabSpawnerMenuType::Hidden);
		}
	}

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.Get().OnInMemoryAssetDeleted().AddRaw(this, &FCineAssemblyToolsEditorModule::OnAssetDeleted);

	RegisterTokens();

	ExtendLevelEditorToolbar();
}

void FCineAssemblyToolsEditorModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(SchemaPropertyChangedHandle);
	TakeRecorderIntegration.Reset();

	if (UObjectInitialized() && FSlateApplication::IsInitialized())
	{
		UToolMenus::Get()->RemoveSection("LevelEditor.LevelEditorToolBar.User", "Productions");
	}
	
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.UnregisterCustomClassLayout(UProductionSettings::StaticClass()->GetFName());
	PropertyModule.UnregisterCustomClassLayout(UCineAssembly::StaticClass()->GetFName());
	PropertyModule.UnregisterCustomClassLayout(UCineAssemblySchema::StaticClass()->GetFName());
	PropertyModule.UnregisterCustomPropertyTypeLayout(FCinematicProduction::StaticStruct()->GetFName());
	PropertyModule.UnregisterCustomPropertyTypeLayout(FAssemblyMetadataDesc::StaticStruct()->GetFName());

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ProductionWizardTabName);

	UMoviePipelineExecutorJob::OnSequenceSetEvent().Remove(OnSequenceSetHandle);

	if (FAssetToolsModule* AssetToolsModule = FModuleManager::GetModulePtr<FAssetToolsModule>("AssetTools"))
	{
		AssetToolsModule->Get().UnregisterSanitizeNameDelegate("CineAssemblyToolsEditorModule");
	}

	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry"))
	{
		AssetRegistryModule->Get().OnInMemoryAssetDeleted().RemoveAll(this);
	}

	if (ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer"))
	{
		SequencerModule->UnRegisterTrackEditor(RegisterSubTrackEditorHandle);

		if (TSharedPtr<FSequencerCustomizationManager> CustomizationManager = SequencerModule->GetSequencerCustomizationManager())
		{
			CustomizationManager->UnregisterInstancedSequencerCustomization(UCineAssembly::StaticClass());
		}
	}

	GConfig->EmptySection(*OpenTabSection, GEditorPerProjectIni);
	SaveOpenTabs(ManagedAssemblyTabs);
	SaveOpenTabs(ManagedSchemaTabs);
}

void FCineAssemblyToolsEditorModule::OnSchemaPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	const UCineAssemblySchema* Schema = Cast<UCineAssemblySchema>(Object);
	if (!Schema)
	{
		return;
	}

	if (PropertyChangedEvent.GetPropertyName() != GET_MEMBER_NAME_CHECKED(UCineAssemblySchema, ThumbnailTexture))
	{
		return;
	}

	UTexture2D* Texture = Schema->ThumbnailTexture.LoadSynchronous();
	FCineAssemblyToolsStyle::Get().SetThumbnailBrushTextureForSchema(Schema->GetSchemaGuid(), Texture);
}

void FCineAssemblyToolsEditorModule::RegisterTokens()
{
	if (UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>())
	{
		UCineAssemblyNamingTokens* CineAssemblyNamingTokens = Cast<UCineAssemblyNamingTokens>(NamingTokensSubsystem->GetNamingTokens(UCineAssemblyNamingTokens::TokenNamespace));

		FGuid ExternalTokensGuid = FGuid::NewGuid();
		TArray<FNamingTokenData>& ExternalTokens = CineAssemblyNamingTokens->RegisterExternalTokens(ExternalTokensGuid);

		// Register the {activeProduction} token with the cine assembly tokens object
		FNamingTokenData ActiveProductionToken;
		ActiveProductionToken.TokenKey = TEXT("activeProduction");
		ActiveProductionToken.DisplayName = LOCTEXT("ActiveProductionTokenName", "Active Production");
		ActiveProductionToken.TokenProcessorNative.BindLambda([]()
			{
				const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
				TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
				if (ActiveProduction.IsSet())
				{
					return FText::FromString(ActiveProduction.GetValue().ProductionName);
				}
				return FText::GetEmpty();
			});

		ExternalTokens.Add(ActiveProductionToken);
	}
}

TSharedRef<SDockTab> FCineAssemblyToolsEditorModule::SpawnProductionWizard(const FSpawnTabArgs& SpawnTabArgs)
{
	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SNew(SProductionWizard)
		];
}

TSharedRef<SDockTab> FCineAssemblyToolsEditorModule::SpawnAssemblyTab(const FSpawnTabArgs& SpawnTabArgs, FGuid AssemblyGuid)
{
	TSharedPtr<SCineAssemblyPropertyEditor> Widget = SNew(SCineAssemblyPropertyEditor, AssemblyGuid);
	return SpawnAssemblyTab(SpawnTabArgs, Widget);
}

TSharedRef<SDockTab> FCineAssemblyToolsEditorModule::SpawnAssemblyTab(const FSpawnTabArgs& SpawnTabArgs, UCineAssembly* Assembly)
{
	TSharedPtr<SCineAssemblyPropertyEditor> Widget = SNew(SCineAssemblyPropertyEditor, Assembly);
	return SpawnAssemblyTab(SpawnTabArgs, Widget);
}

TSharedRef<SDockTab> FCineAssemblyToolsEditorModule::SpawnAssemblyTab(const FSpawnTabArgs& SpawnTabArgs, TSharedPtr<SCineAssemblyPropertyEditor> Widget)
{
	TSharedRef<SDockTab> AssemblyTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label_Lambda([Widget]() -> FText
			{
				return FText::FromString(Widget->GetAssemblyName());
			})
		[
			Widget.ToSharedRef()
		];

	AssemblyTab->SetTabIcon(FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.Assembly").GetIcon());

	// Unregister the spawner and reset the map entry when the tab closes
	const FTabId TabId = SpawnTabArgs.GetTabId();
	AssemblyTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda([this, TabId](TSharedRef<SDockTab> Tab)
		{
			FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(*TabId.ToString());
			ManagedAssemblyTabs[TabId].Invalidate();
		}));

	return AssemblyTab;
}

TSharedRef<SDockTab> FCineAssemblyToolsEditorModule::SpawnSchemaTab(const FSpawnTabArgs& SpawnTabArgs, FGuid SchemaGuid)
{
	TSharedPtr<SCineAssemblySchemaWindow> Widget = SNew(SCineAssemblySchemaWindow, SchemaGuid);
	return SpawnSchemaTab(SpawnTabArgs, Widget);
}

TSharedRef<SDockTab> FCineAssemblyToolsEditorModule::SpawnSchemaTab(const FSpawnTabArgs& SpawnTabArgs, UCineAssemblySchema* Schema)
{
	TSharedPtr<SCineAssemblySchemaWindow> Widget = SNew(SCineAssemblySchemaWindow, Schema);
	return SpawnSchemaTab(SpawnTabArgs, Widget);
}

TSharedRef<SDockTab> FCineAssemblyToolsEditorModule::SpawnSchemaTab(const FSpawnTabArgs& SpawnTabArgs, TSharedPtr<SCineAssemblySchemaWindow> Widget)
{
	TSharedRef<SDockTab> SchemaTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label_Lambda([Widget]() -> FText
			{
				return FText::FromString(Widget->GetSchemaName());
			})
		[
			Widget.ToSharedRef()
		];

	SchemaTab->SetTabIcon(FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.Schema").GetIcon());

	// Unregister the spawner and reset the map entry when the tab closes
	const FTabId TabId = SpawnTabArgs.GetTabId();
	SchemaTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda([this, TabId](TSharedRef<SDockTab> Tab)
		{
			FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(*TabId.ToString());
			ManagedSchemaTabs[TabId].Invalidate();
		}));

	return SchemaTab;
}

void FCineAssemblyToolsEditorModule::RegisterMenus()
{
	UToolMenus* ToolMenus = UToolMenus::Get();

	// Add Content Browser context menu actions to Cine Assembly assets
	const FName ContextMenuName = TEXT("ContentBrowser.AssetContextMenu.LevelSequence");
	if (UToolMenu* Menu = ToolMenus->ExtendMenu(ContextMenuName))
	{
		const FName AssetActionSectionName = TEXT("GetAssetActions");
		FToolMenuSection& Section = Menu->FindOrAddSection(AssetActionSectionName);

		FToolMenuEntry& CineAssemblyActionsEntry = Section.AddDynamicEntry("CineAssemblyActions", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
		{
			if (UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>())
			{
				// Ensure all selected assets are Cine Assemblies
				const TArray<FAssetData>& SelectedAssets = Context->SelectedAssets;
				if (Algo::AllOf(SelectedAssets, [](const FAssetData& AssetData) { return AssetData.IsInstanceOf(UCineAssembly::StaticClass()); }))
				{
					if (SelectedAssets.Num() == 1)
					{
						const FAssetData& CineAssemblyData = SelectedAssets[0];
						UCineAssembly* CineAssembly = Cast<UCineAssembly>(CineAssemblyData.GetAsset());

						// Add an action to open the Cine Assembly asset in Sequencer without loading its associated map
						InSection.AddMenuEntry(
							"OpenInCurrentMapEntry",
							LOCTEXT("OpenInCurrentMap", "Open in Current Map"),
							LOCTEXT("OpenInCurrentMapTooltip", "Opens the level sequence in Sequencer but does not automatically load the associated map"),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.OpenCinematic"),
							FUIAction(FExecuteAction::CreateLambda([CineAssembly]()
								{
									// Note: Normally, when a cine assembly is opened in the editor, its associated map is loaded during the PrepareToActiveAssets step.
									// By directly opening the asset editor (Sequencer) for this asset, we can skip that map loading step. 
									GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets({ CineAssembly });
								})),
							EUserInterfaceActionType::None
						);

						// Add an action to open the Duplicate Assembly window
						InSection.AddMenuEntry(
							"DuplicateAssemblyAction",
							LOCTEXT("DuplicateAssembly", "Duplicate Assembly"),
							LOCTEXT("DuplicateAssemblyTooltip", "Opens the Duplicate Assembly tool"),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate"),
							FUIAction(FExecuteAction::CreateLambda([CineAssembly]()
								{
									TSharedRef<SDuplicateAssemblyWindow> DuplicateAssemblyWindow = SNew(SDuplicateAssemblyWindow, CineAssembly);
									FSlateApplication::Get().AddWindow(DuplicateAssemblyWindow);
								})),
							EUserInterfaceActionType::None
						);
					}

					InSection.AddMenuEntry(
						"OpenInAssetEditor",
						LOCTEXT("OpenInAssetEditor", "Edit Properties"),
						LOCTEXT("OpenInAssetEditorTooltip", "Edit Properties"),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
						FUIAction(FExecuteAction::CreateLambda([this, SelectedAssets]()
							{
								for (const FAssetData& AssetData : SelectedAssets)
								{
									UCineAssembly* CineAssembly = CastChecked<UCineAssembly>(AssetData.GetAsset());
									OpenAssemblyForEdit(CineAssembly);
								}
							})),
						EUserInterfaceActionType::None
					);
				}
			}
		}));

		// Ensure menu order by forcing our entry after the ones added by TakeRecorder even if the load order differs.
		const FName RecordIntoTakeRecorderEntry = "RecordIntoTakeRecorder_Label";
		CineAssemblyActionsEntry.InsertPosition.Name = RecordIntoTakeRecorderEntry;
		CineAssemblyActionsEntry.InsertPosition.Position = EToolMenuInsertType::After;
	}
}

void FCineAssemblyToolsEditorModule::OpenAssemblyForEdit(UCineAssembly* Assembly)
{
	// Check if the assembly is already open in one of the managed tabs, and if so, bring focus to the existing tab
	const FGuid AssemblyID = Assembly->GetAssemblyGuid();
	const FTabId* ExistingTab = ManagedAssemblyTabs.FindKey(AssemblyID);

	if (ExistingTab)
	{
		FGlobalTabmanager::Get()->TryInvokeTab(*ExistingTab);
		return;
	}

	// Try to get the TabID for the next available tab in the map.
	FTabId NewTabId;
	if (!TryGetNextTab(ManagedAssemblyTabs, NewTabId))
	{
		UE_LOGF(LogCineAssemblyToolsEditorModule, Warning, "The Cine Assembly tab could not be opened because the maximum number of Assembly tabs are already open.");
		return;
	}

	// Register a new spawner and invoke a new tab to edit the properties of the assembly asset
	FTabSpawnerEntry& Tmp = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(NewTabId.TabType, FOnSpawnTab::CreateRaw(this, &FCineAssemblyToolsEditorModule::SpawnAssemblyTab, Assembly))
		.SetMenuType(ETabSpawnerMenuType::Hidden);

	FGlobalTabmanager::Get()->TryInvokeTab(NewTabId);

	ManagedAssemblyTabs[NewTabId] = AssemblyID;
}

void FCineAssemblyToolsEditorModule::OpenSchemaForEdit(UCineAssemblySchema* Schema)
{
	// Check if the assembly is already open in one of the managed tabs, and if so, bring focus to the existing tab
	const FGuid SchemaID = Schema->GetSchemaGuid();
	const FTabId* ExistingTab = ManagedSchemaTabs.FindKey(SchemaID);

	if (ExistingTab)
	{
		FGlobalTabmanager::Get()->TryInvokeTab(*ExistingTab);
		return;
	}

	// Try to get the TabID for the next available tab in the map.
	FTabId NewTabId;
	if (!TryGetNextTab(ManagedSchemaTabs, NewTabId))
	{
		UE_LOGF(LogCineAssemblyToolsEditorModule, Warning, "The Cine Assembly Schema tab could not be opened because the maximum number of Schema tabs are already open.");
		return;
	}

	// Register a new spawner and invoke a new tab to edit the properties of the assembly asset
	FTabSpawnerEntry& Tmp = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(NewTabId.TabType, FOnSpawnTab::CreateRaw(this, &FCineAssemblyToolsEditorModule::SpawnSchemaTab, Schema))
		.SetMenuType(ETabSpawnerMenuType::Hidden);

	FGlobalTabmanager::Get()->TryInvokeTab(NewTabId);

	ManagedSchemaTabs[NewTabId] = SchemaID;
}

bool FCineAssemblyToolsEditorModule::TryGetNextTab(const TMap<FTabId, FGuid>& TabMap, FTabId& OutTabId)
{
	// Find the first tab in the map that is not currently mapped to a valid asset
	for (const TPair<FTabId, FGuid>& TabPair : TabMap)
	{
		if (!TabPair.Value.IsValid())
		{
			OutTabId = TabPair.Key;
			return true;
		}
	}

	return false;
}

FGuid FCineAssemblyToolsEditorModule::FindTabAssetInConfig(FName TabName)
{
	FString GuidString;
	if (GConfig->GetString(*OpenTabSection, *TabName.ToString(), GuidString, GEditorPerProjectIni))
	{
		return FGuid(GuidString);
	}
	return FGuid();
}

void FCineAssemblyToolsEditorModule::SaveOpenTabs(const TMap<FTabId, FGuid>& TabMap)
{
	// Write out each managed tab that is associated with a valid asset to a config file so that we can restore them the next time the editor launches
	for (const TPair<FTabId, FGuid>& TabPair : TabMap)
	{
		const FName TabName = TabPair.Key.TabType;
		const FGuid AssetID = TabPair.Value;
		if (AssetID.IsValid())
		{
			GConfig->AddToSection(*OpenTabSection, TabName, AssetID.ToString(), GEditorPerProjectIni);

			if (TSharedPtr<FGlobalTabmanager> GlobalTabManager = FGlobalTabmanager::Get())
			{
				GlobalTabManager->UnregisterNomadTabSpawner(TabName);
			}
		}
	}
}

void FCineAssemblyToolsEditorModule::ExtendLevelEditorToolbar()
{
	if (!FSlateApplication::IsInitialized() || !GetDefault<UProductionSettings>()->IsLevelEditorToolbarEnabled())
	{
		return;
	}
	
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.User");

	// Custom combo button so we resemble the mode selection combo.
	const FToolMenuEntry ProductionsEntry = FToolMenuEntry::InitWidget(
		"ProductionsMenu",
		SNew(SComboButton)
		.ToolTipText(LOCTEXT("LevelEditorToolbarProductionsButtonTooltip", "Change the active production"))
		.OnGetMenuContent(FOnGetContent::CreateRaw(this, &FCineAssemblyToolsEditorModule::GenerateProductionLevelEditorToolbarMenu))
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, 6.f, 0.f)
			[
				SNew(SBox)
				.WidthOverride(16.f)
				.HeightOverride(16.f)
				[
					SNew(SImage)
					.Image(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ProjectSettings.TabIcon").GetIcon())
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.MaxWidth(200.f)
			[
				SNew(STextBlock)
				.Text(
					TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([this]()
					{
						const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
						TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
						return ActiveProduction.IsSet()
							? FText::FromString(ActiveProduction.GetValue().ProductionName)
							: LOCTEXT("LevelEditorToolbarProductionsButtonLabel", "Productions");
					}))
				)
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		],
		FText::GetEmpty()
	);
	
	Menu->FindOrAddSection("Productions").AddEntry(ProductionsEntry);
}

TSharedRef<SWidget> FCineAssemblyToolsEditorModule::GenerateProductionLevelEditorToolbarMenu()
{
    	const FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
    	const TSharedPtr<FUICommandList> CommandBindings = LevelEditorModule.GetGlobalLevelEditorActions();
    
    	constexpr bool bShouldCloseMenuAfterSelection = false;
    	FMenuBuilder MenuBuilder(bShouldCloseMenuAfterSelection, CommandBindings);
	
    	static const FName NoExtensionHook = NAME_None;
    	
    	{
    		MenuBuilder.BeginSection(NoExtensionHook);
    
    		FUIAction OpenProductionWizardAction;
    		OpenProductionWizardAction.ExecuteAction = FExecuteAction::CreateLambda([this]()
    		{
    			FGlobalTabmanager::Get()->TryInvokeTab(ProductionWizardTabName);
    		});
    		
    		MenuBuilder.AddMenuEntry(LOCTEXT("OpenProductionWizard", "Production Wizard"),
				LOCTEXT("OpenProductionsWizardToolTip", "Open the Productions Wizard"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "ProjectSettings.TabIcon"),
				OpenProductionWizardAction);
    
    		MenuBuilder.EndSection();
    	}

		// All production options
    	{
    		MenuBuilder.BeginSection(FName("Productions"), LOCTEXT("ProductionsSection", "Productions"));
    		
    		UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
    		const TArray<FCinematicProduction> Productions = ProductionSettings->GetProductions();

    		ProductionList.Reset(Productions.Num());

    		// Add empty entry
    		{
    			FUIAction NoProductionAction(
				FExecuteAction::CreateLambda([this, ProductionSettings]()
				{
					return ProductionSettings->SetActiveProduction(FGuid());
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, ProductionSettings]()
				{
					return !ProductionSettings->GetActiveProduction().IsSet();
				}));

    			MenuBuilder.AddMenuEntry(
				LOCTEXT("NoProductionName", "No Production"),
				LOCTEXT("NoProductionTooltip", "Clear the active production"),
				FSlateIcon(),
				NoProductionAction,
				NAME_None,
				EUserInterfaceActionType::RadioButton);
    		}

    		// Add production list
    		for (const FCinematicProduction& Production : Productions)
    		{
			    TSharedRef<FCinematicProduction> ProductionRef = MakeShared<FCinematicProduction>(Production);
    			ProductionList.Add(ProductionRef);

    			FUIAction Action(
					FExecuteAction::CreateLambda([this, ProductionRef, ProductionSettings]()
					{
						return ProductionSettings->SetActiveProduction(ProductionRef->ProductionID);
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this, ProductionRef, ProductionSettings]()
					{
						return ProductionSettings->IsActiveProduction(ProductionRef->ProductionID);
					})
				);

    			const FText ProductionName = FText::FromString(Production.ProductionName);
    			
    			MenuBuilder.AddMenuEntry(
					ProductionName,
					ProductionName,
					FSlateIcon(),
					Action,
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);
    		}
    		
    		MenuBuilder.EndSection();
    	}
    	
    	return MenuBuilder.MakeWidget();
}

void FCineAssemblyToolsEditorModule::OnAssetDeleted(UObject* Object)
{
	// We only care about CineAssembly and CineAssemblySchema assets
	if (!(Object->IsA(UCineAssembly::StaticClass()) || Object->IsA(UCineAssemblySchema::StaticClass())))
	{
		return;
	}

	// Find the tab matching the ID of the asset being deleted (if there is one)
	FTabId AssetTabId;
	if (const UCineAssembly* Assembly = Cast<UCineAssembly>(Object))
	{
		if (const FTabId* OpenTabId = ManagedAssemblyTabs.FindKey(Assembly->GetAssemblyGuid()))
		{
			AssetTabId = *OpenTabId;
		}
	}
	else if (const UCineAssemblySchema* Schema = Cast<UCineAssemblySchema>(Object))
	{
		if (const FTabId* OpenTabId = ManagedSchemaTabs.FindKey(Schema->GetSchemaGuid()))
		{
			AssetTabId = *OpenTabId;
		}
	}

	// Get the already opened tab and close it
	if (!AssetTabId.TabType.IsNone())
	{
		constexpr bool bInvokeAsInactive = true;
		if (TSharedPtr<SDockTab> AssetTab = FGlobalTabmanager::Get()->TryInvokeTab(AssetTabId, bInvokeAsInactive))
		{
			AssetTab->RequestCloseTab();
		}
	}
}

void FCineAssemblyToolsEditorModule::OnSequenceSet(UMoviePipelineExecutorJob* Job, ULevelSequence* Sequence)
{
	if (Job)
	{
		if (const UCineAssembly* CineAssembly = Cast<UCineAssembly>(Sequence))
		{
			// For our cine assemblies, we want to always use the map associated with the sequence.
			if (CineAssembly->Level.IsValid())
			{
				Job->Map = CineAssembly->Level;
			}
		}
	}
}

IMPLEMENT_MODULE(FCineAssemblyToolsEditorModule, CineAssemblyToolsEditor);

#undef LOCTEXT_NAMESPACE
