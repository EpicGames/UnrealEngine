// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ICineAssemblyToolsEditorModule.h"
#include "Misc/AssetCategoryPath.h"
#include "Modules/ModuleManager.h"
#include "ProductionExtensions.h"
#include "TakeRecorder/CineAssemblyTakeRecorderIntegration.h"
#include "Widgets/Docking/SDockTab.h"

class UCineAssembly;
class UCineAssemblySchema;
class SCineAssemblyPropertyEditor;
class SCineAssemblySchemaWindow;
class UMoviePipelineExecutorJob;
class ULevelSequence;
struct FCinematicProduction;

namespace UE::CineAssemblyTools
{
	extern FAssetCategoryPath CineAssemblyAssetCategory;
}

class FCineAssemblyToolsEditorModule : public ICineAssemblyToolsEditorModule
{
public:

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	/** Opens a window to edit the properties of the input assembly schema */
	void OpenSchemaForEdit(UCineAssemblySchema* Schema);

	/** Get the ProductionExtensions object. */
	inline const FProductionExtensions& GetProductionExtensions() { return ProductionExtensions; }

	//~ Begin ICineAssemblyToolsEditorModule interface
	virtual void RegisterProductionExtension(const UScriptStruct& DataScriptStruct) override;
	virtual void UnregisterProductionExtension(const UScriptStruct& DataScriptStruct) override;
	virtual void RegisterProductionWizardCustomization(const UScriptStruct& ForDataScriptStruct,
		FGetWidget MakeCustomWidget, TAttribute<FText> Label, TAttribute<FSlateIcon> Icon, bool bShowProductionSelection, bool bHideInWizard) override;
	virtual void UnregisterProductionWizardCustomization(const UScriptStruct& ForDataScriptStruct) override;
	virtual void RegisterProductionWizardUserSettings(const FName Name, FGetWidget MakeCustomWidget, TAttribute<FText> Label, TAttribute<FSlateIcon> Icon) override;
	virtual void UnregisterProductionWizardUserSettings(const FName Name) override;
	//~ End ICineAssemblyToolsEditorModule interface

private:
	void OnPostEngineInit();

	/** Spawns a nomad tab for the Production Wizard tool */
	TSharedRef<SDockTab> SpawnProductionWizard(const FSpawnTabArgs& SpawnTabArgs);

	/** Spawns a nomad tab for an Assembly asset to edit its properties */
	TSharedRef<SDockTab> SpawnAssemblyTab(const FSpawnTabArgs& SpawnTabArgs, UCineAssembly* Assembly);
	TSharedRef<SDockTab> SpawnAssemblyTab(const FSpawnTabArgs& SpawnTabArgs, FGuid AssemblyGuid);
	TSharedRef<SDockTab> SpawnAssemblyTab(const FSpawnTabArgs& SpawnTabArgs, TSharedPtr<SCineAssemblyPropertyEditor> Widget);

	/** Spawns a nomad tab for an Schema asset to edit its properties */
	TSharedRef<SDockTab> SpawnSchemaTab(const FSpawnTabArgs& SpawnTabArgs, UCineAssemblySchema* Schema);
	TSharedRef<SDockTab> SpawnSchemaTab(const FSpawnTabArgs& SpawnTabArgs, FGuid SchemaGuid);
	TSharedRef<SDockTab> SpawnSchemaTab(const FSpawnTabArgs& SpawnTabArgs, TSharedPtr<SCineAssemblySchemaWindow> Widget);

	/** Add context menu actions */
	void RegisterMenus();

	/** Register additional tokens with the cine assembly tokens */
	void RegisterTokens();

	/** Called whenever a job has its sequence set. */
	void OnSequenceSet(UMoviePipelineExecutorJob* Job, ULevelSequence* Sequence);

	/** Opens a window to edit the properties of the input assembly */
	void OpenAssemblyForEdit(UCineAssembly* Assembly);

	/** Returns the next available tab name in the map of managed tabs. If there are no available tabs, returns false */
	bool TryGetNextTab(const TMap<FTabId, FGuid>& TabMap, FTabId& OutTabId);

	/** Callback when an asset is deleted which allows us to check if it was a Cine Assembly and then close the tab for that Assembly (if one was open) */
	void OnAssetDeleted(UObject* Object);

	/** Look up the tab name in the open tabs config section and return the asset ID associated with that tab (if there is one) */
	FGuid FindTabAssetInConfig(FName TabName);

	/** Write out all of the tabs in the input tab map that are currently open (i.e. have a valid asset ID) */
	void SaveOpenTabs(const TMap<FTabId, FGuid>& TabMap);

	/** Extend the level editor toolbar with our menu. */
	void ExtendLevelEditorToolbar();

	/** Generates the toolbar button widget. */
	TSharedRef<SWidget> GenerateProductionLevelEditorToolbarMenu();

	/** Subscribe to CineAssemblySchema property changes in order to refresh Style brushes for Schema thumbnails */
	void OnSchemaPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);

private:
	/** Manages all integration of Cinematic Assembly Tools with Take Recorder. */
	TUniquePtr<FCineAssemblyTakeRecorderIntegration> TakeRecorderIntegration;

	/** Extensions for the Production data and Production Wizard. */
	FProductionExtensions ProductionExtensions;

	/** Map of TabIds to Asset IDs to track which assets are open in which tabs */
	TMap<FTabId, FGuid> ManagedAssemblyTabs;
	TMap<FTabId, FGuid> ManagedSchemaTabs;

	/** Available productions. */
	TArray<TSharedPtr<FCinematicProduction>> ProductionList;

	static const FString OpenTabSection;

	FDelegateHandle OnSequenceSetHandle;
	FDelegateHandle RegisterSubTrackEditorHandle;
	FDelegateHandle SchemaPropertyChangedHandle;
};
