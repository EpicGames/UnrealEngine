// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblyToolsStyle.h"

#include "Engine/Texture2D.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/CoreStyle.h"
#include "Styling/SegmentedControlStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

FName FCineAssemblyToolsStyle::StyleName("CineAssemblyToolsStyle");

FCineAssemblyToolsStyle& FCineAssemblyToolsStyle::Get()
{
	static FCineAssemblyToolsStyle Inst;
	return Inst;
}

FCineAssemblyToolsStyle::FCineAssemblyToolsStyle()
	: FSlateStyleSet(StyleName)
{
	// Icons
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("CinematicAssemblyTools"))->GetContentDir();
	SetContentRoot(ContentDir);
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	Set("ClassIcon.CineAssembly", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/SMAssembly_16.svg")), Icon16x16));
	Set("ClassThumbnail.CineAssembly", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/SMAssembly_64.svg")), Icon64x64));
	Set("ClassIcon.CineAssemblySchema", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/SMAssemblySchema_16.svg")), Icon16x16));
	Set("ClassThumbnail.CineAssemblySchema", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/SMAssemblySchema_64.svg")), Icon64x64));

	Set("Thumbnails.Schema", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/LevelSequenceActor_64.svg")), Icon64x64));

	Set("Icons.Assembly", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/SMAssembly_16.svg")), Icon16x16));
	Set("Icons.Schema", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/SMAssemblySchema_16.svg")), Icon16x16));

	Set("Icons.RevisionControl", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/RevisionControl.svg")), Icon16x16));
	Set("Icons.Productions", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/Camera.svg")), Icon16x16));
	Set("Icons.Sequencer", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/Sequencer_16.svg")), Icon16x16));
	Set("Icons.NamingTokens", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/SMToken_16.svg")), Icon16x16));
	Set("Icons.AssetNaming", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/Rename.svg")), Icon16x16));
	Set("Icons.Folder", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/folder-closed.svg")), Icon16x16));
	Set("Icons.Export", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/ExportAll.svg")), Icon16x16));
	Set("Icons.Initialize", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/Initialize_16.svg")), Icon16x16));
	Set("Icons.Animation", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/Animation_16.svg")), Icon16x16));
	Set("Icons.Notes", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/Note.svg")), Icon16x16));
	Set("Icons.DataAsset", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/DataAsset.svg")), Icon16x16));
	Set("Icons.DragHandle", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/drag-handle.svg")), Icon16x16));

	Set("Badges.RevisionControlWarning", new FSlateVectorImageBrush(RootToCoreContentDir(TEXT("Starship/SourceControl/Status/RevisionControlBadgeWarning.svg")), Icon16x16, FStyleColors::Warning));
	Set("Badges.RevisionControlConnected", new FSlateVectorImageBrush(RootToCoreContentDir(TEXT("Starship/SourceControl/Status/RevisionControlBadgeConnected.svg")), Icon16x16, FStyleColors::Success));

	Set("Badges.FolderNew", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/CircleBadgeDot.svg")), Icon16x16, FStyleColors::Success));
	Set("Badges.FolderRename", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/CircleBadgeDot.svg")), Icon16x16, FStyleColors::Warning));

	// Fonts
	Set("ProductionWizard.TitleFont", DEFAULT_FONT("Bold", 10));
	Set("ProductionWizard.HeadingFont", DEFAULT_FONT("Bold", 14));

	// Brushes
	Set("ProductionWizard.PanelBackground", new FSlateRoundedBoxBrush(FStyleColors::Panel, 0.0f, FStyleColors::Recessed, 2.0f));
	Set("ProductionWizard.RecessedBackground", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 0.0f, FStyleColors::Panel, 2.0f));

	Set("Borders.Background", new FSlateRoundedBoxBrush(FStyleColors::Background, 0.0f, FStyleColors::Panel, 2.0f));
	Set("Borders.PanelNoBorder", new FSlateRoundedBoxBrush(FStyleColors::Panel, 0.0f, FStyleColors::Panel, 0.0f));
	Set("Borders.RecessedNoBorder", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 0.0f, FStyleColors::Recessed, 0.0f));

	// Buttons
	const FButtonStyle RecessedButton = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Recessed, 0.0f, FStyleColors::Recessed, 0.0f))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Panel, 0.0f, FStyleColors::Panel, 0.0f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 0.0f, FStyleColors::Dropdown, 0.0f))
		.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Recessed, 0.0f, FStyleColors::Recessed, 0.0f));

	Set("ProductionWizard.RecessedButton", RecessedButton);

	const FButtonStyle PanelButton = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Panel, 0.0f, FStyleColors::Panel, 0.0f))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Header, 0.0f, FStyleColors::Header, 0.0f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 0.0f, FStyleColors::Dropdown, 0.0f))
		.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Panel, 0.0f, FStyleColors::Panel, 0.0f));

	Set("ProductionWizard.PanelButton", PanelButton);

	// Segmented Controls
	FCheckBoxStyle PrimarySegmentedBoxStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetUncheckedImage(FSlateNoResource())
		.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f))
		.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f))
		.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f))
		.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f))
		.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryPress, 4.0f))
		.SetForegroundColor(FStyleColors::Foreground)
		.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
		.SetPressedForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedPressedForegroundColor(FStyleColors::ForegroundHover)
		.SetPadding(FMargin(8, 3));

	Set("PrimarySegmentedControl", FSegmentedControlStyle()
		.SetControlStyle(PrimarySegmentedBoxStyle)
		.SetFirstControlStyle(PrimarySegmentedBoxStyle)
		.SetLastControlStyle(PrimarySegmentedBoxStyle)
		.SetBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Recessed, 4.0f))
		.SetUniformPadding(FMargin(1))
	);

	// Textbox styles
	FEditableTextBoxStyle DetailsEditableTextBoxStyle = FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");
	DetailsEditableTextBoxStyle.SetFont(DEFAULT_FONT("Regular", 8));

	Set("DetailsEditableTextBox", DetailsEditableTextBoxStyle);

	FTextBlockStyle DetailsArgumentStyle = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>(TEXT("NormalText"));
	DetailsArgumentStyle
		.SetFont(DEFAULT_FONT("Bold", 8))
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f));

	Set("DetailsEditableTextBoxArguments", DetailsArgumentStyle);

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FCineAssemblyToolsStyle::~FCineAssemblyToolsStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FName FCineAssemblyToolsStyle::GetThumbnailBrushNameForSchema(const FGuid& SchemaGuid) const
{
	const FName BrushName = *FString::Printf(TEXT("SchemaThumbnail.%s"), *SchemaGuid.ToString());
	return BrushName;
}

void FCineAssemblyToolsStyle::SetThumbnailBrushTextureForSchema(const FGuid& SchemaGuid, UTexture2D* BrushTexture)
{
	if (FSlateBrush* Result = BrushResources.FindRef(GetThumbnailBrushNameForSchema(SchemaGuid)))
	{
		// Update the resource object and image type of the existing brush
		if (BrushTexture)
		{
			Result->SetResourceObject(BrushTexture);
			Result->ImageType = ESlateBrushImageType::FullColor;
		}
		else
		{
			Result->SetResourceObject(nullptr);
			Result->ImageType = ESlateBrushImageType::Vector;
		}
	}
	else
	{
		// The new brush is initially created to use the schema icon so that it has a valid ResourceName, because the ResourceName property cannot be changed after the brush is created.
		// If the brush's ResourceObject is ever null, a valid ResourceName allows it to immediately fallback to using the schema icon instead.
		FSlateBrush* NewBrush = new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/SMAssemblySchema_64.svg")), FVector2D(64.0f, 64.0f));

		if (BrushTexture)
		{
			NewBrush->SetResourceObject(BrushTexture);
			NewBrush->ImageType = ESlateBrushImageType::FullColor;
		}

		Set(GetThumbnailBrushNameForSchema(SchemaGuid), NewBrush);
	}
}
