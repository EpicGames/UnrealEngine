// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

#include "Misc/Guid.h"

class UTexture2D;

/** Slate style set that defines all the styles for the Cinematic Assembly Tools */
class FCineAssemblyToolsStyle : public FSlateStyleSet
{
public:
	static FName StyleName;

	/** Access the singleton instance for this style set */
	static FCineAssemblyToolsStyle& Get();

	/** Returns the name of the thumbnail brush associated with the schema identified by the input GUID */
	FName GetThumbnailBrushNameForSchema(const FGuid& SchemaGuid) const;

	/**
	 * Sets the texture resource used by the thumbnail brush associated with the schema identified by the input GUID.
	 * If the schema does not have a thumbnail brush associated with it yet, a new one will be created first.
	 * If the texture resource is null, the brush will fallback to using a predefined icon.
	 */
	void SetThumbnailBrushTextureForSchema(const FGuid& SchemaGuid, UTexture2D* BrushTexture);

private:

	FCineAssemblyToolsStyle();
	~FCineAssemblyToolsStyle();
};
