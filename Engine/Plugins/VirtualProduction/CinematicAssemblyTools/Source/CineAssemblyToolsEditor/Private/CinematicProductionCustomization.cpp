// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinematicProductionCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "ProductionSettings.h"
#include "Widgets/Input/SEditableTextBox.h"

TSharedRef<IPropertyTypeCustomization> FCinematicProductionCustomization::MakeInstance()
{
	return MakeShareable(new FCinematicProductionCustomization);
}

void FCinematicProductionCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			InPropertyHandle->CreatePropertyValueWidget()
		];
}

void FCinematicProductionCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	uint32 NumChildren;
	PropertyHandle->GetNumChildren(NumChildren);

	for (uint32 Index = 0; Index < NumChildren; ++Index)
	{
		TSharedRef<IPropertyHandle> ChildPropertyHandle = PropertyHandle->GetChildHandle(Index).ToSharedRef();

		// Replace the default textbox widget that would be used for this FString property with one that will limit the maximum length of the Production Name
		if (ChildPropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FCinematicProduction, ProductionName))
		{
			IDetailPropertyRow& ProductionNameRow = ChildBuilder.AddProperty(ChildPropertyHandle);
			ProductionNameRow.CustomWidget()
				.NameContent()
				[
					ChildPropertyHandle->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SEditableTextBox)
						.Font(CustomizationUtils.GetRegularFont())
						.MaximumLength(UProductionSettings::ProductionNameMaxLength)
						.Text_Lambda([this, ChildPropertyHandle]() -> FText
							{
								FString ProductionName;
								ChildPropertyHandle->GetValue(ProductionName);
								return FText::FromString(ProductionName);
							})
						.OnTextCommitted_Lambda([this, ChildPropertyHandle](const FText& InText, ETextCommit::Type InCommitType)
							{
								const FString NewProductionName = InText.ToString();
								ChildPropertyHandle->SetValue(NewProductionName);
							})
				];
		}
		else
		{
			ChildBuilder.AddProperty(ChildPropertyHandle);
		}
	}
}
