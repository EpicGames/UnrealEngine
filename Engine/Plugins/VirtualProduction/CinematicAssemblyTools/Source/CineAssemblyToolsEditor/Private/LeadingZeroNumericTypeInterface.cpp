// Copyright Epic Games, Inc. All Rights Reserved.

#include "LeadingZeroNumericTypeInterface.h"

FString FLeadingZeroNumericTypeInterface::ToString(const int32& Value) const
{
	const FNumberFormattingOptions NumberFormattingOptions = FNumberFormattingOptions()
		.SetUseGrouping(false)
		.SetMinimumIntegralDigits(MinimumIntegralDigits);

	return FastDecimalFormat::NumberToString(Value, ExpressionParser::GetLocalizedNumberFormattingRules(), NumberFormattingOptions);
}

TOptional<int32> FLeadingZeroNumericTypeInterface::FromString(const FString& InString, const int32& InExistingValue)
{
	// Convert the string to an integer using the base default type interface
	const TOptional<int32> Value = TDefaultNumericTypeInterface<int32>::FromString(InString, InExistingValue);

	// If the value was successfully parsed, update the minimum number of digits to match the length of the input string
	if (Value.IsSet())
	{
		MinimumIntegralDigits = InString.Len();
	}

	return Value;
}
