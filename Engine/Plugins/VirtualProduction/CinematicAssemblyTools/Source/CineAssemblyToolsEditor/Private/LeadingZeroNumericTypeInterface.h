// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/NumericTypeInterface.h"

struct FLeadingZeroNumericTypeInterface : public TDefaultNumericTypeInterface<int32>
{
	FLeadingZeroNumericTypeInterface() = default;

	//~ Begin INumericTypeInterface overrides
	virtual FString ToString(const int32& Value) const override;
	virtual TOptional<int32> FromString(const FString& InString, const int32& InExistingValue) override;
	//~ End INumericTypeInterface overrides

private:
	int32 MinimumIntegralDigits = 1;
};