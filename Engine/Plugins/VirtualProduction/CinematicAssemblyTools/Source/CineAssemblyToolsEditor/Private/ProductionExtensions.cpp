// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProductionExtensions.h"

#include "CineAssemblyToolsEditorModule.h"

DEFINE_LOG_CATEGORY_STATIC(LogProductionExtensions, Log, All)

void FProductionExtensions::RegisterProductionExtension(const UScriptStruct& DataScriptStruct)
{
	if (!DataScriptStruct.IsInBlueprint())
	{
		// We do not allow or support blueprint generated structs
		ProductionExtensions.Add(&DataScriptStruct);
	}
	else
	{
		UE_LOGF(LogProductionExtensions, Warning, "Blueprint structs are not supported as Production Extended Data types.")
	}
}

void FProductionExtensions::UnregisterProductionExtension(const UScriptStruct& DataScriptStruct)
{
	ProductionExtensions.Remove(&DataScriptStruct);
}

void FProductionExtensions::RegisterProductionWizardCustomization(const UScriptStruct& ForDataScriptStruct, FGetWidget MakeCustomWidget, 
	TAttribute<FText> Label, TAttribute<FSlateIcon> Icon, bool bShowProductionSelection, bool bHideInWizard)
{
	// only register known extension types
	if (!ProductionExtensions.Contains(&ForDataScriptStruct))
	{
		return;
	}

	FProductionWizardCustomizationParameters Customization;
	Customization.Label = Label;
	Customization.Icon = Icon;
	Customization.MakeCustomWidget = MakeCustomWidget;
	Customization.bShowProductionSelection = bShowProductionSelection;
	Customization.bHideInWizard = bHideInWizard;
	ProductionWizardCustomizations.Add(&ForDataScriptStruct, Customization);
}

void FProductionExtensions::UnregisterProductionWizardCustomization(const UScriptStruct& ForDataScriptStruct)
{
	ProductionWizardCustomizations.Remove(&ForDataScriptStruct);
}

void FProductionExtensions::RegisterProductionWizardUserSettings(const FName Name, FGetWidget MakeCustomWidget, TAttribute<FText> Label, TAttribute<FSlateIcon> Icon)
{
	FProductionWizardUserSettingsExtensionParameters Context;
	Context.Label = Label.IsSet() ? Label.Get() : FText::FromString(Name.ToString());
	Context.MakeCustomWidget = MakeCustomWidget;
	Context.Icon = Icon;
	ProductionWizardExtendedUserSettings.Add(Name, Context);
}

void FProductionExtensions::UnregisterProductionWizardUserSettings(const FName Name)
{
	ProductionWizardExtendedUserSettings.Remove(Name);
}
