// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Textures/SlateIcon.h"
#include "ICineAssemblyToolsEditorModule.h"

/** Struct holding details on a Production Settings customization for the Production Wizard interface. */
struct FProductionWizardCustomizationParameters
{
	/** Label override to display on the tab in the Production Wizard. */
	TAttribute<FText> Label;

	/** Icon override to display on the tab in the Production Wizard. */
	TAttribute<FSlateIcon> Icon;

	/** Callback to create the SWidget to display in the Production Wizard. */
	FGetWidget MakeCustomWidget = nullptr;

	/** Whether the production selection combo box should be shown for this customization in the Production Wizard. */
	bool bShowProductionSelection = true;

	/** Hides the tab and widget completely from the Production Wizard. */
	bool bHideInWizard = false;
};

/** Struct holding details on a Users Settings extension for the Production Wizard interface. */
struct FProductionWizardUserSettingsExtensionParameters
{
	/** Label for the tab. */
	FText Label;

	/** Optional icon to display on the tab. */
	TAttribute<FSlateIcon> Icon;

	/** Callback to create the widget to display. */
	FGetWidget MakeCustomWidget = nullptr;
};

/** Class that allows registering of extensions to Productions and the Production Wizard. */
class FProductionExtensions
{
public:
	/** Register a new production extension struct type. */
	void RegisterProductionExtension(const UScriptStruct& DataScriptStruct);

	/** Unregister a Production extension struct type. */
	void UnregisterProductionExtension(const UScriptStruct& DataScriptStruct);

	/**
	 * Register a customization for the Production Wizard for the given extended data struct type.
	 * The extended data struct type must already have been registered as a Production extension using RegisterProductionExtension.
	 */
	void RegisterProductionWizardCustomization(const UScriptStruct& ForDataScriptStruct,
		FGetWidget MakeCustomWidget, TAttribute<FText> Label = TAttribute<FText>(),
		TAttribute<FSlateIcon> Icon = TAttribute<FSlateIcon>(), bool bShowProductionSelection = true, bool bHideInWizard = false);

	/** Unregister the registered customization for the given extended data struct type. */
	void UnregisterProductionWizardCustomization(const UScriptStruct& ForDataScriptStruct);

	/** Register a custom User Setting tab to show in the Production wizard by providing a callback to create the SWidget to show. */
	void RegisterProductionWizardUserSettings(const FName Name, FGetWidget MakeCustomWidget,
		TAttribute<FText> Label = TAttribute<FText>(), TAttribute<FSlateIcon> Icon = TAttribute<FSlateIcon>());

	/** Unregister the custom User Settings tab from the Production Wizard with the given name. */
	void UnregisterProductionWizardUserSettings(const FName Name);

	/** Gets all registered Production Extension types. */
	inline const TSet<const UScriptStruct*>& GetProductionExtensions() const { return ProductionExtensions; }

	/** Get registered production wizard customizations. */
	inline const TMap<const UScriptStruct*, FProductionWizardCustomizationParameters>& GetProductionWizardCustomizations() const { return ProductionWizardCustomizations; }

	/** Get the map of all registered custom Production Wizard User Settings tabs. */
	inline const TMap<FName, FProductionWizardUserSettingsExtensionParameters>& GetProductionWizardUserSettingsExtensions() const { return ProductionWizardExtendedUserSettings; }

private:
	/** Set of all registered Production Extension types. */
	TSet<const UScriptStruct*> ProductionExtensions;

	/** Map of all registered Production Wizard Production Settings customizations. */
	TMap<const UScriptStruct*, FProductionWizardCustomizationParameters> ProductionWizardCustomizations;

	/** Map of all registered Production Wizard Users Settings extensions. */
	TMap<FName, FProductionWizardUserSettingsExtensionParameters> ProductionWizardExtendedUserSettings;
};
