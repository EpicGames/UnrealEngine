// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProductionFunctionLibrary.h"

#include "CineAssemblyEditorFunctionLibrary.h"

UProductionSettings* UProductionFunctionLibrary::GetProductionSettings()
{
	return GetMutableDefault<UProductionSettings>();
}

TArray<FCinematicProduction> UProductionFunctionLibrary::GetAllProductions()
{
	const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
	return ProductionSettings->GetProductions();
}

bool UProductionFunctionLibrary::GetProduction(FGuid ProductionID, FCinematicProduction& Production)
{
	const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
	TOptional<const FCinematicProduction> FoundProduction = ProductionSettings->GetProduction(ProductionID);
	if (FoundProduction.IsSet())
	{
		Production = FoundProduction.GetValue();
		return true;
	}

	Production.ProductionID.Invalidate();
	return false;
}

bool UProductionFunctionLibrary::GetActiveProduction(FCinematicProduction& Production)
{
	const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
	TOptional<const FCinematicProduction> FoundProduction = ProductionSettings->GetActiveProduction();
	if (FoundProduction.IsSet())
	{
		Production = FoundProduction.GetValue();
		return true;
	}

	Production.ProductionID.Invalidate();
	return false;
}

void UProductionFunctionLibrary::SetActiveProduction(FCinematicProduction Production)
{
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ProductionSettings->SetActiveProduction(Production.ProductionID);
}

void UProductionFunctionLibrary::SetActiveProductionByID(FGuid ProductionID)
{
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ProductionSettings->SetActiveProduction(ProductionID);
}

void UProductionFunctionLibrary::ClearActiveProduction()
{
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ProductionSettings->SetActiveProduction(FGuid());
}

bool UProductionFunctionLibrary::IsActiveProduction(FGuid ProductionID)
{
	const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
	return ProductionSettings->IsActiveProduction(ProductionID);
}

void UProductionFunctionLibrary::AddProduction(FCinematicProduction Production)
{
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ProductionSettings->AddProduction(Production);
}

void UProductionFunctionLibrary::DeleteProduction(FGuid ProductionID)
{
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ProductionSettings->DeleteProduction(ProductionID);
}

void UProductionFunctionLibrary::RenameProduction(FGuid ProductionID, FString NewName)
{
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ProductionSettings->RenameProduction(ProductionID, NewName);
}

bool UProductionFunctionLibrary::GetProductionExtendedData(FGuid ProductionID, const UScriptStruct* DataStruct, FInstancedStruct& OutData)
{
	if (DataStruct == nullptr)
	{
		return false;
	}

	const UProductionSettings* ProductionSettings = GetDefault<UProductionSettings>();
	if (FConstStructView FoundStruct = ProductionSettings->GetProductionExtendedData(ProductionID, *DataStruct); FoundStruct.IsValid())
	{
		OutData.InitializeAs(DataStruct, FoundStruct.GetMemory());
		return true;
	}

	return false;
}

bool UProductionFunctionLibrary::SetProductionExtendedData(FGuid ProductionID, const FInstancedStruct& Data)
{
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	return ProductionSettings->SetProductionExtendedData(ProductionID, Data);
}

UCineAssembly* UProductionFunctionLibrary::CreateAssembly(UCineAssemblySchema* Schema, TSoftObjectPtr<UWorld> Level, TSoftObjectPtr<UCineAssembly> ParentAssembly, const TMap<FString, FString>& Metadata, const FString& Path, const FString& Name, bool bUseDefaultNameFromSchema)
{
	const FString NameOverride = (Schema && bUseDefaultNameFromSchema) ? FString() : Name;
	return UCineAssemblyEditorFunctionLibrary::CreateAssembly(Schema, Level, ParentAssembly, Metadata, Path, NameOverride);
}
