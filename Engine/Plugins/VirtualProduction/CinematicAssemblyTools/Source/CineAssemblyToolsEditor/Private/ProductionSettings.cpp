// Copyright Epic Games, Inc. All Rights Reserved.
#include "ProductionSettings.h"

#include "Algo/Contains.h"
#include "Algo/Find.h"
#include "Algo/IndexOf.h"
#include "AssetToolsModule.h"
#include "CineAssemblyToolsAnalytics.h"
#include "CineAssemblyToolsEditorModule.h"
#include "Engine/Engine.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "LevelSequenceProjectSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/OutputDeviceNull.h"
#include "MovieSceneToolsProjectSettings.h"
#include "NamingTokensEngineSubsystem.h"
#include "ProductionExtensions.h"
#include "Sections/MovieSceneSubSection.h"
#include "SourceControlHelpers.h"
#include "SSettingsEditorCheckoutNotice.h"

#define LOCTEXT_NAMESPACE "ProductionSettings"

DEFINE_LOG_CATEGORY_STATIC(LogProductionSettings, Log, All)

namespace UE::CineAssemblyTools::ProductionSettings::Private
{
	constexpr int32 TopDownHBiasValue = -100;
	constexpr int32 BottomUpHBiasValue = 100;
};

FCinematicProduction::FCinematicProduction()
	: ProductionName()
	, DefaultDisplayRate(FFrameRate(24, 1))
	, DefaultStartFrame(0)
	, SubsequencePriority(ESubsequencePriority::BottomUp)
{
	if (!ProductionID.IsValid())
	{
		ProductionID = FGuid::NewGuid();
	}

	// Ensure the engine is initialized and config loaded.
	if (GEngine && GEngine->IsInitialized())
	{
		LoadAllExtendedData();
	}
}

FInstancedStruct* FCinematicProduction::LoadExtendedData(const UScriptStruct& ForStruct)
{
	// Only data types that have been registered in Production Extensions can be loaded into a Production
	FCineAssemblyToolsEditorModule& CatEdModule = FModuleManager::Get().GetModuleChecked<FCineAssemblyToolsEditorModule>("CineAssemblyToolsEditor");

	if (!CatEdModule.GetProductionExtensions().GetProductionExtensions().Contains(&ForStruct))
	{
		return nullptr;
	}

	// Look to see if we have an exported value to import for this type.
	FInstancedStruct* Data = nullptr;
	if (FString* ConfigData = ExtendedDataConfig.Find(ForStruct.GetPathName()))
	{
		if (TOptional<FInstancedStruct*> ImportedData = ImportExtendedData(*ConfigData); ImportedData.IsSet())
		{
			Data = ImportedData.GetValue();
		}
	}
	else
	{
		Data = &AddUniqueExtendedData(ForStruct);
	}

	if (Data != nullptr)
	{
		// Sync up our config representation with any new additions
		ExportExtendedData(*Data);
	}

	return Data;
}

FInstancedStruct* FCinematicProduction::FindOrLoadExtendedData(const UScriptStruct& ForStruct)
{
	if (FInstancedStruct* Existing = ExtendedData.Find(ForStruct.GetPathName()))
	{
		ExportExtendedData(ForStruct);
		return Existing;
	}
	return LoadExtendedData(ForStruct);
}

const FInstancedStruct* FCinematicProduction::FindExtendedData(const UScriptStruct& ForStruct) const
{
	return ExtendedData.Find(ForStruct.GetPathName());
}

void FCinematicProduction::ExportExtendedData(const UScriptStruct& ForData)
{
	if (FInstancedStruct* DataStruct = ExtendedData.Find(ForData.GetPathName()))
	{
		ExportExtendedData(*DataStruct);
	}
}

const FName FCinematicProduction::GetExtendedDataMemberName()
{
	return GET_MEMBER_NAME_CHECKED(FCinematicProduction, ExtendedData);
}

void FCinematicProduction::LoadAllExtendedData()
{
	FCineAssemblyToolsEditorModule& CatEdModule = FModuleManager::Get().GetModuleChecked<FCineAssemblyToolsEditorModule>("CineAssemblyToolsEditor");
	
	FScopedModifyProductionExtendedData ModifyGuard(*this);
	for (const UScriptStruct* ExtendedDataType : CatEdModule.GetProductionExtensions().GetProductionExtensions())
	{
		if (ExtendedDataType != nullptr)
		{
			LoadExtendedData(*ExtendedDataType);
		}
	}
}

TOptional<FInstancedStruct*> FCinematicProduction::ImportExtendedData(const FString& ImportText)
{
	FInstancedStruct NewStruct;
	const TCHAR* Buffer = *ImportText;

	FOutputDeviceNull SilenceErrors;
	NewStruct.ImportTextItem(Buffer, PPF_None, nullptr, &SilenceErrors);

	if (NewStruct.IsValid())
	{
		return TOptional<FInstancedStruct*>(&ExtendedData.Add(NewStruct.GetScriptStruct()->GetPathName(), NewStruct));
	}
	return TOptional<FInstancedStruct*>();
}

void FCinematicProduction::ExportExtendedData(const FInstancedStruct& InData)
{
	const FSoftObjectPath ObjectPath = InData.GetScriptStruct()->GetPathName();
	if (FString* ExistingConfig = ExtendedDataConfig.Find(ObjectPath))
	{
		ExistingConfig->Empty();
		InData.ExportTextItem(*ExistingConfig, InData, nullptr, PPF_None, nullptr);
	}
	else
	{
		FString NewData;
		InData.ExportTextItem(NewData, InData, nullptr, PPF_None, nullptr);
		ExtendedDataConfig.Add(ObjectPath, NewData);
	}
	NotifyExtendedDataExported(InData.GetScriptStruct());
}

FInstancedStruct& FCinematicProduction::AddUniqueExtendedData(const UScriptStruct& ScriptStruct)
{
	if (FInstancedStruct* Existing = ExtendedData.Find(ScriptStruct.GetPathName()))
	{
		return *Existing;
	}

	FInstancedStruct Data;
	Data.InitializeAs(&ScriptStruct);
	return ExtendedData.Add(ScriptStruct.GetPathName(), Data);
}

void FCinematicProduction::NotifyExtendedDataExported(const UScriptStruct* ExportedStruct)
{
	if (!bBlockNotifyExtendedDataExported)
	{
		ExtendedDataExported.Broadcast(*this, ExportedStruct);
	}
}

FName UProductionSettings::GetContainerName() const
{
	return TEXT("Project");
}

FName UProductionSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

FName UProductionSettings::GetSectionName() const
{
	return TEXT("Production Settings");
}

void UProductionSettings::PostInitProperties()
{
	Super::PostInitProperties();

	CacheProjectDefaults();

	// Cache the original values for these tooltips to reset if the active production is set to "None"
	const ULevelSequenceProjectSettings* const LevelSequenceSettings = GetMutableDefault<ULevelSequenceProjectSettings>();
	FProperty* DefaultDisplayRateProperty = LevelSequenceSettings->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ULevelSequenceProjectSettings, DefaultDisplayRate));
	OriginalDefaultDisplayRateTooltip = DefaultDisplayRateProperty->GetMetaData(TEXT("ToolTip"));

	const UMovieSceneToolsProjectSettings* const MovieSceneToolsSettings = GetDefault<UMovieSceneToolsProjectSettings>();
	FProperty* DefaultStartTimeProperty = MovieSceneToolsSettings->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMovieSceneToolsProjectSettings, DefaultStartTime));
	OriginalDefaultStartTimeTooltip = DefaultStartTimeProperty->GetMetaData(TEXT("ToolTip"));

	// Load the serialized active production name from user settings
	const FString ConfigSection = UProductionSettings::StaticClass()->GetPathName();
	const FString PropertyName = GET_MEMBER_NAME_CHECKED(UProductionSettings, ActiveProductionName).ToString();

	FString LoadedProductionName;
	if (GConfig->GetString(*ConfigSection, *PropertyName, LoadedProductionName, GEditorPerProjectIni))
	{
		ActiveProductionName = LoadedProductionName;
	}
	else
	{
		ActiveProductionName = FString(TEXT(""));
	}

	// Initialize the active production based on the serialized active production name
	const FCinematicProduction* ActiveProduction = Algo::FindBy(Productions, ActiveProductionName, &FCinematicProduction::ProductionName);
	ActiveProductionID = ActiveProduction ? ActiveProduction->ProductionID : FGuid();

	// Apply the initial overrides to existing project settings based on the current active production
	if (ActiveProductionID.IsValid())
	{
		ApplyProjectOverrides();
	}

	FCoreDelegates::GetOnPostEngineInit().AddUObject(this, &UProductionSettings::OnPostEngineInit);
}

void UProductionSettings::OnPostEngineInit()
{
	// Register a filter with the naming tokens subsystem to apply the active production's namespace deny list when evaluating tokens
	UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();
	NamingTokensSubsystem->RegisterNamespaceFilter("CinematicProductionSettings", FFilterNamespace::CreateUObject(this, &UProductionSettings::FilterNamingTokenNamespaces));

	// Build and load extended production data
	ReloadProductionsExtendedData();
	BindToProductions();
	ProductionListChangedDelegate.AddLambda([this]() { BindToProductions(); });
}

void UProductionSettings::CacheProjectDefaults()
{
	// Cache the original values for these properties to reset if the active production is set to "None"
	const ULevelSequenceProjectSettings* const LevelSequenceSettings = GetMutableDefault<ULevelSequenceProjectSettings>();
	ProjectDefaultDisplayRate = LevelSequenceSettings->DefaultDisplayRate;

	const UMovieSceneToolsProjectSettings* const MovieSceneToolsSettings = GetDefault<UMovieSceneToolsProjectSettings>();
	ProjectDefaultStartTime = MovieSceneToolsSettings->DefaultStartTime;
}

FStructView UProductionSettings::GetMutableProductionExtendedData(FGuid ProductionID, const UScriptStruct& DataStruct)
{
	FStructView Result;
	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		if (FInstancedStruct* Found = Production->FindOrLoadExtendedData(DataStruct))
		{
			Result = *Found;
		}
	}
	return Result;
}

FConstStructView UProductionSettings::GetProductionExtendedData(FGuid ProductionID, const UScriptStruct& DataStruct) const
{
	FConstStructView Result;
	if (const FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		if (const FInstancedStruct* Found = Production->FindExtendedData(DataStruct))
		{
			Result = *Found;
		}
	}
	return Result;
}

bool UProductionSettings::SetProductionExtendedData(FGuid ProductionID, const FConstStructView& DataStructView)
{
	if (!DataStructView.IsValid())
	{
		return false;
	}

	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		if (FInstancedStruct* Found = Production->FindOrLoadExtendedData(*DataStructView.GetScriptStruct()))
		{
			FScopedModifyProductionExtendedData ScopedModify(*Production, *DataStructView.GetScriptStruct());
			*Found = DataStructView;
			return true;
		}
	}

	return false;
}

FScopedModifyProductionExtendedData UProductionSettings::ModifyProductionExtendedData(FGuid ProductionID)
{
	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		return FScopedModifyProductionExtendedData(*Production);
	}
	return FScopedModifyProductionExtendedData();
}

FScopedModifyProductionExtendedData UProductionSettings::ModifyProductionExtendedData(FGuid ProductionID, const UScriptStruct& ForStruct)
{
	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		return FScopedModifyProductionExtendedData(*Production, ForStruct);
	}
	return FScopedModifyProductionExtendedData();
}

FName UProductionSettings::GetProductionsPropertyMemberName() 
{
	return GET_MEMBER_NAME_CHECKED(UProductionSettings, Productions); 
}

void UProductionSettings::ApplyProjectOverrides()
{
	// Modify the property flags and tooltip of these overridden project settings
	// They should be read-only when driven by an active production, and writable when the active production is None
	ULevelSequenceProjectSettings* LevelSequenceSettings = GetMutableDefault<ULevelSequenceProjectSettings>();
	FProperty* DefaultDisplayRateProperty = LevelSequenceSettings->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ULevelSequenceProjectSettings, DefaultDisplayRate));
 
	UMovieSceneToolsProjectSettings* MovieSceneToolsSettings = GetMutableDefault<UMovieSceneToolsProjectSettings>();
 	FProperty* DefaultStartTimeProperty = MovieSceneToolsSettings->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMovieSceneToolsProjectSettings, DefaultStartTime));

	if (ActiveProductionID.IsValid())
	{
		DefaultDisplayRateProperty->SetPropertyFlags(EPropertyFlags::CPF_EditConst);
		DefaultStartTimeProperty->SetPropertyFlags(EPropertyFlags::CPF_EditConst);

		const FText ToolTipAddition = LOCTEXT("ToolTipAddition", "This property is being driven by the active production. To edit this value, change the production settings or set a different active production.");

		DefaultDisplayRateProperty->SetMetaData(TEXT("ToolTip"), *FString::Printf(TEXT("%s\n\n%s"), *OriginalDefaultDisplayRateTooltip, *ToolTipAddition.ToString()));
		DefaultStartTimeProperty->SetMetaData(TEXT("ToolTip"), *FString::Printf(TEXT("%s\n\n%s"), *OriginalDefaultStartTimeTooltip, *ToolTipAddition.ToString()));
	}
	else
	{
		DefaultDisplayRateProperty->ClearPropertyFlags(EPropertyFlags::CPF_EditConst);
		DefaultStartTimeProperty->ClearPropertyFlags(EPropertyFlags::CPF_EditConst);

		DefaultDisplayRateProperty->SetMetaData(TEXT("ToolTip"), *OriginalDefaultDisplayRateTooltip);
		DefaultStartTimeProperty->SetMetaData(TEXT("ToolTip"), *OriginalDefaultStartTimeTooltip);
	}

	OverrideDefaultDisplayRate();
	OverrideDefaultStartTime();
	OverrideSubsequenceHierarchicalBias();
	OverrideDefaultAssetNames();
}

void UProductionSettings::OverrideDefaultDisplayRate()
{
	// Update the DefaultDisplayRate property of the level sequence project settings to the value of the active production
	// If there is no active production, the level sequence project setting is reset to its default config value
	ULevelSequenceProjectSettings* LevelSequenceSettings = GetMutableDefault<ULevelSequenceProjectSettings>();

	if (const FCinematicProduction* ActiveProduction = GetActiveProductionPtr())
	{
		const FText FrameRateText = ActiveProduction->DefaultDisplayRate.ToPrettyText();
		LevelSequenceSettings->DefaultDisplayRate = FrameRateText.ToString();
	}
	else
	{
		LevelSequenceSettings->DefaultDisplayRate = ProjectDefaultDisplayRate;
	}

	// Update the default display rate cvar to match
	if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(TEXT("LevelSequence.DefaultDisplayRate")))
	{
		ConsoleVariable->Set(*LevelSequenceSettings->DefaultDisplayRate, ECVF_SetByProjectSetting);
	}
}

void UProductionSettings::OverrideDefaultStartTime()
{
	// Update the DefaultStartTime property of the movie scene tools project settings to the value of the active production
	// If there is no active production, the movie scene tools project setting is reset to its default config value
	UMovieSceneToolsProjectSettings* MovieSceneToolsSettings = GetMutableDefault<UMovieSceneToolsProjectSettings>();

	if (const FCinematicProduction* ActiveProduction = GetActiveProductionPtr())
	{
		MovieSceneToolsSettings->DefaultStartTime = ActiveProduction->DefaultDisplayRate.AsSeconds(FFrameNumber(ActiveProduction->DefaultStartFrame));
	}
	else
	{
		MovieSceneToolsSettings->DefaultStartTime = ProjectDefaultStartTime;
	}
}

void UProductionSettings::OverrideSubsequenceHierarchicalBias()
{
	if (const FCinematicProduction* ActiveProduction = GetActiveProductionPtr())
	{
		using namespace UE::CineAssemblyTools::ProductionSettings::Private;
		const int32 HierarchicalBiasValue = (ActiveProduction->SubsequencePriority == ESubsequencePriority::TopDown) ? TopDownHBiasValue : BottomUpHBiasValue;

		// Write the new HBias value to the MovieScene SubSection section of the editor per project config file
		const FString SubSectionClassPath = UMovieSceneSubSection::StaticClass()->GetPathName();
		const FString ParametersPropertyName = GET_MEMBER_NAME_CHECKED(UMovieSceneSubSection, Parameters).ToString();
		const FString HBiasPropertyName = GET_MEMBER_NAME_CHECKED(FMovieSceneSectionParameters, HierarchicalBias).ToString();

		GConfig->SetString(*SubSectionClassPath, *ParametersPropertyName, *FString::Printf(TEXT("(%s=%d)"), *HBiasPropertyName, HierarchicalBiasValue), GEditorPerProjectIni);
		GConfig->Flush(false, GEditorPerProjectIni);

		// Update the CDO for the MovieScene SubSection so that new objects will use the new value 
		UMovieSceneSubSection* SubSectionCDO = GetMutableDefault<UMovieSceneSubSection>();
		SubSectionCDO->ReloadConfig();
	}
}

void UProductionSettings::OverrideDefaultAssetNames()
{
	// Register the active production's default asset names with AssetTools
	if (const FCinematicProduction* ActiveProduction = GetActiveProductionPtr())
	{
		IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
		for (const TPair<TObjectPtr<const UClass>, FString>& Naming : ActiveProduction->DefaultAssetNames)
		{
			if (Naming.Key)
			{
				AssetTools.RegisterDefaultAssetNameForClass(Naming.Key, Naming.Value);
				RegisteredAssetNameClasses.Add(Naming.Key);
			}
		}
	}
}

void UProductionSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UProductionSettings, Productions))
	{
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd)
		{
			// Make sure the production name is unique
			const int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.MemberProperty->GetFName().ToString());
			Productions[ArrayIndex].ProductionName = GetUniqueProductionName();
			OnProductionCreatedDynamic.Broadcast(Productions[ArrayIndex]);
		}
		else
		{
			// It is possible that a production was just deleted. If it was the active production, funnel the change
			// through SetActiveProduction so all overrides (display rate, start time, subsequence priority, asset names)
			// are unregistered consistently.
			const FCinematicProduction* StillActiveProduction = Algo::FindBy(Productions, ActiveProductionName, &FCinematicProduction::ProductionName);
			const FGuid NewActiveProductionID = StillActiveProduction ? StillActiveProduction->ProductionID : FGuid();

			if (NewActiveProductionID != ActiveProductionID)
			{
				SetActiveProduction(NewActiveProductionID);
			}
		}

		ProductionListChangedDelegate.Broadcast();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FCinematicProduction, ProductionName))
	{
		// If the active production's name property just changed, we need to update the serialized ActiveProductionName to match.
		// Note: We don't know if the edited property belonged to the active production, but if it did not, the ActiveProductionName will remain unchanged.
		SetActiveProductionName();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FCinematicProduction, DefaultDisplayRate))
	{
		OverrideDefaultDisplayRate();
		OverrideDefaultStartTime();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FCinematicProduction, DefaultStartFrame))
	{
		OverrideDefaultStartTime();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FCinematicProduction, SubsequencePriority))
	{
		OverrideSubsequenceHierarchicalBias();
	}
}

void UProductionSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* HeadNode = PropertyChangedEvent.PropertyChain.GetHead();
		HeadNode != nullptr
		&& HeadNode->GetValue() != nullptr)
	{
		// Is this a Production change?
		if (HeadNode->GetValue()->GetFName() == GET_MEMBER_NAME_CHECKED(UProductionSettings, Productions))
		{
			HandleProductionsChangeChainProperty(*HeadNode, PropertyChangedEvent);
		}
	}
}

void UProductionSettings::PostReloadConfig(FProperty* PropertyThatWasLoaded)
{
	Super::PostReloadConfig(PropertyThatWasLoaded);
	// Reload all productions extended data
	ReloadProductionsExtendedData();
}

void UProductionSettings::PostEditUndo()
{
	Super::PostEditUndo();
	// Make sure bindings are valid as the undo will recreate struct instances
	BindToProductions();
	ReloadProductionsExtendedData();
}

bool UProductionSettings::IsValidProductionName(const FString& ProductionName) const
{
	return ProductionName.Len() <= ProductionNameMaxLength;
}

void UProductionSettings::AddProduction()
{
	FCinematicProduction NewProduction;
	NewProduction.ProductionName = GetUniqueProductionName();
	AddProduction(NewProduction);
}

void UProductionSettings::AddProduction(const FCinematicProduction& ProductionToAdd)
{
	if (!IsValidProductionName(ProductionToAdd.ProductionName))
	{
		UE_LOGF(LogProductionSettings, Error, "Failed to add Production named %ls because the name is too long. The maximum production name length is %d", *ProductionToAdd.ProductionName, ProductionNameMaxLength);
		return;
	}

	Productions.Add(ProductionToAdd);

	if (Productions.Num() == 1)
	{
		SetActiveProduction(ProductionToAdd.ProductionID);
	}

	UpdateConfig();

	UE::CineAssemblyToolsAnalytics::RecordEvent_CreateProduction();

	ProductionListChangedDelegate.Broadcast();
	OnProductionCreatedDynamic.Broadcast(ProductionToAdd);
}

void UProductionSettings::DuplicateProduction(FGuid ProductionID)
{
	if (const FCinematicProduction* ProductionToCopy = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		FCinematicProduction DuplicateProduction = *ProductionToCopy;

		// Give the duplicate production its own GUID and a unique name
		DuplicateProduction.ProductionID = FGuid::NewGuid();
		DuplicateProduction.ProductionName = GetUniqueProductionName(ProductionToCopy->ProductionName);

		if (!IsValidProductionName(DuplicateProduction.ProductionName))
		{
			UE_LOGF(LogProductionSettings, Error, "Failed to duplicate Production named %ls because the name is too long. The maximum production name length is %d", *ProductionToCopy->ProductionName, ProductionNameMaxLength);
			return;
		}

		Productions.Add(DuplicateProduction);

		UpdateConfig();

		ProductionListChangedDelegate.Broadcast();
		OnProductionCreatedDynamic.Broadcast(DuplicateProduction);
	}
}

void UProductionSettings::UpdateConfig()
{
	// Try to update the default config file. If unsuccessful, try to make the file writable and try again
	bool bUpdateResult = TryUpdateDefaultConfigFile();
	if (!bUpdateResult)
	{
		const FString ConfigFilePath = GetDefaultConfigFilename();

		bool bMakeWritableResult = SettingsHelpers::CheckOutOrAddFile(ConfigFilePath);
		if (!bMakeWritableResult)
		{
			bMakeWritableResult = SettingsHelpers::MakeWritable(ConfigFilePath);
		}

		// Try again to update the config file
		TryUpdateDefaultConfigFile();
	}
}

void UProductionSettings::NotifySettingsChange(const FCinematicProduction& Production)
{
	UpdateConfig();
	OnProductionSettingsChangedDynamic.Broadcast(Production);
}

void UProductionSettings::ReloadProductionsExtendedData()
{
	// For every registered extended data, attempt to load that into each production
	FCineAssemblyToolsEditorModule& CatEdModule = FModuleManager::Get().GetModuleChecked<FCineAssemblyToolsEditorModule>("CineAssemblyToolsEditor");
	const TSet<const UScriptStruct*>& Extensions = CatEdModule.GetProductionExtensions().GetProductionExtensions();

	for (FCinematicProduction& Production : Productions)
	{
		FScopedModifyProductionExtendedData ModifyGuard(Production);
		for (const UScriptStruct* Struct : Extensions)
		{
			if (Struct != nullptr)
			{
				Production.LoadExtendedData(*Struct);
			}
		}
	}
}

void UProductionSettings::BindToProductions()
{
	for (FCinematicProduction& Production : Productions)
	{
		if (!Production.OnExtendedDataExported().IsBoundToObject(this))
		{
			Production.OnExtendedDataExported().AddUObject(this, &UProductionSettings::HandleProductionExtendedDataUpdated);
		}
	}
}

void UProductionSettings::HandleProductionExtendedDataUpdated(const FCinematicProduction& SenderProduction, const UScriptStruct* ForData)
{
	UpdateConfig();
}

void UProductionSettings::HandleProductionsChangeChainProperty(const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode& ProductionsChainNode, FPropertyChangedChainEvent& WithinPropertyChangedEvent)
{
	check(ProductionsChainNode.GetValue() != nullptr);

	const int32 ProductionIndex = WithinPropertyChangedEvent.GetArrayIndex(ProductionsChainNode.GetValue()->GetName());

	// If the index of the modified Production is not valid, return now as we can't handle anything
	if (!Productions.IsValidIndex(ProductionIndex))
	{
		return;
	}

	// Check if this is is an ExtendedData change.
	if (const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* ExtendedDataNode = ProductionsChainNode.GetNextNode();
		ExtendedDataNode != nullptr
		&& ExtendedDataNode->GetValue() != nullptr
		&& ExtendedDataNode->GetValue()->GetFName() == FCinematicProduction::GetExtendedDataMemberName())
	{
		// Get the extended data property node that changed. From here we can try to extract the type of ExtendedData.
		const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* ChangedDataPropertyNode = ExtendedDataNode->GetNextNode();

		const UScriptStruct* ChangedStruct = ChangedDataPropertyNode != nullptr && ChangedDataPropertyNode->GetValue() != nullptr
			? ChangedDataPropertyNode->GetValue()->GetTypedOwner<UScriptStruct>()
			: nullptr;

		if (ChangedStruct != nullptr)
		{
			Productions[ProductionIndex].ExportExtendedData(*ChangedStruct);
		}
		else
		{
			// Something else changed about the extended data but we can't get the exact type, maybe a reset, so fallback to an untargeted update of the production.
			FScopedModifyProductionExtendedData ModifyGuard(Productions[ProductionIndex]);
		}
	}

	OnProductionSettingsChangedDynamic.Broadcast(Productions[ProductionIndex]);
}

void UProductionSettings::DeleteProduction(FGuid ProductionID)
{
	if (!ProductionID.IsValid())
	{
		return;
	}

	const int32 IndexToDelete = Algo::IndexOfBy(Productions, ProductionID, &FCinematicProduction::ProductionID);

	if (IndexToDelete != INDEX_NONE)
	{
		// Determine whether this was the active production before removing it
		const bool bIsActiveProduction = IsActiveProduction(ProductionID);

		Productions.RemoveAt(IndexToDelete);

		if (bIsActiveProduction)
		{
			SetActiveProduction(ProductionID);
		}

		UpdateConfig();

		ProductionListChangedDelegate.Broadcast();
	}
}

void UProductionSettings::RenameProduction(FGuid ProductionID, FString NewName)
{
	if (!IsValidProductionName(NewName))
	{
		UE_LOGF(LogProductionSettings, Error, "Failed to rename Production to %ls because the name is too long. The maximum production name length is %d", *NewName, ProductionNameMaxLength);
		return;
	}

	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		Production->ProductionName = NewName;
		SetActiveProductionName();
		NotifySettingsChange(*Production);
	}
}

const TArray<FCinematicProduction> UProductionSettings::GetProductions() const
{
	return Productions;
}

TOptional<const FCinematicProduction> UProductionSettings::GetProduction(FGuid ProductionID) const
{
	if (const FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		return *Production;
	}
	return TOptional<const FCinematicProduction>();
}

TOptional<const FCinematicProduction> UProductionSettings::GetActiveProduction() const
{
	return GetProduction(ActiveProductionID);
}

FGuid UProductionSettings::GetActiveProductionID() const
{
	return ActiveProductionID;
}

const FCinematicProduction* UProductionSettings::GetActiveProductionPtr() const
{
	if (ActiveProductionID.IsValid())
	{
		return Algo::FindBy(Productions, ActiveProductionID, &FCinematicProduction::ProductionID);
	}
	return nullptr;
}

void UProductionSettings::SetActiveProduction(FGuid ProductionID)
{
	// If the active production is currently None, before setting it and applying the project overrides, cache some project defaults to restore later if the active production is set to None again
	if (!ActiveProductionID.IsValid())
	{
		CacheProjectDefaults();
	}

	// Unregister any default asset names we have registered with AssetTools.
	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
	for (const UClass* Class : RegisteredAssetNameClasses)
	{
		AssetTools.UnregisterDefaultAssetNameForClass(Class);
	}
	RegisteredAssetNameClasses.Empty();

	// Update the active production ID to the input ID, if it matches a production that exists in the production list
	if (Algo::ContainsBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		ActiveProductionID = ProductionID;
	}
	else
	{
		ActiveProductionID = FGuid();
	}

	SetActiveProductionName();

	// Apply the overrides based on the new active production (even if it is None, because some properties will be reset to default)
	ApplyProjectOverrides();

	ActiveProductionChangedDelegate.Broadcast();

	if (const FCinematicProduction* NewActiveProduction = GetActiveProductionPtr())
	{
		OnActiveProductionSetDynamic.Broadcast(*NewActiveProduction, true);
	}
	else
	{
		static FCinematicProduction InvalidProduction;
		InvalidProduction.ProductionID.Invalidate();
		OnActiveProductionSetDynamic.Broadcast(InvalidProduction, false);
	}
}

bool UProductionSettings::IsActiveProduction(FGuid ProductionID) const
{
	if (ActiveProductionID.IsValid() && ProductionID.IsValid())
	{
		return (ProductionID == ActiveProductionID);
	}
	return false;
}

FFrameRate UProductionSettings::GetActiveDisplayRate() const
{
	if (const FCinematicProduction* ActiveProduction = GetActiveProductionPtr())
	{
		return ActiveProduction->DefaultDisplayRate;
	}

	// If there is no active production, return the current level sequence project setting instead
	const ULevelSequenceProjectSettings* const LevelSequenceSettings = GetDefault<ULevelSequenceProjectSettings>();

	FFrameRate DefaultSetting;
	TryParseString(DefaultSetting, *LevelSequenceSettings->DefaultDisplayRate);

	return DefaultSetting;
}

int32 UProductionSettings::GetActiveStartFrame() const
{
	if (const FCinematicProduction* ActiveProduction = GetActiveProductionPtr())
	{
		return ActiveProduction->DefaultStartFrame;
	}

	// If there is no active production, compute the equivalent frame number from the current movie scene tools project setting instead
	const UMovieSceneToolsProjectSettings* const MovieSceneToolsSettings = GetDefault<UMovieSceneToolsProjectSettings>();
	const double StartTimeInSeconds = MovieSceneToolsSettings->DefaultStartTime;

	FFrameRate DefaultFrameRate = GetActiveDisplayRate();
	return DefaultFrameRate.AsFrameNumber(StartTimeInSeconds).Value;
}

ESubsequencePriority UProductionSettings::GetActiveSubsequencePriority() const
{
	using namespace UE::CineAssemblyTools::ProductionSettings::Private;

	if (const FCinematicProduction* ActiveProduction = GetActiveProductionPtr())
	{
		return ActiveProduction->SubsequencePriority;
	}

	// If there is no active production, return the value based on the current movie scene subsection setting
	const UMovieSceneSubSection* const SubSectionCDO = GetDefault<UMovieSceneSubSection>();

	if (SubSectionCDO->Parameters.HierarchicalBias == TopDownHBiasValue)
	{
		return ESubsequencePriority::TopDown;
	}
	return ESubsequencePriority::BottomUp;
}

void UProductionSettings::SetDisplayRate(FGuid ProductionID, FFrameRate DisplayRate)
{
	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		Production->DefaultDisplayRate = DisplayRate;

		if (IsActiveProduction(ProductionID))
		{
			OverrideDefaultDisplayRate();
			OverrideDefaultStartTime();
		}

		NotifySettingsChange(*Production);
	}
}

void UProductionSettings::SetStartFrame(FGuid ProductionID, int32 StartFrame)
{
	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		Production->DefaultStartFrame = StartFrame;

		if (IsActiveProduction(ProductionID))
		{
			OverrideDefaultStartTime();
		}

		NotifySettingsChange(*Production);
	}
}

void UProductionSettings::SetSubsequencePriority(FGuid ProductionID, ESubsequencePriority Priority)
{
	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		Production->SubsequencePriority = Priority;

		if (IsActiveProduction(ProductionID))
		{
			OverrideSubsequenceHierarchicalBias();
		}

		NotifySettingsChange(*Production);
	}
}

void UProductionSettings::AddNamespaceToDenyList(FGuid ProductionID, const FString& Namespace)
{
	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		Production->NamingTokenNamespaceDenyList.Add(Namespace);
		NotifySettingsChange(*Production);
	}
}

void UProductionSettings::RemoveNamespaceFromDenyList(FGuid ProductionID, const FString& Namespace)
{
	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		Production->NamingTokenNamespaceDenyList.Remove(Namespace);
		NotifySettingsChange(*Production);
	}
}

void UProductionSettings::FilterNamingTokenNamespaces(TSet<FString>& Namespaces)
{
	if (const FCinematicProduction* ActiveProduction = GetActiveProductionPtr())
	{
		for (const FString& Namespace : ActiveProduction->NamingTokenNamespaceDenyList)
		{
			Namespaces.Remove(Namespace);
		}
	}
}

void UProductionSettings::AddAssetNaming(FGuid ProductionID, const UClass* AssetClass, const FString& DefaultName)
{
	if (!AssetClass)
	{
		return;
	}

	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		Production->DefaultAssetNames.Add(AssetClass, DefaultName);

		if (IsActiveProduction(ProductionID))
		{
			IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
			AssetTools.RegisterDefaultAssetNameForClass(AssetClass, DefaultName);
			RegisteredAssetNameClasses.Add(AssetClass);
		}

		NotifySettingsChange(*Production);

		UE::CineAssemblyToolsAnalytics::RecordEvent_ProductionAddAssetNaming();
	}
}

void UProductionSettings::RemoveAssetNaming(FGuid ProductionID, const UClass* AssetClass)
{
	if (!AssetClass)
	{
		return;
	}

	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		Production->DefaultAssetNames.Remove(AssetClass);

		if (IsActiveProduction(ProductionID))
		{
			IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
			AssetTools.UnregisterDefaultAssetNameForClass(AssetClass);
			RegisteredAssetNameClasses.Remove(AssetClass);
		}

		NotifySettingsChange(*Production);
	}
}

void UProductionSettings::AddTemplateFolder(FGuid ProductionID, const FString& Path, bool bCreateIfMissing)
{
	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		FFolderTemplate NewTemplate;
		NewTemplate.InternalPath = Path;
		NewTemplate.bCreateIfMissing = bCreateIfMissing;

		Production->TemplateFolders.Add(MoveTemp(NewTemplate));
		NotifySettingsChange(*Production);
	}
}

void UProductionSettings::RemoveTemplateFolder(FGuid ProductionID, const FString& Path)
{
	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		const int32 IndexToDelete = Algo::IndexOfBy(Production->TemplateFolders, Path, &FFolderTemplate::InternalPath);
		if (Production->TemplateFolders.IsValidIndex(IndexToDelete))
		{
			Production->TemplateFolders.RemoveAt(IndexToDelete);
		}

		NotifySettingsChange(*Production);
	}
}

void UProductionSettings::SetTemplateFolderHierarchy(FGuid ProductionID, const TArray<FFolderTemplate>& TemplateHierarchy)
{
	if (FCinematicProduction* Production = Algo::FindBy(Productions, ProductionID, &FCinematicProduction::ProductionID))
	{
		Production->TemplateFolders = TemplateHierarchy;

		NotifySettingsChange(*Production);
	}
}

void UProductionSettings::SetActiveDisplayRate(FFrameRate DisplayRate)
{
	if (ActiveProductionID.IsValid())
	{
		SetDisplayRate(ActiveProductionID, DisplayRate);
	}
}

void UProductionSettings::SetActiveStartFrame(int32 StartFrame)
{
	if (ActiveProductionID.IsValid())
	{
		SetStartFrame(ActiveProductionID, StartFrame);
	}
}

void UProductionSettings::SetActiveSubsequencePriority(ESubsequencePriority Priority)
{
	if (ActiveProductionID.IsValid())
	{
		SetSubsequencePriority(ActiveProductionID, Priority);
	}
}

void UProductionSettings::SetActiveProductionName()
{
	if (const FCinematicProduction* ActiveProduction = GetActiveProductionPtr())
	{
		ActiveProductionName = ActiveProduction->ProductionName;
	}
	else
	{
		ActiveProductionName = TEXT("");
	}

	const FString ConfigSection = UProductionSettings::StaticClass()->GetPathName();
	const FString PropertyName = GET_MEMBER_NAME_CHECKED(UProductionSettings, ActiveProductionName).ToString();

	GConfig->SetString(*ConfigSection, *PropertyName, *ActiveProductionName, GEditorPerProjectIni);
	GConfig->Flush(false, GEditorPerProjectIni);
}

FString UProductionSettings::GetUniqueProductionName() const
{
	const FText BaseName = LOCTEXT("NewProductionBaseName", "NewProduction");
	return GetUniqueProductionName(BaseName.ToString());
}

FString UProductionSettings::GetUniqueProductionName(const FString& BaseName) const
{
	// This implementation is based on a similar utility in AssetTools for creating a unique asset name.
	// If the input BaseName does not end in a numeric character, then a 1 will be appended to it.
	// If it already ends in a numeric character, then that number will get incremented.
	// Ex: NewProduction -> NewProduction1
	// Ex: NewProduction1 -> NewProduction2

	// Find the index in the string of the last non-numeric character
	int32 CharIndex = BaseName.Len() - 1;
	while (CharIndex >= 0 && BaseName[CharIndex] >= TEXT('0') && BaseName[CharIndex] <= TEXT('9'))
	{
		--CharIndex;
	}

	// Trim the numeric characters off the end of the BaseName string, but remember the integer that was trimmed off to increment and append to the output
	int32 IntSuffix = 1;
	FString TrimmedBaseName = BaseName;
	if (CharIndex >= 0 && CharIndex < BaseName.Len() - 1)
	{
		TrimmedBaseName = BaseName.Left(CharIndex + 1);

		const FString TrailingInteger = BaseName.RightChop(CharIndex + 1);
		IntSuffix = FCString::Atoi(*TrailingInteger) + 1;
	}

	FString WorkingName = TrimmedBaseName;

	while (Algo::ContainsBy(Productions, WorkingName, &FCinematicProduction::ProductionName))
	{
		WorkingName = FString::Printf(TEXT("%s%d"), *TrimmedBaseName, IntSuffix);
		IntSuffix++;
	}

	return WorkingName;
}

#undef LOCTEXT_NAMESPACE
