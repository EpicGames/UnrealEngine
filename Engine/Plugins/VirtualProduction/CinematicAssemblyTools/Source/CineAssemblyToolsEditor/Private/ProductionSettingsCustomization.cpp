// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProductionSettingsCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "ProductionSettings.h"
#include "Widgets/Input/SComboButton.h"

TSharedRef<IDetailCustomization> FProductionSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FProductionSettingsCustomization);
}

void FProductionSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	DetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);

	PropertyUtilities = DetailBuilder.GetPropertyUtilities();

	// Ensure that we are only customizing one object
	if (CustomizedObjects.Num() != 1)
	{
		return;
	}

	// Ensure that we are only customizing one object
	UProductionSettings* CustomizedProductionSettings = Cast<UProductionSettings>(CustomizedObjects[0]);
	if (CustomizedProductionSettings != GetDefault<UProductionSettings>())
	{
		return;
	}

	// Hide the default active production property, and replace with a combo button listing the available productions in this project
 	TSharedRef<IPropertyHandle> ActiveProductionNameHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UProductionSettings, ActiveProductionName));
 	DetailBuilder.HideProperty(ActiveProductionNameHandle);

 	DetailBuilder.AddCustomRowToCategory(ActiveProductionNameHandle, ActiveProductionNameHandle->GetPropertyDisplayName())
		.NameContent()
		[
			ActiveProductionNameHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SComboButton)
				.VAlign(VAlign_Center)
				.OnGetMenuContent(this, &FProductionSettingsCustomization::BuildProductionNameMenu)
				.ButtonContent()
				[
					SNew(STextBlock).Text(this, &FProductionSettingsCustomization::GetActiveProductionName)
						.Font(IDetailLayoutBuilder::GetDetailFont())
				]
		];
}

TSharedRef<SWidget> FProductionSettingsCustomization::BuildProductionNameMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	// Always add a "None" option to set the active production to None
	FName NoActiveProductionName = NAME_None;
	MenuBuilder.AddMenuEntry(
		FText::FromName(NoActiveProductionName),
		FText::FromName(NoActiveProductionName),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateRaw(this, &FProductionSettingsCustomization::SetActiveProduction, FGuid())),
		NAME_None,
		EUserInterfaceActionType::None
	);

	const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();

	// Add a menu option with the production name for each production available in this project
	const TArray<FCinematicProduction> Productions = ProductionSettings->GetProductions();
	for (const FCinematicProduction& Production : Productions)
	{
		MenuBuilder.AddMenuEntry(
			FText::FromString(Production.ProductionName),
			FText::FromString(Production.ProductionName),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateRaw(this, &FProductionSettingsCustomization::SetActiveProduction, Production.ProductionID)),
			NAME_None,
			EUserInterfaceActionType::None
		);
	}

	return MenuBuilder.MakeWidget();
}

FText FProductionSettingsCustomization::GetActiveProductionName() const
{
	const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();

	TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
	const FString ActiveProductionName = ActiveProduction.IsSet() ? ActiveProduction.GetValue().ProductionName : FString();

	if (ActiveProductionName.IsEmpty())
	{
		return FText::FromName(NAME_None);
	}
	return FText::FromString(ActiveProductionName);
}

void FProductionSettingsCustomization::SetActiveProduction(FGuid ProductionID)
{
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ProductionSettings->SetActiveProduction(ProductionID);

	// The active production influences whether certain sequencer settings are writable or read-only.
	// This forces the details view of the other sequencer settings categories to refresh immediately to respect the new property flags.
	PropertyUtilities->RequestForceRefresh();
}
