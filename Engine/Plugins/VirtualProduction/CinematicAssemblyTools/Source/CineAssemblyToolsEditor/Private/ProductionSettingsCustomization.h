// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

#include "IPropertyUtilities.h"

class IDetailLayoutBuilder;
class SWidget;

/**
 * Detail customization for UProductionSettings
 */
class FProductionSettingsCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface */

private:
	/** Set the active production of the production project settings */
	void SetActiveProduction(FGuid ProductionID);

	/** Get the name of the currently active production */
	FText GetActiveProductionName() const;

	/** Build the menu widget for the production name combo button */
	TSharedRef<SWidget> BuildProductionNameMenu();

	/** Property Utils, used to refresh the property settings details when changing the active production */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};
