// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScopedModifyProductionExtendedData.h"

#include "CineAssemblyToolsEditorModule.h"
#include "ProductionSettings.h"

FScopedModifyProductionExtendedData::FScopedModifyProductionExtendedData(FCinematicProduction& InProduction) :
	TargetProduction(InProduction)
{
	Start();
}

FScopedModifyProductionExtendedData::FScopedModifyProductionExtendedData(FCinematicProduction& InProduction, const UScriptStruct& InTargetStruct) :
	TargetProduction(InProduction), TargetStruct(&InTargetStruct)
{
	Start();
}

FScopedModifyProductionExtendedData::~FScopedModifyProductionExtendedData()
{
	Finish();
}

bool FScopedModifyProductionExtendedData::IsActive() const
{
	if (bHasFinished)
	{
		return false;
	}

	if (!TargetStruct.IsSet())
	{
		return TargetProduction.IsValid();
	}
	return TargetProduction.IsValid() && TargetProduction->FindExtendedData(*TargetStruct.GetValue());
}

void FScopedModifyProductionExtendedData::Finish()
{
	const UScriptStruct* ExportedStruct = nullptr;
	bool bExported = false;

	if (IsActive())
	{
		if (TargetStruct.IsSet())
		{
			ExportedStruct = TargetStruct.GetValue();
			TargetProduction->ExportExtendedData(*ExportedStruct);
		}
		else
		{
			FCineAssemblyToolsEditorModule& CatEdModule = FModuleManager::Get().GetModuleChecked<FCineAssemblyToolsEditorModule>("CineAssemblyToolsEditor");
			const TSet<const UScriptStruct*>& Extensions = CatEdModule.GetProductionExtensions().GetProductionExtensions();
			for (const UScriptStruct* Struct : Extensions)
			{
				if (Struct != nullptr)
				{
					TargetProduction->ExportExtendedData(*Struct);
				}
			}
		}
		bExported = true;

		// Reset the block flag
		TargetProduction->bBlockNotifyExtendedDataExported = bOriginalBlockValue;

		if (bExported)
		{
			TargetProduction->NotifyExtendedDataExported(ExportedStruct);
		}
	}

	bHasFinished = true;
}

void FScopedModifyProductionExtendedData::Start()
{
	bHasFinished = false;
	if (TargetProduction.IsValid())
	{
		bOriginalBlockValue = TargetProduction->bBlockNotifyExtendedDataExported;
		TargetProduction->bBlockNotifyExtendedDataExported = true;
	}
}
