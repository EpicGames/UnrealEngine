// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/CineAssemblySchemaSequencerCustomization.h"

#include "CineAssembly.h"
#include "CineAssemblySchema.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "Tracks/MovieSceneSubTrack.h"

DEFINE_LOG_CATEGORY_STATIC(LogCineAssemblySchemaSequencerCustomization, Log, All);

void FCineAssemblySchemaSequencerCustomization::RegisterSequencerCustomization(FSequencerCustomizationBuilder& Builder)
{
	// Only customize CineAssemblies that are schema templates
	UCineAssembly* Assembly = Cast<UCineAssembly>(&Builder.GetFocusedSequence());
	if (!Assembly || !Assembly->GetTypedOuter<UCineAssemblySchema>())
	{
		return;
	}

	FSequencerCustomizationInfo CustomizationInfo;
	CustomizationInfo.OnPaste.BindRaw(this, &FCineAssemblySchemaSequencerCustomization::OnPaste);

	Builder.AddCustomization(CustomizationInfo);
}

ESequencerPasteSupport FCineAssemblySchemaSequencerCustomization::OnPaste()
{
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	// The following track types are unsupported. If the clipboard contains any of these, we want to block pasting to avoid adding incompatible track types.
	if (ClipboardText.Contains(UMovieSceneEventTrack::StaticClass()->GetPathName()) ||
		ClipboardText.Contains(UMovieSceneSubTrack::StaticClass()->GetPathName()) ||
		ClipboardText.Contains(UMovieSceneCinematicShotTrack::StaticClass()->GetPathName()))
	{
		UE_LOGF(LogCineAssemblySchemaSequencerCustomization, Warning, "Paste tracks failed because one or more of the copied tracks are not supported in the Schema Template Sequence.");
		return ESequencerPasteSupport::All & ~ESequencerPasteSupport::Tracks;
	}

	return ESequencerPasteSupport::All;
}
