// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerCustomizationManager.h"

/**
 * Sequencer customization applied when a UCineAssembly is the focused sequence in a Schema window
 */
class FCineAssemblySchemaSequencerCustomization : public ISequencerCustomization
{
public:
	virtual void RegisterSequencerCustomization(FSequencerCustomizationBuilder& Builder) override;
	virtual void UnregisterSequencerCustomization() override {}

private:
	ESequencerPasteSupport OnPaste();
};
