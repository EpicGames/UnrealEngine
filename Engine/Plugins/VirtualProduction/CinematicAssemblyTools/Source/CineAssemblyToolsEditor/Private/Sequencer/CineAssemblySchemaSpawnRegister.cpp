// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblySchemaSpawnRegister.h"
#include "ILevelSequenceModule.h"
#include "Modules/ModuleManager.h"

FCineAssemblySchemaSpawnRegister::FCineAssemblySchemaSpawnRegister()
{
	// Generate a set of object spawners that will be used to set up the defaults for supported spawnable types (primarily actors)
	ILevelSequenceModule& LevelSequenceModule = FModuleManager::GetModuleChecked<ILevelSequenceModule>("LevelSequence");
	LevelSequenceModule.GenerateObjectSpawners(MovieSceneObjectSpawners);
}

UObject* FCineAssemblySchemaSpawnRegister::SpawnObject(const FGuid& BindingId, UMovieScene& MovieScene, FMovieSceneSequenceIDRef Template, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex)
{
	// Spawnables in a schema template sequence never actually get spawned in the world
	return nullptr;
}

UObject* FCineAssemblySchemaSpawnRegister::SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	// Spawnables in a schema template sequence never actually get spawned in the world
	return nullptr;
}

void FCineAssemblySchemaSpawnRegister::DestroySpawnedObject(UObject& Object, UMovieSceneSpawnableBindingBase* CustomSpawnableBinding)
{
	// Nothing was ever spawned, so there is nothing to destroy
}

#if WITH_EDITOR

bool FCineAssemblySchemaSpawnRegister::CanSpawnObject(UClass* InClass) const
{
	for (TSharedRef<IMovieSceneObjectSpawner> MovieSceneObjectSpawner : MovieSceneObjectSpawners)
	{
		if (InClass->IsChildOf(MovieSceneObjectSpawner->GetSupportedTemplateType()))
		{
			return true;
		}
	}
	return false;
}

void FCineAssemblySchemaSpawnRegister::SetupDefaultsForSpawnable(UObject* SpawnedObject, const FGuid& Guid, const TOptional<FTransformData>& TransformData, TSharedRef<ISequencer> Sequencer, USequencerSettings* Settings)
{
	for (TSharedPtr<IMovieSceneObjectSpawner> MovieSceneObjectSpawner : MovieSceneObjectSpawners)
	{
		if (MovieSceneObjectSpawner->CanSetupDefaultsForSpawnable(SpawnedObject))
		{
			MovieSceneObjectSpawner->SetupDefaultsForSpawnable(SpawnedObject, Guid, TransformData, Sequencer, Settings);
			return;
		}
	}
}

#endif // WITH_EDITOR
