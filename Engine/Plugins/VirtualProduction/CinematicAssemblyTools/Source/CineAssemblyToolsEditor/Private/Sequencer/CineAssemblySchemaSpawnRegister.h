// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSpawnRegister.h"

/** Movie scene spawn register that handles spawning objects in a Cine Assembly Schema's template sequence */
class FCineAssemblySchemaSpawnRegister : public FMovieSceneSpawnRegister
{
public:
	FCineAssemblySchemaSpawnRegister();
	~FCineAssemblySchemaSpawnRegister() = default;

protected:
	//~ Begin FMovieSceneSpawnRegister interface
	virtual UObject* SpawnObject(const FGuid& BindingId, UMovieScene& MovieScene, FMovieSceneSequenceIDRef Template, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex) override;
	virtual UObject* SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) override;

	virtual void DestroySpawnedObject(UObject& Object, UMovieSceneSpawnableBindingBase* CustomSpawnableBinding) override;

#if WITH_EDITOR
	virtual bool CanSpawnObject(UClass* InClass) const override;
	virtual void SetupDefaultsForSpawnable(UObject* SpawnedObject, const FGuid& Guid, const TOptional<FTransformData>& TransformData, TSharedRef<ISequencer> Sequencer, USequencerSettings* Settings) override;
#endif
	//~ End FMovieSceneSpawnRegister interface

private:
	TArray<TSharedRef<IMovieSceneObjectSpawner>> MovieSceneObjectSpawners;
};
