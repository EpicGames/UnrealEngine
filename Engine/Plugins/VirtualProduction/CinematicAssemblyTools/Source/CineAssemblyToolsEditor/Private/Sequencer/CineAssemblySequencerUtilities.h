// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ISequencer;
class UMovieSceneSequence;
class UMovieSceneSubSection;

/** Utility functions for Sequencer and MovieScene operations used by the Cine Assembly Tools */
struct FCineAssemblySequencerUtilities
{
public:
	/** Create a new spawnable camera binding without actually spawning a camera */
	static void CreateCamera(TSharedRef<ISequencer> Sequencer);

	/**
	 * Replace the Sequence in the input SubSection with the input Sequence
	 * If needed, the SubSection will be first unlocked to ensure the assignment is succesful, and then its original lock status is restored.
	 */
	static void ReplaceSubSequence(UMovieSceneSubSection* InSubSection, UMovieSceneSequence* InSequence);

	/**
	 * Copy every root track and binding from InSrcSequence's MovieScene into InDstSequence's MovieScene.
	 * Tracks are deep-duplicated and Bindings are recreated with fresh GUIDs.
	 * Any relevant binding references are reassigned to the new GUIDs in the InDstSequence.
	 */
	static void DuplicateMovieSceneContents(UMovieSceneSequence* InSrcSequence, UMovieSceneSequence* InDstSequence);
};
