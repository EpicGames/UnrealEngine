// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSubAssemblySectionCustomization.h"

#include "CineAssembly.h"
#include "CineAssemblyNamingTokens.h"
#include "CineAssemblySchema.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "LevelSequence.h"
#include "MovieSceneSubAssemblySection.h"
#include "PropertyCustomizationHelpers.h"
#include "SNamingTokensEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SubAssemblySectionDetailCustomization"

namespace UE::CineAssemblyTools::Private
{
	/** Applies an override value for the given key, or removes the entry if the new value matches the default (keeps the map sparse) */
	void ApplyOverride(UMovieSceneSubAssemblySection* Section, const FString& Key, const FString& NewStringValue, const FString& DefaultStringValue)
	{
		Section->Modify();
		if (NewStringValue == DefaultStringValue)
		{
			Section->MetadataOverrides.Remove(Key);
		}
		else
		{
			Section->MetadataOverrides.Add(Key, NewStringValue);
		}
	}

	/** Returns a widget for editing a String-typed metadata override (with or without token evaluation) */
	TSharedRef<SWidget> MakeStringValueWidget(UMovieSceneSubAssemblySection* Section, const FAssemblyMetadataDesc& MetadataDesc)
	{
		if (!MetadataDesc.bEvaluateTokens)
		{
			return SNew(SEditableTextBox)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text_Lambda([Section, MetadataDesc]()
					{
						const FString DefaultValue = MetadataDesc.DefaultValue.IsType<FString>() ? MetadataDesc.DefaultValue.Get<FString>() : FString();
						if (const FString* Override = Section->MetadataOverrides.Find(MetadataDesc.Key))
						{
							return FText::FromString(*Override);
						}
						return FText::FromString(DefaultValue);
					})
				.OnTextCommitted_Lambda([Section, MetadataDesc](const FText& InText, ETextCommit::Type)
					{
						const FString DefaultValue = MetadataDesc.DefaultValue.IsType<FString>() ? MetadataDesc.DefaultValue.Get<FString>() : FString();
						ApplyOverride(Section, MetadataDesc.Key, InText.ToString(), DefaultValue);
					});
		}
		else
		{
			FNamingTokenFilterArgs FilterArgs;
			FilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

			return SNew(SNamingTokensEditableTextBox)
				.ShouldEvaluateTokens(false)
				.FilterArgs(FilterArgs)
				.Text_Lambda([Section, MetadataDesc]()
					{
						const FString DefaultValue = MetadataDesc.DefaultValue.IsType<FString>() ? MetadataDesc.DefaultValue.Get<FString>() : FString();
						if (const FString* Override = Section->MetadataOverrides.Find(MetadataDesc.Key))
						{
							return FText::FromString(*Override);
						}
						return FText::FromString(DefaultValue);
					})
				.OnTextCommitted_Lambda([Section, MetadataDesc](const FText& InText, ETextCommit::Type)
					{
						const FString DefaultValue = MetadataDesc.DefaultValue.IsType<FString>() ? MetadataDesc.DefaultValue.Get<FString>() : FString();
						ApplyOverride(Section, MetadataDesc.Key, InText.ToString(), DefaultValue);
					});
		}
	}
}

TSharedRef<IDetailCustomization> FSubAssemblySectionDetailCustomization::MakeInstance()
{
	return MakeShared<FSubAssemblySectionDetailCustomization>();
}

void FSubAssemblySectionDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<>> CustomizedObjects;
	DetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);
	if (CustomizedObjects.Num() != 1)
	{
		return;
	}

	UMovieSceneSubAssemblySection* SubAssemblySection = Cast<UMovieSceneSubAssemblySection>(CustomizedObjects[0]);
	if (!SubAssemblySection)
	{
		return;
	}

	// Sort the detail categories to maintain the expected order and allow Metadata to appear after the Sequence details
	constexpr int32 SectionSortOrder = 1000;
	constexpr int32 SequenceSortOrder = 1001;
	constexpr int32 MetadataSortOrder = 1002;

	IDetailCategoryBuilder& SectionCategory = DetailBuilder.EditCategory("Section");
	IDetailCategoryBuilder& SequenceCategory = DetailBuilder.EditCategory("Sequence");

	SectionCategory.SetSortOrder(SectionSortOrder);
	SequenceCategory.SetSortOrder(SequenceSortOrder);

	// For template sections, add a category of metadata overrides based on the referenced schema's AssemblyMetadata.
	if (SubAssemblySection->IsTemplateSection())
	{
		IDetailCategoryBuilder& MetadataCategory = DetailBuilder.EditCategory("MetadataOverrides", LOCTEXT("MetadataOverridesCategory", "Metadata Overrides"));
		MetadataCategory.SetSortOrder(MetadataSortOrder);

		if (const UCineAssemblySchema* Schema = SubAssemblySection->GetTemplateSchema())
		{
			for (const FAssemblyMetadataDesc& MetadataDesc : Schema->AssemblyMetadata)
			{
				AddMetadataOverrideRow(MetadataCategory, SubAssemblySection, MetadataDesc);
			}
		}
	}

	// Force the Label property to appear first in the Sequence category
	TSharedRef<IPropertyHandle> LabelPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieSceneSubAssemblySection, Label));
	SequenceCategory.AddProperty(LabelPropertyHandle);

	// Add the SectionType property to the Sequence Category, but disable it when the Section's Assembly Template is not a valid Sequence.
	// This could mean that the Assembly Template is null, or possibly a Schema. 
	// In either of these cases, it does not make sense for the user to change the SectionType from Template to Reference, because Reference Sections must refer to a valid Sequence.
	TSharedRef<IPropertyHandle> SectionTypePropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieSceneSubAssemblySection, SectionType));
	IDetailPropertyRow& SectionTypePropertyRow = SequenceCategory.AddProperty(SectionTypePropertyHandle);
	SectionTypePropertyRow.IsEnabled(TAttribute<bool>::CreateLambda([SubAssemblySection]()
		{
			UMovieSceneSequence* Sequence = Cast<UMovieSceneSequence>(SubAssemblySection->GetAssemblyTemplate());
			return (SubAssemblySection->SectionType == ESubAssemblySectionType::Reference) || (Sequence != nullptr) ? true : false;
		}));


	// Add the SubSequence property to the Sequence Category, but hide it when the SectionType is Template, because the Assembly Template property will be displayed instead
	TSharedRef<IPropertyHandle> SubSequencePropertyHandle = DetailBuilder.GetProperty(FName(TEXT("SubSequence")), UMovieSceneSubSection::StaticClass());
	IDetailPropertyRow& SubSequencePropertyRow = SequenceCategory.AddProperty(SubSequencePropertyHandle);
	SubSequencePropertyRow.Visibility(TAttribute<EVisibility>::CreateLambda([SubAssemblySection]()
		{ 
			return (SubAssemblySection->SectionType != ESubAssemblySectionType::Template) ? EVisibility::Visible : EVisibility::Collapsed; 
		}));

	// Add the AssemblyTemplate property to the Sequence Category, but hide it when the SectionType is Reference, because the SubSequence property will be displayed instead
	TSharedRef<IPropertyHandle> AssemblyTemplatePropertyHandle = DetailBuilder.GetProperty(UMovieSceneSubAssemblySection::AssemblyTemplatePropertyName);
	IDetailPropertyRow& AssemblyTemplatePropertyRow = SequenceCategory.AddProperty(AssemblyTemplatePropertyHandle);
	AssemblyTemplatePropertyRow.Visibility(TAttribute<EVisibility>::CreateLambda([SubAssemblySection]()
		{
			return (SubAssemblySection->SectionType == ESubAssemblySectionType::Template) ? EVisibility::Visible : EVisibility::Collapsed;
		}));

	const UCineAssemblySchema* OwningSchema = SubAssemblySection ? SubAssemblySection->GetTypedOuter<UCineAssemblySchema>() : nullptr;

	// Further customize the AssemblyTemplate property so that the asset picker widget only allows Level Sequences, Cine Assemblies, and Schemas
	AssemblyTemplatePropertyRow.CustomWidget()
		.NameContent()
		[
			AssemblyTemplatePropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SObjectPropertyEntryBox)
				.AllowedClass(UObject::StaticClass())
				.ThumbnailPool(DetailBuilder.GetThumbnailPool())
				.AllowCreate(true)
				.OnShouldFilterAsset_Lambda([this, OwningSchema](const FAssetData& InAssetData)
					{
						return OnShouldFilterAsset(InAssetData, OwningSchema);
					})
				.ObjectPath_Lambda([SubAssemblySection]()
					{			
						return (SubAssemblySection->GetAssemblyTemplate()) ? SubAssemblySection->GetAssemblyTemplate()->GetPathName() : TEXT("");
					})
				.OnObjectChanged_Lambda([SubAssemblySection](const FAssetData& InAssetData)
					{
						SubAssemblySection->Modify();
						SubAssemblySection->SetAssemblyTemplate(InAssetData.GetAsset());
					})
		];
}

bool FSubAssemblySectionDetailCustomization::OnShouldFilterAsset(const FAssetData& InAssetData, const UCineAssemblySchema* OwningSchema)
{
	// Filter out assets that aren't a supported sequence or schema type
	if ((InAssetData.GetClass() != ULevelSequence::StaticClass())
		&& (InAssetData.GetClass() != UCineAssembly::StaticClass())
		&& (InAssetData.GetClass() != UCineAssemblySchema::StaticClass()))
	{
		return true;
	}

	// Filter out the Schema that owns this section; selecting it as the template would cause infinite recursion at assembly creation time
	if (OwningSchema && InAssetData.GetSoftObjectPath() == FSoftObjectPath(OwningSchema))
	{
		return true;
	}

	return false;
}

void FSubAssemblySectionDetailCustomization::AddMetadataOverrideRow(IDetailCategoryBuilder& Category, UMovieSceneSubAssemblySection* Section, const FAssemblyMetadataDesc& MetadataDesc)
{
	using namespace UE::CineAssemblyTools::Private;

	if (MetadataDesc.Key.IsEmpty())
	{
		return;
	}

	TSharedRef<SWidget> ValueWidget = SNullWidget::NullWidget;

	switch (MetadataDesc.Type)
	{
		case ECineAssemblyMetadataType::String:
			ValueWidget = MakeStringValueWidget(Section, MetadataDesc);
			break;
		case ECineAssemblyMetadataType::Bool:
			ValueWidget = SNew(SCheckBox)
				.IsChecked_Lambda([Section, MetadataDesc]()
					{
						const bool DefaultValue = MetadataDesc.DefaultValue.IsType<bool>() ? MetadataDesc.DefaultValue.Get<bool>() : false;
						const bool bValue = Section->MetadataOverrides.Contains(MetadataDesc.Key) ? Section->MetadataOverrides[MetadataDesc.Key].ToBool() : DefaultValue;
						return bValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([Section, MetadataDesc](ECheckBoxState CheckBoxState)
					{
						const bool DefaultValue = MetadataDesc.DefaultValue.IsType<bool>() ? MetadataDesc.DefaultValue.Get<bool>() : false;
						const bool bNewValue = (CheckBoxState == ECheckBoxState::Checked);
						ApplyOverride(Section, MetadataDesc.Key, bNewValue ? TEXT("true") : TEXT("false"), DefaultValue ? TEXT("true") : TEXT("false"));
					});
			break;
		case ECineAssemblyMetadataType::Integer:
			ValueWidget = SNew(SNumericEntryBox<int32>)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Value_Lambda([Section, MetadataDesc]()
					{
						const int32 DefaultValue = MetadataDesc.DefaultValue.IsType<int32>() ? MetadataDesc.DefaultValue.Get<int32>() : 0;
						if (const FString* Override = Section->MetadataOverrides.Find(MetadataDesc.Key))
						{
							return FCString::Atoi(**Override);
						}
						return DefaultValue;
					})
				.OnValueCommitted_Lambda([Section, MetadataDesc](int32 InValue, ETextCommit::Type)
					{
						const int32 DefaultValue = MetadataDesc.DefaultValue.IsType<int32>() ? MetadataDesc.DefaultValue.Get<int32>() : 0;
						ApplyOverride(Section, MetadataDesc.Key, LexToString(InValue), LexToString(DefaultValue));
					});
			break;
		case ECineAssemblyMetadataType::Float:
			ValueWidget = SNew(SNumericEntryBox<float>)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Value_Lambda([Section, MetadataDesc]()
					{
						const float DefaultValue = MetadataDesc.DefaultValue.IsType<float>() ? MetadataDesc.DefaultValue.Get<float>() : 0.0f;
						if (const FString* Override = Section->MetadataOverrides.Find(MetadataDesc.Key))
						{
							return FCString::Atof(**Override);
						}
						return DefaultValue;
					})
				.OnValueCommitted_Lambda([Section, MetadataDesc](float InValue, ETextCommit::Type)
					{
						const float DefaultValue = MetadataDesc.DefaultValue.IsType<float>() ? MetadataDesc.DefaultValue.Get<float>() : 0.0f;
						ApplyOverride(Section, MetadataDesc.Key, LexToString(InValue), LexToString(DefaultValue));
					});
			break;
		case ECineAssemblyMetadataType::AssetPath:
			ValueWidget = SNew(SObjectPropertyEntryBox)
				.AllowedClass(!MetadataDesc.AssetClass.IsNull() ? MetadataDesc.AssetClass.ResolveClass() : UObject::StaticClass())
				.AllowCreate(false)
				.ObjectPath_Lambda([Section, MetadataDesc]()
					{
						const FString DefaultValue = MetadataDesc.DefaultValue.IsType<FString>() ? MetadataDesc.DefaultValue.Get<FString>() : FString();
						if (const FString* Override = Section->MetadataOverrides.Find(MetadataDesc.Key))
						{
							return *Override;
						}
						return DefaultValue;
					})
				.OnObjectChanged_Lambda([Section, MetadataDesc](const FAssetData& InAssetData)
					{
						const FString DefaultValue = MetadataDesc.DefaultValue.IsType<FString>() ? MetadataDesc.DefaultValue.Get<FString>() : FString();
						ApplyOverride(Section, MetadataDesc.Key, InAssetData.GetObjectPathString(), DefaultValue);
					});
			break;
		case ECineAssemblyMetadataType::CineAssembly:
			ValueWidget = SNew(SObjectPropertyEntryBox)
				.AllowedClass(UCineAssembly::StaticClass())
				.AllowCreate(false)
				.OnShouldFilterAsset_Lambda([SchemaType = MetadataDesc.SchemaType](const FAssetData& InAssetData)
					{
						if (!SchemaType.IsValid())
						{
							return false;
						}
						const FAssetDataTagMapSharedView::FFindTagResult AssemblyType = InAssetData.TagsAndValues.FindTag(UCineAssembly::AssetRegistryTag_AssemblyType);
						if (AssemblyType.IsSet())
						{
							return !AssemblyType.GetValue().Equals(SchemaType.GetAssetName());
						}
						return true;
					})
				.ObjectPath_Lambda([Section, MetadataDesc]()
					{
						const FString DefaultValue = MetadataDesc.DefaultValue.IsType<FString>() ? MetadataDesc.DefaultValue.Get<FString>() : FString();
						if (const FString* Override = Section->MetadataOverrides.Find(MetadataDesc.Key))
						{
							return *Override;
						}
						return DefaultValue;
					})
				.OnObjectChanged_Lambda([Section, MetadataDesc](const FAssetData& InAssetData)
					{
						const FString DefaultValue = MetadataDesc.DefaultValue.IsType<FString>() ? MetadataDesc.DefaultValue.Get<FString>() : FString();
						ApplyOverride(Section, MetadataDesc.Key, InAssetData.GetObjectPathString(), DefaultValue);
					});
			break;
		default:
			checkNoEntry();
	}

	Category.AddCustomRow(FText::FromString(MetadataDesc.Key))
		.NameContent()
		[
			SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text(FText::FromString(MetadataDesc.Key))
		]
		.ValueContent()
		[
			ValueWidget
		];
}

#undef LOCTEXT_NAMESPACE
