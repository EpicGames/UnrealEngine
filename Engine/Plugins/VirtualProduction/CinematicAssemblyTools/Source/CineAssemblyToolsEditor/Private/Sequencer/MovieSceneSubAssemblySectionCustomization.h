// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IDetailCategoryBuilder;
class UMovieSceneSubAssemblySection;
class UCineAssemblySchema;
struct FAssemblyMetadataDesc;
struct FAssetData;

/** Detail Customization for UMovieSceneSubAssemblySection */
class FSubAssemblySectionDetailCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this customization for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface */

private:
	/** Determines whether the input asset should be filtered out of the Assembly Template asset picker */
	bool OnShouldFilterAsset(const FAssetData& InAssetData, const UCineAssemblySchema* OwningSchema);

	/** Adds a row to the input category to display a metadata override for the input metadata descriptor */
	void AddMetadataOverrideRow(IDetailCategoryBuilder& Category, UMovieSceneSubAssemblySection* Section, const FAssemblyMetadataDesc& MetadataDesc);
};
