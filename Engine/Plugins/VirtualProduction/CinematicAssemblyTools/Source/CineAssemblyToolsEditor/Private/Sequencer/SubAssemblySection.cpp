// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubAssemblySection.h"

#include "CineAssembly.h"
#include "CineAssemblyNamingTokens.h"
#include "IDetailsView.h"
#include "MovieSceneSubAssemblySectionCustomization.h"

#define LOCTEXT_NAMESPACE "CineAssemblySchemaSubSection"

FSubAssemblySection::FSubAssemblySection(TSharedPtr<ISequencer> InSequencer, UMovieSceneSubAssemblySection* InSection)
	: TSubSectionMixin(InSequencer, *CastChecked<UMovieSceneSubSection>(InSection))
	, MovieSceneSubAssemblySection(InSection)
{
}

UMovieSceneSection* FSubAssemblySection::GetSectionObject()
{
	return MovieSceneSubAssemblySection;
}

FText FSubAssemblySection::GetSectionTitle() const
{
	// The section text is handled by the Section Widget
	return FText::GetEmpty();
}

float FSubAssemblySection::GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const
{
	static constexpr float ShotSectionHeight = 98.0f;
	static constexpr float SubsequenceSectionHeight = 50.0f;

	return MovieSceneSubAssemblySection->IsCinematicShotSection() ? ShotSectionHeight : SubsequenceSectionHeight;
}

TSharedRef<SWidget> FSubAssemblySection::GenerateSectionWidget()
{
	const FMargin WarningIconPadding = (MovieSceneSubAssemblySection->IsCinematicShotSection()) ? FMargin(8.0f, 6.0f, 4.0f, 0.0f) : FMargin(8.0f, 2.0f, 4.0f, 0.0f);
	const FMargin SectionTextPadding = (MovieSceneSubAssemblySection->IsCinematicShotSection()) ? FMargin(0.0f, 12.0f, 0.0f, 0.0f) : FMargin(0.0f, 2.0f, 0.0f, 0.0f);
	const FMargin OriginTextPadding = (MovieSceneSubAssemblySection->IsCinematicShotSection()) ? FMargin(0.0f, 15.0f, 0.0f, 0.0f) : FMargin(0.0f, 5.0f, 0.0f, 0.0f);

	FNamingTokenFilterArgs FilterArgs;
	FilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

	return SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(WarningIconPadding)
				[
					SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Warning"))
						.ColorAndOpacity(FStyleColors::Warning)
						.ToolTipText(LOCTEXT("DuplicateNameWarning", "Another SubAssembly section in this schema would resolve to the same name. Set different Metadata Overrides or rename the section to differentiate them."))
						.Visibility_Lambda([this]() { return HasDuplicateName() ? EVisibility::Visible : EVisibility::Collapsed; })
				]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Top)
			.Padding(SectionTextPadding)
			[
				SAssignNew(EditableTextBlock, SNamingTokensEditableTextBox)
					.SupportInlineEdit(true)
					.ShouldEvaluateTokens(false)
					.FilterArgs(FilterArgs)
					.DisplayTokenIcon(false)
					.DisplayBorderImage(false)
					.Text(this, &FSubAssemblySection::GetSectionText)
					.OnTextCommitted(this, &FSubAssemblySection::SetSectionText)
					.OnValidateTokenizedText(this, &FSubAssemblySection::ValidateSectionText)
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(OriginTextPadding)
			[
				SNew(STextBlock)
					.Text(this, &FSubAssemblySection::GetOriginText)
					.Font(FCoreStyle::GetDefaultFontStyle("Italic", 10))
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(8.0f, 2.0f, 0.0f, 0.0f))
		[
			SNew(STextBlock)
				.Text(this, &FSubAssemblySection::GetLabelText)
				.Visibility(this, &FSubAssemblySection::GetLabelVisibility)
				.Font(FCoreStyle::GetDefaultFontStyle("Italic", 10))
		];
}

int32 FSubAssemblySection::OnPaintSection(FSequencerSectionPainter& InPainter) const
{
	InPainter.LayerId = InPainter.PaintSectionBackground();

	// Draw the film border used by shot tracks
	if (MovieSceneSubAssemblySection->IsCinematicShotSection())
	{
		static const FSlateBrush* FilmBorder = FAppStyle::GetBrush("Sequencer.Section.FilmBorder");

		FVector2D LocalHeaderSize = InPainter.HeaderGeometry.GetLocalSize();

		FSlateDrawElement::MakeBox(
			InPainter.DrawElements,
			InPainter.LayerId++,
			InPainter.SectionGeometry.ToPaintGeometry(FVector2D(LocalHeaderSize.X - 2.f, 7.f), FSlateLayoutTransform(FVector2D(1.f, 4.f))),
			FilmBorder,
			InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect
		);

		FSlateDrawElement::MakeBox(
			InPainter.DrawElements,
			InPainter.LayerId++,
			InPainter.SectionGeometry.ToPaintGeometry(FVector2D(LocalHeaderSize.X - 2.f, 7.f), FSlateLayoutTransform(FVector2D(1.f, LocalHeaderSize.Y - 11.f))),
			FilmBorder,
			InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect
		);
	}

	return InPainter.LayerId;
}

FReply FSubAssemblySection::OnSectionDoubleClicked(const FGeometry& SectionGeometry, const FPointerEvent& MouseEvent)
{
	// TODO: Check the geometry to determine what double clicking should do.
	// If double clicking on the Naming Token Textbox, it should allow the user to rename the section.
	// If double clicking anywhere else, and the Section's sequence is a "Schema Template", then it should open that Schema in a separate window.
	if (MovieSceneSubAssemblySection->IsTemplateSection())
	{
		EditableTextBlock->EnterEditingMode();
	}

	return FReply::Handled();
}

FText FSubAssemblySection::GetSectionText() const
{
	if (MovieSceneSubAssemblySection->IsTemplateSection())
	{
		return MovieSceneSubAssemblySection->GetSequenceName();
	}

	return MovieSceneSubAssemblySection->GetSequence() ? MovieSceneSubAssemblySection->GetSequence()->GetDisplayName() : FText::GetEmpty();
}

void FSubAssemblySection::SetSectionText(const FText& InText, ETextCommit::Type InCommitType)
{
	MovieSceneSubAssemblySection->Modify();
	if (MovieSceneSubAssemblySection->IsTemplateSection())
	{
		MovieSceneSubAssemblySection->SetSequenceName(InText);
	}
}

bool FSubAssemblySection::ValidateSectionText(const FText& InText, FText& OutErrorMessage) const
{
	// SubAssemblies cannot use the {assembly} token in their name because it resolves to the SubAssembly name itself, causing infinite name recursion.
	// The {parent} token is the correct way to reference the top-level assembly name.
	const FString PotentialName = InText.ToString();
	if (PotentialName.Contains(TEXT("{assembly}")) || PotentialName.Contains(TEXT("{cat:assembly}")))
	{
		OutErrorMessage = LOCTEXT("SubAssemblyAssemblyTokenError", "The {assembly} token cannot be used in an assembly name because it is self-referential. Use {parent} instead to reference the top-level assembly name.");
		return false;
	}
	return true;
}

FText FSubAssemblySection::GetOriginText() const
{
	if (MovieSceneSubAssemblySection->IsTemplateSection())
	{
		if (UObject* TemplateObject = MovieSceneSubAssemblySection->GetAssemblyTemplate())
		{
			return FText::Format(LOCTEXT("TemplateOriginText", "from {0}"), FText::FromString(TemplateObject->GetName()));
		}
		return FText::GetEmpty();
	}

	// The origin text of a reference section simply indicates to the user that the sequence is a reference
	return LOCTEXT("ReferenceText", "Reference");
}

FText FSubAssemblySection::GetLabelText() const
{
	// Labels are only meaningful for template sections (reference sections reuse an existing sequence).
	if (MovieSceneSubAssemblySection && MovieSceneSubAssemblySection->IsTemplateSection() && !MovieSceneSubAssemblySection->Label.IsNone())
	{
		return FText::Format(LOCTEXT("LabelText", "[{0}]"), FText::FromName(MovieSceneSubAssemblySection->Label));
	}
	return FText::GetEmpty();
}

EVisibility FSubAssemblySection::GetLabelVisibility() const
{
	const bool bHasTemplateLabel = MovieSceneSubAssemblySection && MovieSceneSubAssemblySection->IsTemplateSection() && !MovieSceneSubAssemblySection->Label.IsNone();
	return bHasTemplateLabel ? EVisibility::Visible : EVisibility::Collapsed;
}

bool FSubAssemblySection::HasDuplicateName() const
{
	if (!MovieSceneSubAssemblySection || !MovieSceneSubAssemblySection->IsTemplateSection())
	{
		return false;
	}

	UMovieScene* MovieScene = MovieSceneSubAssemblySection->GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return false;
	}

	// Cache the template name and overrides for this section, to compared against other sections in the MovieScene
	const FString TemplateName = MovieSceneSubAssemblySection->GetSequenceName().ToString();
	const TMap<FString, FString>& MetadataOverrides = MovieSceneSubAssemblySection->MetadataOverrides;

	// Find any other template section with the same name template and equivalent metadata overrides
	for (UMovieSceneSection* Section : MovieScene->GetAllSections())
	{
		UMovieSceneSubAssemblySection* OtherSection = Cast<UMovieSceneSubAssemblySection>(Section);
		if (!OtherSection || OtherSection == MovieSceneSubAssemblySection || !OtherSection->IsTemplateSection())
		{
			continue;
		}

		// Warning is only valid if the sections share the same name
		if (OtherSection->GetSequenceName().ToString() != TemplateName)
		{
			continue;
		}

		// If the names match and the metadata overrides are the same, then it is likely that these sections would end up with a naming conflict
		if (OtherSection->MetadataOverrides.OrderIndependentCompareEqual(MetadataOverrides))
		{
			return true;
		}
	}

	return false;
}

void FSubAssemblySection::CustomizePropertiesDetailsView(TSharedRef<IDetailsView> DetailsView, const FSequencerSectionPropertyDetailsViewCustomizationParams& InParams) const
{
	DetailsView->RegisterInstancedCustomPropertyLayout(UMovieSceneSubAssemblySection::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FSubAssemblySectionDetailCustomization::MakeInstance));
}

void FSubAssemblySection::BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("RenameAction", "Rename"),
		LOCTEXT("RenameActionTooltip", "Rename the new sequence that will be created from this section"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]() { EditableTextBlock->EnterEditingMode(); }),
			FCanExecuteAction::CreateLambda([this]() { return MovieSceneSubAssemblySection->IsTemplateSection(); })
		),
		NAME_None,
		EUserInterfaceActionType::None
	);
}

#undef LOCTEXT_NAMESPACE