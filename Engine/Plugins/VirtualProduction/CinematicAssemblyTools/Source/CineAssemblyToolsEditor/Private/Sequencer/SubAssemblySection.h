// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerSection.h"

#include "MovieSceneSubAssemblySection.h"
#include "SNamingTokensEditableTextBox.h"
#include "TrackEditors/SubTrackEditorBase.h"

/**
 * An ISequencerSection that handles the UI of a UMovieSceneSubAssemblySection
 */
class FSubAssemblySection : public TSubSectionMixin<>, public TSharedFromThis<FSubAssemblySection>
{
public:
	FSubAssemblySection(TSharedPtr<ISequencer> InSequencer, UMovieSceneSubAssemblySection* InSection);

public:
	//~ Begin ISequencerSection interface
	virtual UMovieSceneSection* GetSectionObject() override;
	virtual FText GetSectionTitle() const override;
	virtual float GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const override;
	virtual TSharedRef<SWidget> GenerateSectionWidget() override;
	virtual int32 OnPaintSection(FSequencerSectionPainter& InPainter) const override;
	virtual FReply OnSectionDoubleClicked(const FGeometry& SectionGeometry, const FPointerEvent& MouseEvent) override;
	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding) override;
	virtual void CustomizePropertiesDetailsView(TSharedRef<IDetailsView> DetailsView, const FSequencerSectionPropertyDetailsViewCustomizationParams& InParams) const override;
	//~ End ISequencerSection interface

private:
	/** Get the name of the sequence to display in the Section Widget */
	FText GetSectionText() const;

	/** Set the name of the sequence in a template section */
	void SetSectionText(const FText& InText, ETextCommit::Type InCommitType);

	/** Validate the tokenized Subassembly name */
	bool ValidateSectionText(const FText& InText, FText& OutErrorMessage) const;

	/** If this is a template section, return the name of the template object. Otherwise, return text indicating that this is a reference section. */
	FText GetOriginText() const;

	/** Returns the section's label wrapped in square brackets for display, or empty text if no label is set */
	FText GetLabelText() const;

	/** Visibility for the label line (hidden when no label is set) */
	EVisibility GetLabelVisibility() const;

	/** Returns true if this template section could produce a SubAssembly with the same resolved name as another template section in the same MovieScene */
	bool HasDuplicateName() const;

private:
	/** MovieSceneSubAssemblySection handled by this sequencer section */
	UMovieSceneSubAssemblySection* MovieSceneSubAssemblySection;

	/** Editable textbox widget, used for renaming the sequence in template sections */
	TSharedPtr<SNamingTokensEditableTextBox> EditableTextBlock;
};
