// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubAssemblyTrackEditor.h"

#include "CineAssembly.h"
#include "CineAssemblySchema.h"
#include "ContentBrowserModule.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "IContentBrowserSingleton.h"
#include "MovieSceneSubAssemblySection.h"
#include "MovieSceneSubAssemblyTrack.h"
#include "MovieSceneToolsProjectSettings.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Sections/MovieSceneSubSection.h"
#include "SequencerSettings.h"
#include "SequencerUtilities.h"
#include "SubAssemblySection.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "SubAssemblyTrackEditor"

FSubAssemblyTrackEditor::FSubAssemblyTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FSubTrackEditor(InSequencer)
{
}

TSharedRef<ISequencerTrackEditor> FSubAssemblyTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
{
	return MakeShared<FSubAssemblyTrackEditor>(OwningSequencer);
}

TSubclassOf<UMovieSceneSubTrack> FSubAssemblyTrackEditor::GetSubTrackClass() const
{
	return UMovieSceneSubAssemblyTrack::StaticClass();
}

bool FSubAssemblyTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	// If the focused Sequence that is currently open belongs to a Schema, then we know we are in the Schema Sequencer tool.
	// In this case, CineAssemblies are always supported, and other sequences are supported if they support normal SubTracks
	if (GetFocusedMovieScene()->GetTypedOuter<UCineAssemblySchema>())
	{
		if (InSequence && InSequence->IsA(UCineAssembly::StaticClass()))
		{
			return true;
		}

		ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneSubTrack::StaticClass()) : ETrackSupport::NotSupported;
		return TrackSupported == ETrackSupport::Supported;
	}

	// Outside of the Schema Sequencer, SubAssemblyTracks are not supported, and will not appear in the "Add Track" menu
	return false;
}

bool FSubAssemblyTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid)
{
	if (UMovieScene* FocusedMovieScene = GetFocusedMovieScene())
	{
		const FFrameNumber FrameNumber = GetSequencer()->GetLocalTime().Time.FrameNumber;
		
		// If the drop is successful, a new track will be created and the dropped asset will be added to it
		UMovieSceneSubAssemblyTrack* NullTrack = nullptr;
		return AddDroppedAsset(Asset, NullTrack, FrameNumber);
	}
	return false;
}

FReply FSubAssemblyTrackEditor::OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams)
{
	if (!DragDropParams.Track.IsValid())
	{
		return FReply::Unhandled();
	}

	if (!DragDropParams.Track.Get()->IsA(GetSubTrackClass()))
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

	if (!Operation.IsValid() || !Operation->IsOfType<FAssetDragDropOp>())
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);

	bool bAnyDropped = false;
	for (const FAssetData& AssetData : DragDropOp->GetAssets())
	{
		bAnyDropped |= AddDroppedAsset(AssetData.GetAsset(), Cast<UMovieSceneSubAssemblyTrack>(DragDropParams.Track), DragDropParams.FrameNumber);
	}

	return bAnyDropped ? FReply::Handled() : FReply::Unhandled();
}

bool FSubAssemblyTrackEditor::AddDroppedAsset(UObject* Asset, UMovieSceneSubAssemblyTrack* Track, FFrameNumber FrameNumber)
{
	// If the dropped asset is a Schema, it can only be added as a template
	if (Asset && Asset->IsA(UCineAssemblySchema::StaticClass()))
	{
		OnDroppedAsTemplate(Asset, Track, FrameNumber);
		return true;
	}

	// Otherwise, check that the dropped asset is a valid Sequence that is supported by this track editor
	UMovieSceneSequence* Sequence = Cast<UMovieSceneSequence>(Asset);
	if (!Sequence)
	{
		return false;
	}

	if (!Sequence->GetMovieScene())
	{
		return false;
	}

	if (!SupportsSequence(Sequence))
	{
		return false;
	}

	// Build and display a menu to let the user choose between adding the dropped sequence as a reference, or as a template for a new sequence
	constexpr bool bShouldCloseAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseAfterMenuSelection, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("OnDroppedNewSequence", "Add New Sequence"),
		LOCTEXT("OnDroppedNewSequenceTooltip", "Add a new Subsequence using the dropped Sequence as the base"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateRaw(this, &FSubAssemblyTrackEditor::OnDroppedAsTemplate, Asset, Track, FrameNumber))
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("OnDroppedReference", "Add Reference to Sequence"),
		LOCTEXT("OnDroppedReferenceTooltip", "Add a reference to the dropped Sequence"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateRaw(this, &FSubAssemblyTrackEditor::OnDroppedAsReference, Sequence, Track, FrameNumber))
	);

	// Open the menu where at the user's current mouse position
	UE::Slate::FDeprecateVector2DParameter MousePosition = FSlateApplication::Get().GetCursorPos();
	FWidgetPath WidgetPath = FSlateApplication::Get().LocateWindowUnderMouse(MousePosition, FSlateApplication::Get().GetInteractiveTopLevelWindows(), false, 0);
	if (WidgetPath.IsValid() && WidgetPath.Widgets.Num() > 0)
	{
		FSlateApplication::Get().PushMenu(WidgetPath.GetLastWidget(), WidgetPath, MenuBuilder.MakeWidget(), MousePosition, FPopupTransitionEffect::ContextMenu);
	}

	return true;
}

TSharedRef<ISequencerSection> FSubAssemblyTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	UMovieSceneSubAssemblySection* SubAssemblySection = Cast<UMovieSceneSubAssemblySection>(&SectionObject);
	return MakeShared<FSubAssemblySection>(GetSequencer(), SubAssemblySection);
}

void FSubAssemblyTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	// This Track Editor mimics both the Subsequence Track Editor and Cinematic Shot Track Editor for the purpose of creating the Schema's template sequence
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShotTrackName", "Shot Track"),
		LOCTEXT("ShotTrackToolTip", "A cinematic shot track."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.CinematicShot"),
		FUIAction(FExecuteAction::CreateLambda([this]() 
			{
				AddSubAssemblyTrack(ESubAssemblyTrackType::CinematicShotTrack);
			}))
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SubTrackName", "Subsequence Track"),
		LOCTEXT("SubTrackToolTip", "A track that can contain other sequences."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.Sub"),
		FUIAction(FExecuteAction::CreateLambda([this]()
			{
				AddSubAssemblyTrack(ESubAssemblyTrackType::SubsequenceTrack);
			}))
	);
}

UMovieSceneSubAssemblyTrack* FSubAssemblyTrackEditor::AddSubAssemblyTrack(ESubAssemblyTrackType TrackType)
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (!FocusedMovieScene)
	{
		return nullptr;
	}

	// Regardless of the track type, a new MovieSceneSubAssemblyTrack is added to the Schema's template sequence
	UMovieSceneSubAssemblyTrack* NewTrack = Cast<UMovieSceneSubAssemblyTrack>(FocusedMovieScene->AddTrack(UMovieSceneSubAssemblyTrack::StaticClass()));
	NewTrack->TrackType = TrackType;

	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(NewTrack, FGuid());
	}

	return NewTrack;
}

TSharedPtr<SWidget> FSubAssemblyTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	return UE::Sequencer::MakeAddButton(GetSubTrackName(), FOnGetContent::CreateRaw(this, &FSubAssemblyTrackEditor::BuildAddSubSequenceComboButtonContent, Params.ViewModel.AsWeak()), Params.ViewModel);
}

TSharedPtr<SWidget> FSubAssemblyTrackEditor::BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName)
{
	// This label column widget resembles an SOutlinerItemViewBase, but allows the different track types to each support their own icon and text label
	if (ColumnName == UE::Sequencer::FCommonOutlinerNames::Label)
	{
		UE::Sequencer::TViewModelPtr<UE::Sequencer::ITrackExtension> TrackModel = Params.ViewModel->FindAncestorOfType<UE::Sequencer::ITrackExtension>(true);
		if (!TrackModel)
		{
			return SNullWidget::NullWidget;
		}

		UMovieSceneSubAssemblyTrack* Track = Cast<UMovieSceneSubAssemblyTrack>(TrackModel->GetTrack());
		if (!Track)
		{
			return SNullWidget::NullWidget;
		}

		return SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SExpanderArrow, Params.TreeViewRow)
					.IndentAmount(10.0f)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 0.0f)
			[
				SNew(SImage)
					.Image(FAppStyle::Get().GetBrush(Track->GetTrackIconBrush()))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
					.Text(Track->GetDefaultDisplayName())
					.Font(FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont"))
			];
	}

	return FSubTrackEditor::BuildOutlinerColumnWidget(Params, ColumnName);
}

TSharedRef<SWidget> FSubAssemblyTrackEditor::BuildAddSubSequenceComboButtonContent(UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FViewModel> WeakViewModel)
{
	using namespace UE::Sequencer;

	TViewModelPtr<FViewModel> ViewModel = WeakViewModel.Pin();
	if (!ViewModel)
	{
		return SNullWidget::NullWidget;
	}

	TViewModelPtr<ITrackExtension> TrackModel = ViewModel->FindAncestorOfType<ITrackExtension>(true);
	if (!TrackModel)
	{
		return SNullWidget::NullWidget;
	}

	UMovieSceneSubAssemblyTrack* Track = Cast<UMovieSceneSubAssemblyTrack>(TrackModel->GetTrack());
	if (!Track)
	{
		return SNullWidget::NullWidget;
	}

	constexpr bool bShouldCloseAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseAfterMenuSelection, nullptr);

	MenuBuilder.AddSubMenu(
		LOCTEXT("AddNewSequence", "Add New Sequence"),
		LOCTEXT("AddNewSequenceTooltip", "Add a new Sequence to this track using an existing Schema or Sequence as a base"),
		FNewMenuDelegate::CreateRaw(this, &FSubAssemblyTrackEditor::AddNewSequenceSubMenu, Track));

	MenuBuilder.AddSubMenu(
		LOCTEXT("AddReference", "Add Reference to Sequence"),
		LOCTEXT("AddReferenceTooltip", "Add a reference to an existing Sequence to this track"),
		FNewMenuDelegate::CreateRaw(this, &FSubAssemblyTrackEditor::AddReferenceSubMenu, Track));

	FSequencerUtilities::MakeTimeWarpMenuEntry(MenuBuilder, WeakViewModel);

	return MenuBuilder.MakeWidget();
}

void FSubAssemblyTrackEditor::AddNewSequenceSubMenu(FMenuBuilder& MenuBuilder, UMovieSceneSubAssemblyTrack* Track)
{
	MenuBuilder.AddSubMenu(
		LOCTEXT("UseSchemaTemplate", "Using Schema Template"),
		LOCTEXT("UseSchemaTemplateTooltip", "Use an existing Schema as the base for a new Subsequence"),
		FNewMenuDelegate::CreateLambda([this, Track](FMenuBuilder& InMenuBuilder)
			{
				InMenuBuilder.BeginSection(TEXT("SchemaTemplateSection"), LOCTEXT("AddNewSequenceFromSchema", "Add New Sequence Using Schema Template"));
				{
					// Only display CineAssemblySchemas in the asset picker
					TArray<FTopLevelAssetPath> ClassPaths = { UCineAssemblySchema::StaticClass()->GetClassPathName() };

					constexpr bool bUseAssetAsTemplate = true;
					TSharedRef<SWidget> AssetPickerWidget = BuildAssetPicker(Track, ClassPaths, bUseAssetAsTemplate);

					InMenuBuilder.AddWidget(AssetPickerWidget, FText::GetEmpty());
				}
				InMenuBuilder.EndSection();
			}));

	MenuBuilder.AddSubMenu(
		LOCTEXT("UseSequenceTemplate", "Using Sequence Template"),
		LOCTEXT("UseSequenceTemplateTooltip", "Use an existing Sequence/Assembly as the base for a new Subsequence"),
		FNewMenuDelegate::CreateLambda([this, Track](FMenuBuilder& InMenuBuilder)
			{
				InMenuBuilder.BeginSection(TEXT("SequenceTemplateSection"), LOCTEXT("AddNewSequenceFromSequence", "Add New Sequence Using Sequence Template"));
				{
					// Only display LevelSequences and CineAssemblies in the asset picker
					TArray<FTopLevelAssetPath> ClassPaths = { UCineAssembly::StaticClass()->GetClassPathName(), ULevelSequence::StaticClass()->GetClassPathName() };

					constexpr bool bUseAssetAsTemplate = true;
					TSharedRef<SWidget> AssetPickerWidget = BuildAssetPicker(Track, ClassPaths, bUseAssetAsTemplate);

					InMenuBuilder.AddWidget(AssetPickerWidget, FText::GetEmpty());
				}
				InMenuBuilder.EndSection();
			}));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("EmptySequence", "Empty Sequence"),
		LOCTEXT("EmptySequenceTooltip", "Add a new blank Sequence to this track"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateRaw(this, &FSubAssemblyTrackEditor::AddNewSequenceToTrack, Track, (UObject*)nullptr))
	);
}

void FSubAssemblyTrackEditor::AddReferenceSubMenu(FMenuBuilder& MenuBuilder, UMovieSceneSubAssemblyTrack* Track)
{
	MenuBuilder.BeginSection(TEXT("SequenceReference"), LOCTEXT("AddReferenceToSequence", "Add Reference To Sequence"));
	{
		// Only display LevelSequences and CineAssemblies in the asset picker
		TArray<FTopLevelAssetPath> ClassPaths = { ULevelSequence::StaticClass()->GetClassPathName() , UCineAssembly::StaticClass()->GetClassPathName() };

		constexpr bool bUseAssetAsTemplate = false;
		TSharedRef<SWidget> AssetPickerWidget = BuildAssetPicker(Track, ClassPaths, bUseAssetAsTemplate);

		MenuBuilder.AddWidget(AssetPickerWidget, FText::GetEmpty());
	}
	MenuBuilder.EndSection();
}

TSharedRef<SWidget> FSubAssemblyTrackEditor::BuildAssetPicker(UMovieSceneSubAssemblyTrack* Track, TArray<FTopLevelAssetPath> ClassPaths, bool bUseAssetAsTemplate)
{
	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bAddFilterUI = true;
	AssetPickerConfig.bShowTypeInColumnView = false;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.Filter.ClassPaths = ClassPaths;
	AssetPickerConfig.SaveSettingsName = TEXT("SubAssemblyTrackAssetPicker");

	// Filter out the Schema that owns this track because selecting it as a template would cause infinite recursion at assembly creation time
	if (const UCineAssemblySchema* OwningSchema = Track ? Track->GetTypedOuter<UCineAssemblySchema>() : nullptr)
	{
		const FSoftObjectPath OwningSchemaPath(OwningSchema);
		AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([OwningSchemaPath](const FAssetData& InAssetData)
			{
				return InAssetData.GetSoftObjectPath() == OwningSchemaPath;
			});
	}

	if (bUseAssetAsTemplate)
	{
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &FSubAssemblyTrackEditor::OnAssetSelectedAsTemplate, Track);
		AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateRaw(this, &FSubAssemblyTrackEditor::OnAssetEnterPressedAsTemplate, Track);
	}
	else
	{
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &FSubAssemblyTrackEditor::OnAssetSelectedAsReference, Track);
		AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateRaw(this, &FSubAssemblyTrackEditor::OnAssetEnterPressedAsReference, Track);
	}

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
	const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	return SNew(SBox)
		.WidthOverride(WidthOverride)
		.HeightOverride(HeightOverride)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];
}

void FSubAssemblyTrackEditor::AddNewSequenceToTrack(UMovieSceneSubAssemblyTrack* Track, UObject* TemplateObject)
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (!FocusedMovieScene || !Track)
	{
		return;
	}

	const FFrameNumber StartFrame = GetTimeForKey();
	int32 Duration = FocusedMovieScene->GetPlaybackRange().Size<FFrameNumber>().Value;

	// If the template object is a valid sequence, use its duration instead of the default duration
	UMovieSceneSequence* TemplateSequence = Cast<UMovieSceneSequence>(TemplateObject);
	if (TemplateSequence && TemplateSequence->GetMovieScene())
	{
		Duration = TemplateSequence->GetMovieScene()->GetPlaybackRange().Size<FFrameNumber>().Value;
	}

	// Add a new SubAssembly section to the input track, and set its assembly template object
	UMovieSceneSubAssemblySection* SubAssemblySection = Cast<UMovieSceneSubAssemblySection>(Track->AddSequence(nullptr, StartFrame, Duration));
	SubAssemblySection->SetAssemblyTemplate(TemplateObject);
	SubAssemblySection->SetDefaultLabel();

	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

void FSubAssemblyTrackEditor::OnAssetSelectedAsTemplate(const FAssetData& AssetData, UMovieSceneSubAssemblyTrack* Track)
{
	AddNewSequenceToTrack(Track, AssetData.GetAsset());
}

void FSubAssemblyTrackEditor::OnAssetEnterPressedAsTemplate(const TArray<FAssetData>& AssetData, UMovieSceneSubAssemblyTrack* Track)
{
	if (AssetData.Num() > 0)
	{
		OnAssetSelectedAsTemplate(AssetData[0], Track);
	}
}

void FSubAssemblyTrackEditor::OnAssetSelectedAsReference(const FAssetData& AssetData, UMovieSceneSubAssemblyTrack* Track)
{
	if (UMovieSceneSequence* SelectedSequence = Cast<UMovieSceneSequence>(AssetData.GetAsset()))
	{
		FSubTrackEditor::AddSequenceToTrack(SelectedSequence, Track);
	}
}

void FSubAssemblyTrackEditor::OnAssetEnterPressedAsReference(const TArray<FAssetData>& AssetData, UMovieSceneSubAssemblyTrack* Track)
{
	if (AssetData.Num() > 0)
	{
		OnAssetSelectedAsReference(AssetData[0], Track);
	}
}

void FSubAssemblyTrackEditor::OnDroppedAsTemplate(UObject* Asset, UMovieSceneSubAssemblyTrack* Track, FFrameNumber FrameNumber)
{
	// If the input track was null, create a new one
	if (!Track)
	{
		const ESubAssemblyTrackType TrackType = GetTrackTypeForAsset(Asset);
		Track = AddSubAssemblyTrack(TrackType);
	}

	// Initiate keying at the input frame number to ensure that this is where the dropped sequence will be placed
	FMovieSceneTrackEditor::BeginKeying(FrameNumber);
	AddNewSequenceToTrack(Track, Asset);
	FMovieSceneTrackEditor::EndKeying();
}

void FSubAssemblyTrackEditor::OnDroppedAsReference(UMovieSceneSequence* Sequence, UMovieSceneSubAssemblyTrack* Track, FFrameNumber FrameNumber)
{
	// If the input track was null, create a new one
	if (!Track)
	{
		const ESubAssemblyTrackType TrackType = GetTrackTypeForAsset(Sequence);
		Track = AddSubAssemblyTrack(TrackType);
	}

	// Initiate keying at the input frame number to ensure that this is where the dropped sequence will be placed
	FMovieSceneTrackEditor::BeginKeying(FrameNumber);
	FSubTrackEditor::AddSequenceToTrack(Sequence, Track);
	FMovieSceneTrackEditor::EndKeying();
}

ESubAssemblyTrackType FSubAssemblyTrackEditor::GetTrackTypeForAsset(UObject* Asset)
{
	if (UMovieSceneSequence* Sequence = Cast<UMovieSceneSequence>(Asset))
	{
		if (UMovieScene* MovieScene = Sequence->GetMovieScene())
		{
			if (MovieScene->GetCameraCutTrack())
			{
				return ESubAssemblyTrackType::CinematicShotTrack;
			}
		}
	}

	return ESubAssemblyTrackType::SubsequenceTrack;
}

#undef LOCTEXT_NAMESPACE
