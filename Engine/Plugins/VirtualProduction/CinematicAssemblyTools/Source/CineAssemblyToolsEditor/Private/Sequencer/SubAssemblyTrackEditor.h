// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TrackEditors/SubTrackEditor.h"

enum class ESubAssemblyTrackType : uint8;

class FMenuBuilder;
class UCineAssembly;
class UMovieSceneSubTrack;
class UMovieSceneSubAssemblyTrack;

struct FAssetData;

/**
 * Sequencer Track Editor for SubAssembly Tracks (extends the functionality of normal SubTracks for CineAssemblies)
 */
class FSubAssemblyTrackEditor : public FSubTrackEditor
{
public:
	FSubAssemblyTrackEditor(TSharedRef<ISequencer> InSequencer);
	virtual ~FSubAssemblyTrackEditor() = default;

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

	//~ Begin ISequencerTrackEditor interface
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual TSharedPtr<SWidget> BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName) override;
	virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override;
	virtual FReply OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	//~ End ISequencerTrackEditor interface

	//~ Begin FSubTrackEditor interface
	virtual TSubclassOf<UMovieSceneSubTrack> GetSubTrackClass() const override;
	//~ End FSubTrackEditor interface

private:
	/** Adds a new SubAssembly Track to the focused movie scene, and sets it Track Type appropriately */
	UMovieSceneSubAssemblyTrack* AddSubAssemblyTrack(ESubAssemblyTrackType TrackType);

	/** Build the menu to display when clicking the "+" button on a SubAssembly Track */
	TSharedRef<SWidget> BuildAddSubSequenceComboButtonContent(UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FViewModel> WeakViewModel);

	/** Build an Asset Picker widget that will only display assets matching one of the input class paths */
	TSharedRef<SWidget> BuildAssetPicker(UMovieSceneSubAssemblyTrack* Track, TArray<FTopLevelAssetPath> ClassPaths, bool bUseAssetAsTemplate);

	/** Adds the "New Sequence" submenu to the SubAssembly track menu */
	void AddNewSequenceSubMenu(FMenuBuilder& MenuBuilder, UMovieSceneSubAssemblyTrack* Track);

	/** Adds the "Reference" Sequence submenu to the SubAssembly track menu */
	void AddReferenceSubMenu(FMenuBuilder& MenuBuilder, UMovieSceneSubAssemblyTrack* Track);

	/** Callback when an asset is selected in the asset picker as a template for a new sequence to add to the track */
	void OnAssetSelectedAsTemplate(const FAssetData& AssetData, UMovieSceneSubAssemblyTrack* Track);

	/** Callback when one or more assets are selected in the asset picker as a template for a new sequence to add to the track */
	void OnAssetEnterPressedAsTemplate(const TArray<FAssetData>& AssetData, UMovieSceneSubAssemblyTrack* Track);

	/** Callback when an asset is selected in the asset picker as a reference to add to the track */
	void OnAssetSelectedAsReference(const FAssetData& AssetData, UMovieSceneSubAssemblyTrack* Track);

	/** Callback when one or more assets are selected in the asset picker as a reference to add to the track */
	void OnAssetEnterPressedAsReference(const TArray<FAssetData>& AssetData, UMovieSceneSubAssemblyTrack* Track);

	/** Creates and adds a new sequence to the input Track, then finds the newly created SubAssemblySection and assigns the input TemplateObject to it */
	void AddNewSequenceToTrack(UMovieSceneSubAssemblyTrack* Track, UObject* TemplateObject);

	/** Adds the input Asset onto the input Track (or a new Track if that is null) at the input FrameNumber */
	bool AddDroppedAsset(UObject* Asset, UMovieSceneSubAssemblyTrack* Track, FFrameNumber FrameNumber);

	/** Callback when an asset is dropped and handled by this track editor to be added as a template for a new sequence */
	void OnDroppedAsTemplate(UObject* Asset, UMovieSceneSubAssemblyTrack* Track, FFrameNumber FrameNumber);

	/** Callback when an asset is dropped and handled by this track editor to be added as a reference */
	void OnDroppedAsReference(UMovieSceneSequence* Sequence, UMovieSceneSubAssemblyTrack* Track, FFrameNumber FrameNumber);

	/** Return the appropriate track type for the input asset */
	ESubAssemblyTrackType GetTrackTypeForAsset(UObject* Asset);
};