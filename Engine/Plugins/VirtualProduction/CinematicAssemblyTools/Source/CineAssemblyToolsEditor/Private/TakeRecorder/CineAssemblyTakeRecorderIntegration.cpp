// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblyTakeRecorderIntegration.h"

#include "CineAssembly.h"
#include "CineAssemblyFactory.h"
#include "CineAssemblySchema.h"
#include "CineAssemblyMovieSceneDecorations.h"
#include "CineAssemblyTakeRecorderSettings.h"
#include "CineAssemblyToolsAnalytics.h"
#include "Editor.h"
#include "Engine/Engine.h"
#include "ILevelSequenceEditorToolkit.h"
#include "ISequencer.h"
#include "ITakeRecorderModule.h"
#include "MovieScene.h"
#include "MovieSceneBinding.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "ProductionSettings.h"
#include "Recorder/TakeRecorder.h"
#include "Recorder/TakeRecorderSubsystem.h"
#include "Sections/MovieSceneSubSection.h"
#include "Sequencer/CineAssemblySequencerUtilities.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "TakeMetaData.h"
#include "TakePreset.h"
#include "TakeRecorderSources.h"
#include "TakesUtils.h"

DEFINE_LOG_CATEGORY_STATIC(LogCineAssemblyTakeRecorderIntegration, Log, All)

namespace UE::CineAssemblyTools::Private
{
	/** Locate the open ISequencer instance for InLevelSequence (if any). Returns null if no editor is currently open for it. */
	TSharedPtr<ISequencer> FindSequencer(ULevelSequence* InLevelSequence)
	{
		if (!GEditor)
		{
			return nullptr;
		}

		UAssetEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		if (!EditorSubsystem)
		{
			return nullptr;
		}

		IAssetEditorInstance* AssetEditor = EditorSubsystem->FindEditorForAsset(InLevelSequence, false);
		ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);
		return LevelSequenceEditor ? LevelSequenceEditor->GetSequencer() : nullptr;
	}

	/** Returns true if the input SubSection carries the full-range decoration */
	bool IsFullRangeSection(const UMovieSceneSubSection* InSubSection)
	{
		return InSubSection && InSubSection->FindDecoration<UCineAssemblyFullRangeDecoration>() != nullptr;
	}

	/** Create a scoped FViewModelHierarchyOperation that defers view-model broadcasts on InAssembly's open Sequencer until destruction. Returns null if no Sequencer is open. */
	TUniquePtr<UE::Sequencer::FViewModelHierarchyOperation> MakeUniqueHierarchyOp(UCineAssembly* InAssembly)
	{
		if (TSharedPtr<ISequencer> Sequencer = FindSequencer(InAssembly))
		{
			if (TSharedPtr<UE::Sequencer::FSequencerEditorViewModel> EditorViewModel = Sequencer->GetViewModel())
			{
				if (UE::Sequencer::FViewModelPtr RootModel = EditorViewModel->GetRootModel())
				{
					if (TSharedPtr<UE::Sequencer::FSharedViewModelData> SharedData = RootModel->GetSharedData())
					{
						return MakeUnique<UE::Sequencer::FViewModelHierarchyOperation>(SharedData);
					}
				}
			}
		}
		return nullptr;
	}

	/** Strip every schema-derived track and binding from the assembly's MovieScene, leaving anything the user authored manually intact. */
	void RemoveSchemaContent(UCineAssembly* InAssembly)
	{
		UMovieScene* MovieScene = InAssembly->GetMovieScene();
		if (!MovieScene)
		{
			return;
		}

		TArray<UMovieSceneTrack*> TracksToRemove;
		for (UMovieSceneTrack* Track : MovieScene->GetTracks())
		{
			if (Track && Track->FindDecoration<UCineAssemblySchemaContentDecoration>())
			{
				TracksToRemove.Add(Track);
			}
		}

		bool bRemoveCameraCutTrack = false;
		if (UMovieSceneTrack* CameraCutTrack = MovieScene->GetCameraCutTrack())
		{
			if (CameraCutTrack->FindDecoration<UCineAssemblySchemaContentDecoration>())
			{
				bRemoveCameraCutTrack = true;
			}
		}

		TArray<FGuid> BindingsToRemove;
		for (const FMovieSceneBinding& Binding : AsConst(*MovieScene).GetBindings())
		{
			// A binding is fully schema-derived if all of its tracks are tagged with the decoration.
			// If the binding has any untagged tracks, they were added by the user, and we will not delete the binding.
			bool bHasTaggedTrack = false;
			bool bHasUntaggedTrack = false;
			for (UMovieSceneTrack* Track : Binding.GetTracks())
			{
				if (Track)
				{
					if (Track->FindDecoration<UCineAssemblySchemaContentDecoration>())
					{
						TracksToRemove.Add(Track);
						bHasTaggedTrack = true;
					}
					else
					{
						bHasUntaggedTrack = true;
					}
				}
			}

			if (bHasTaggedTrack && !bHasUntaggedTrack)
			{
				BindingsToRemove.Add(Binding.GetObjectGuid());
			}
		}

		// Batch view-model change broadcasts while we bulk-remove tracks and bindings.
		TUniquePtr<UE::Sequencer::FViewModelHierarchyOperation> BatchedHierarchyOp = MakeUniqueHierarchyOp(InAssembly);

		for (UMovieSceneTrack* Track : TracksToRemove)
		{
			MovieScene->RemoveTrack(*Track);
		}

		if (bRemoveCameraCutTrack)
		{
			MovieScene->RemoveCameraCutTrack();
		}

		for (const FGuid& BindingGuid : BindingsToRemove)
		{
			if (!MovieScene->RemovePossessable(BindingGuid))
			{
				MovieScene->RemoveSpawnable(BindingGuid);
			}
		}
	}

	/** Rebuild InAssembly's SubAssemblies list from the SubSections currently in its MovieScene that have the SchemaContent decoration */
	void RebuildSubAssemblyList(UCineAssembly* InAssembly)
	{
		const UMovieScene* MovieScene = InAssembly->GetMovieScene();
		if (!MovieScene)
		{
			return;
		}

		InAssembly->SubAssemblies.Reset();
		for (UMovieSceneTrack* Track : MovieScene->GetTracks())
		{
			if (!Track || !Track->FindDecoration<UCineAssemblySchemaContentDecoration>())
			{
				continue;
			}

			for (UMovieSceneSection* Section : Track->GetAllSections())
			{
				if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
				{
					InAssembly->SubAssemblies.Add(SubSection);
				}
			}
		}
	}

	/**
	 * Tag every track in InAssembly's MovieScene (root tracks, camera cut track, and tracks under each binding) with
	 * a UCineAssemblySchemaContentDecoration. Applied to the schema's PreparedTemplate before paste; the decoration
	 * rides through StaticDuplicateObject as a sub-object so it lands on the target's duplicates automatically.
	 */
	void TagSchemaContent(UCineAssembly* InAssembly)
	{
		const UMovieScene* MovieScene = InAssembly->GetMovieScene();
		if (!MovieScene)
		{
			return;
		}

		for (UMovieSceneTrack* Track : MovieScene->GetTracks())
		{
			if (Track)
			{
				Track->GetOrCreateDecoration<UCineAssemblySchemaContentDecoration>();
			}
		}

		if (UMovieSceneTrack* CameraCutTrack = MovieScene->GetCameraCutTrack())
		{
			CameraCutTrack->GetOrCreateDecoration<UCineAssemblySchemaContentDecoration>();
		}

		for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
		{
			for (UMovieSceneTrack* Track : Binding.GetTracks())
			{
				if (Track)
				{
					Track->GetOrCreateDecoration<UCineAssemblySchemaContentDecoration>();
				}
			}
		}
	}

	/**
	 * Tag every top-level SubSection in InAssembly's MovieScene whose outer range exactly matches the playback range.
	 * The decoration tells us which SubSections should auto-expand during recording.
	 */
	void TagFullRangeSubSections(UCineAssembly* InAssembly)
	{
		const UMovieScene* MovieScene = InAssembly->GetMovieScene();
		if (!MovieScene)
		{
			return;
		}

		const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
		for (UMovieSceneTrack* Track : MovieScene->GetTracks())
		{
			if (!Track)
			{
				continue;
			}

			for (UMovieSceneSection* Section : Track->GetAllSections())
			{
				UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
				if (SubSection && SubSection->GetRange() == PlaybackRange)
				{
					SubSection->GetOrCreateDecoration<UCineAssemblyFullRangeDecoration>();
				}
			}
		}
	}

	/**
	 * Apply the schema's content to the target assembly. Strip any prior schema-derived tracks and bindings, then copy
	 * the new schema's tracks and bindings in (with binding-ID remap and decoration tagging so a future swap can clean up).
	 */
	void ApplySchemaToPendingTake(UCineAssembly* InTargetAssembly, UCineAssemblySchema* InSchema)
	{
		if (!InSchema->TemplateSequence)
		{
			return;
		}

		// Remove any content left over from a previous schema
		RemoveSchemaContent(InTargetAssembly);

		// Re-initialize the assembly with the input Schema, but skip duplicating the template sequence, so we can manually copy the tracks and bindings into the PendingTake
		constexpr bool bDuplicateMovieScene = false;
		InTargetAssembly->InitializeFromSchema(InSchema, bDuplicateMovieScene);

		// Duplicate the Schema's TemplateSequence into a transient intermediate assembly where we can run ConvertSubAssemblyTracks before copying them into the pending take.
		// If we copy the SubAssembly tracks directly into the pending take first (and convert them second), the open Sequencer might try to display unsupported tracks.
		// This avoids the problem and ensures that the Pending Take only ever has supported track types.
		UCineAssembly* PreparedTemplate = Cast<UCineAssembly>(StaticDuplicateObject(InSchema->TemplateSequence, GetTransientPackage(), NAME_None, RF_Transient));
		if (!PreparedTemplate)
		{
			return;
		}
		PreparedTemplate->ConvertSubAssemblyTracks();

		// Tag the tracks from the Schema's template sequence with the SchemaContent decoration and tag full-range SubSections
		TagSchemaContent(PreparedTemplate);
		TagFullRangeSubSections(PreparedTemplate);

		// Copy the prepared template's tracks and bindings into the target assembly's MovieScene.
		FCineAssemblySequencerUtilities::DuplicateMovieSceneContents(PreparedTemplate, InTargetAssembly);

		// Normally, an Assembly's managed SubAssembly list is populated during ConvertSubAssemblyTracks. However, this step is performed on the intermediate
		// assembly, and then duplicated into the target assembly. So we have to manually reconstruct that list using the SchemaContent tagged SubSections
		RebuildSubAssemblyList(InTargetAssembly);

		if (TSharedPtr<ISequencer> TargetSequencer = FindSequencer(InTargetAssembly))
		{
			TargetSequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
		}
	}

	/** 
	 * Collapse each managed SubAssembly (section and inner movie scene) of the input Assembly that has the UCineAssemblyFullRangeDecoration
	 * to a 1-frame range so it can properly auto-expand during recording 
	 */
	void CollapseFullRangeSubSections(UCineAssembly* InAssembly)
	{
		for (UMovieSceneSubSection* SubSection : InAssembly->SubAssemblies)
		{
			if (!SubSection || !IsFullRangeSection(SubSection))
			{
				continue;
			}

			SubSection->SetEndFrame(SubSection->GetInclusiveStartFrame() + 1);

			if (UMovieSceneSequence* Sequence = SubSection->GetSequence())
			{
				if (UMovieScene* InnerMovieScene = Sequence->GetMovieScene())
				{
					const FFrameNumber StartFrame = InnerMovieScene->GetPlaybackRange().HasLowerBound() ? InnerMovieScene->GetPlaybackRange().GetLowerBoundValue() : FFrameNumber(0);
					InnerMovieScene->SetPlaybackRange(TRange<FFrameNumber>(StartFrame, StartFrame + 1));
				}
			}
		}
	}

	/** 
	 * Expand each managed SubAssembly (section and inner movie scene) of the input Assembly that has the UCineAssemblyFullRangeDecoration
	 * to the input end frame 
	 */
	void ExpandFullRangeSubSections(UCineAssembly* InAssembly, FFrameNumber InEndFrame)
	{
		for (UMovieSceneSubSection* SubSection : InAssembly->SubAssemblies)
		{
			if (!SubSection || !IsFullRangeSection(SubSection) || !SubSection->HasStartFrame())
			{
				continue;
			}

			const FFrameNumber OuterStart = SubSection->GetInclusiveStartFrame();
			if (InEndFrame <= OuterStart)
			{
				continue;
			}
			SubSection->SetEndFrame(InEndFrame);

			if (UMovieSceneSequence* Sequence = SubSection->GetSequence())
			{
				if (UMovieScene* InnerMovieScene = Sequence->GetMovieScene())
				{
					const TRange<FFrameNumber> InnerRange = InnerMovieScene->GetPlaybackRange();
					const FFrameNumber InnerStart = InnerRange.HasLowerBound() ? InnerRange.GetLowerBoundValue() : FFrameNumber(0);
					const FFrameNumber InnerEnd = InnerStart + (InEndFrame - OuterStart).Value;
					InnerMovieScene->SetPlaybackRange(TRange<FFrameNumber>(InnerStart, InnerEnd));
				}
			}
		}
	}

	/**
	 * Recursively save the SubAssemblies and Associated Assets of the input Assembly (and mark SubAssemblies as read-only).
	 * Mirrors TakeRecorder's behavior of automatically locking and saving subsequences.
	 */
	void SaveRecordedAssets(UCineAssembly* InAssembly, UTakeRecorderSources* Sources)
	{
		const bool bSaveRecordedAssets = Sources ? Sources->GetSettings().bSaveRecordedAssets : false;
		const bool bRecordToPossessable = Sources ? Sources->GetSettings().bRecordToPossessable : false;

		// Save the associated assets of the recorded assembly
		if (bSaveRecordedAssets)
		{
			for (const FAssemblyAssociatedAssetDesc& AssetDesc : InAssembly->AssociatedAssets)
			{
				if (UObject* AssociatedAsset = AssetDesc.CreatedAsset.LoadSynchronous())
				{
					TakesUtils::SaveAsset(AssociatedAsset);
				}
			}
		}

		// Recursively mark every SubAssembly as read-only and save it
		for (UMovieSceneSubSection* SubSection : InAssembly->SubAssemblies)
		{
			UMovieSceneSequence* SubSequence = SubSection ? SubSection->GetSequence() : nullptr;
			if (!SubSequence)
			{
				continue;
			}

			if (UMovieScene* MovieScene = SubSequence->GetMovieScene())
			{
				MovieScene->SetReadOnly(true);
			}

			if (UCineAssembly* SubAssembly = Cast<UCineAssembly>(SubSequence))
			{
				SaveRecordedAssets(SubAssembly, Sources);
			}

			if (bSaveRecordedAssets)
			{
				TakesUtils::SaveAsset(SubSequence, bRecordToPossessable);
			}
		}
	}
}

FCineAssemblyTakeRecorderIntegration::FCineAssemblyTakeRecorderIntegration()
{
	// Register the custom CineAssembly settings with Take Recorder
	UCineAssemblyTakeRecorderSettings* Settings = GetMutableDefault<UCineAssemblyTakeRecorderSettings>();
	Settings->OnAssemblySchemaChanged().AddRaw(this, &FCineAssemblyTakeRecorderIntegration::OnAssemblySchemaSettingChanged);

	ITakeRecorderModule& TakeRecorderModule = ITakeRecorderModule::Get();
	TakeRecorderModule.RegisterSettingsObject(Settings);

	// Bind to the various TakeRecorder events that require intervention to properly apply the CineAssembly settings
	UTakeRecorder::OnRecordingInitialized().AddRaw(this, &FCineAssemblyTakeRecorderIntegration::OnRecordingInitialized);
	UTakePreset::OnTakePresetAllocated().AddRaw(this, &FCineAssemblyTakeRecorderIntegration::OnTakePresetAllocated);

	if (UTakeRecorderSubsystem* TakeRecorderSubsystem = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>())
	{
		TakeRecorderSubsystem->GetOnPendingTakeClearedEvent().AddRaw(this, &FCineAssemblyTakeRecorderIntegration::OnPendingTakeCleared);
	}
}

FCineAssemblyTakeRecorderIntegration::~FCineAssemblyTakeRecorderIntegration()
{
	UTakeRecorder::OnRecordingInitialized().RemoveAll(this);
	UTakePreset::OnTakePresetAllocated().RemoveAll(this);

	if (!IsEngineExitRequested())
	{
		if (UCineAssemblyTakeRecorderSettings* Settings = GetMutableDefault<UCineAssemblyTakeRecorderSettings>())
		{
			Settings->OnAssemblySchemaChanged().RemoveAll(this);
		}
	}

	if (GEngine)
	{
		if (UTakeRecorderSubsystem* TakeRecorderSubsystem = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>())
		{
			TakeRecorderSubsystem->GetOnPendingTakeClearedEvent().RemoveAll(this);
		}
	}

	if (PendingTakeLevelSequenceChangedHandle.IsValid())
	{
		if (UTakePreset* PendingTake = ITakeRecorderModule::Get().GetPendingTake())
		{
			PendingTake->RemoveOnLevelSequenceChanged(PendingTakeLevelSequenceChangedHandle);
		}
		PendingTakeLevelSequenceChangedHandle.Reset();
	}
}

void FCineAssemblyTakeRecorderIntegration::OnRecordingInitialized(UTakeRecorder* TakeRecorder)
{
	if (!TakeRecorder)
	{
		return;
	}

	TakeRecorder->OnRecordingStopped().AddRaw(this, &FCineAssemblyTakeRecorderIntegration::OnRecordingStopped);
	TakeRecorder->OnTickRecording().AddRaw(this, &FCineAssemblyTakeRecorderIntegration::OnTickRecording);

	UCineAssembly* Assembly = Cast<UCineAssembly>(TakeRecorder->GetSequence());
	if (!Assembly)
	{
		return;
	}

	FString AssemblyName;
	Assembly->GetName(AssemblyName);

	Assembly->AssemblyName.Template = AssemblyName;
	Assembly->AssemblyName.Resolved = FText::FromString(AssemblyName);

	Assembly->Level = FSoftObjectPath(TakesUtils::DiscoverSourceWorld());

	// Set the production of this recorded subsequence to the current active production
	const UProductionSettings* ProductionSettings = GetDefault<UProductionSettings>();
	TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
	if (ActiveProduction.IsSet())
	{
		Assembly->Production = ActiveProduction->ProductionID;
		Assembly->ProductionName = ActiveProduction->ProductionName;
	}

	const UCineAssemblyTakeRecorderSettings* SchemaSettings = GetDefault<UCineAssemblyTakeRecorderSettings>();
	if (!SchemaSettings->AssemblySchema.IsNull())
	{
		FString CurrentAssemblyPath;
		Assembly->GetPathName(nullptr, CurrentAssemblyPath);
		CurrentAssemblyPath = FPaths::GetPath(CurrentAssemblyPath);

		// If the schema defines some additional default assembly path, move the assembly asset accordingly
		if (!Assembly->PathRelativeToRoot.Template.IsEmpty())
		{
			FString UniquePackageName;
			FString UniqueAssetName;
			UCineAssemblyFactory::MakeUniqueNameAndPath(Assembly, CurrentAssemblyPath, UniquePackageName, UniqueAssetName);

			UPackage* Package = CreatePackage(*UniquePackageName);
			Assembly->Rename(*UniqueAssetName, Package);
		}

		// Give this assembly its own SubAssembly tree so the pending take's references stay intact for the next take.
		Assembly->DuplicateManagedSubAssemblies();

		// These actions are normally applied by the CineAssembly Factory, but the Factory only operates on new transient CineAssemblies, not existing ones
		UCineAssemblyFactory::CreateSubFolders(Assembly);
		Assembly->ResolveMovieSceneTokens();
		UCineAssemblyFactory::CreateConfiguredSubAssemblies(Assembly, CurrentAssemblyPath);
		UCineAssemblyFactory::CreateAssociatedAssets(Assembly);
	}

	UE::CineAssemblyTools::Private::CollapseFullRangeSubSections(Assembly);

	UE::CineAssemblyToolsAnalytics::RecordEvent_RecordAssembly();
}

void FCineAssemblyTakeRecorderIntegration::OnTickRecording(UTakeRecorder* TakeRecorder, const FQualifiedFrameTime& CurrentFrameTime)
{
	if (!TakeRecorder)
	{
		return;
	}

	if (UCineAssembly* Assembly = Cast<UCineAssembly>(TakeRecorder->GetSequence()))
	{
		UE::CineAssemblyTools::Private::ExpandFullRangeSubSections(Assembly, CurrentFrameTime.Time.CeilToFrame());
	}
}

void FCineAssemblyTakeRecorderIntegration::OnRecordingStopped(UTakeRecorder* TakeRecorder)
{
	if (!TakeRecorder)
	{
		return;
	}

	if (UCineAssembly* Assembly = Cast<UCineAssembly>(TakeRecorder->GetSequence()))
	{
		UTakeRecorderSources* Sources = Assembly->FindMetaData<UTakeRecorderSources>();
		if (!Sources)
		{
			UE_LOGF(LogCineAssemblyTakeRecorderIntegration, Error, "Could not determine how to save SubAssembly assets %ls.", *Assembly->GetFName().ToString());
		}
		UE::CineAssemblyTools::Private::SaveRecordedAssets(Assembly, Sources);
	}

	TakeRecorder->OnRecordingStopped().RemoveAll(this);
	TakeRecorder->OnTickRecording().RemoveAll(this);
}

void FCineAssemblyTakeRecorderIntegration::OnAssemblySchemaSettingChanged()
{
	using namespace UE::CineAssemblyTools::Private;

	UTakePreset* PendingTake = ITakeRecorderModule::Get().GetPendingTake();
	UCineAssembly* PendingAssembly = PendingTake ? Cast<UCineAssembly>(PendingTake->GetLevelSequence()) : nullptr;
	if (!PendingAssembly)
	{
		return;
	}

	// If the AssemblySchema property is valid, apply the contents of the selected schema to the Pending Assembly. 
	// Otherwise, remove the existing schema content and reset the schema of the pending assembly.
	const UCineAssemblyTakeRecorderSettings* Settings = GetDefault<UCineAssemblyTakeRecorderSettings>();
	if (UCineAssemblySchema* Schema = Settings->AssemblySchema.LoadSynchronous())
	{
		ApplySchemaToPendingTake(PendingAssembly, Schema);
	}
	else
	{
		RemoveSchemaContent(PendingAssembly);

		constexpr bool bDuplicateMovieScene = false;
		PendingAssembly->InitializeFromSchema(nullptr, bDuplicateMovieScene);

		if (TSharedPtr<ISequencer> Sequencer = FindSequencer(PendingAssembly))
		{
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
		}
	}
}

void FCineAssemblyTakeRecorderIntegration::OnTakePresetAllocated()
{
	UTakePreset* PendingTake = ITakeRecorderModule::Get().GetPendingTake();
	if (!PendingTake)
	{
		return;
	}

	// Register a callback to be notified when the pending take's level sequence changes so we can apply the Assembly Schema to it
	if (!PendingTakeLevelSequenceChangedHandle.IsValid())
	{
		PendingTakeLevelSequenceChangedHandle = PendingTake->AddOnLevelSequenceChanged(FSimpleDelegate::CreateRaw(this, &FCineAssemblyTakeRecorderIntegration::OnPendingTakeLevelSequenceChanged));
	}

	// Apply the Assembly Schema now since the pending take was just freshly allocated
	OnPendingTakeLevelSequenceChanged();
}

void FCineAssemblyTakeRecorderIntegration::OnPendingTakeLevelSequenceChanged()
{
	using namespace UE::CineAssemblyTools::Private;

	UTakePreset* PendingTake = ITakeRecorderModule::Get().GetPendingTake();
	UCineAssembly* PendingAssembly = PendingTake ? Cast<UCineAssembly>(PendingTake->GetLevelSequence()) : nullptr;
	if (!PendingAssembly)
	{
		return;
	}

	const UCineAssemblyTakeRecorderSettings* Settings = GetDefault<UCineAssemblyTakeRecorderSettings>();
	if (UCineAssemblySchema* Schema = Settings->AssemblySchema.LoadSynchronous())
	{
		ApplySchemaToPendingTake(PendingAssembly, Schema);
	}
}

void FCineAssemblyTakeRecorderIntegration::OnPendingTakeCleared()
{
	// Set the AssemblySchema property to None, which will subsequently trigger the schema content to be cleared from the pending take (and not re-applied)
	// This keeps our take recorder settings consistent with what's actually in the pending take, rather than displaying an applied schema when we have no schema content being recorded.
	UCineAssemblyTakeRecorderSettings* Settings = GetMutableDefault<UCineAssemblyTakeRecorderSettings>();
	if (!Settings->AssemblySchema.IsNull())
	{
		Settings->AssemblySchema.Reset();

		FProperty* SchemaProperty = UCineAssemblyTakeRecorderSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UCineAssemblyTakeRecorderSettings, AssemblySchema));
		if (SchemaProperty)
		{
			FPropertyChangedEvent PropertyChangedEvent(SchemaProperty, EPropertyChangeType::ValueSet);
			Settings->PostEditChangeProperty(PropertyChangedEvent);
		}
	}
}
