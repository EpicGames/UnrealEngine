// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Templates/UnrealTemplate.h"

class UTakePreset;
class UTakeRecorder;
struct FQualifiedFrameTime;

/** Manages Cine Assembly Tools interactions with Take Recorder */
class FCineAssemblyTakeRecorderIntegration : public FNoncopyable
{
public:
	FCineAssemblyTakeRecorderIntegration();
	~FCineAssemblyTakeRecorderIntegration();

private:
	/** Bind to TakeRecorder's OnTickRecording and OnRecordingStopped, and complete the pending assembly configuration. */
	void OnRecordingInitialized(UTakeRecorder* TakeRecorder);

	/** Expand full-range SubSections to the current recording frame. */
	void OnTickRecording(UTakeRecorder* TakeRecorder, const FQualifiedFrameTime& CurrentFrameTime);

	/** Save the SubAssembly and Associated Assets when recording completes. */
	void OnRecordingStopped(UTakeRecorder* TakeRecorder);

	/** Apply the AssemblySchema to the pending take. */
	void OnAssemblySchemaSettingChanged();

	/** Bind to the allocated TakePreset's OnLevelSequenceChanged event to apply the schema content to the pending take. */
	void OnTakePresetAllocated();

	/** Apply the configured schema when the pending take's level sequence is recreated. */
	void OnPendingTakeLevelSequenceChanged();

	/** Null the AssemblySchema setting so that the cleared pending take does not immediately repopulate the take with schema content. */
	void OnPendingTakeCleared();

private:
	/** Subscription handle on the pending take's per-instance OnLevelSequenceChanged event. */
	FDelegateHandle PendingTakeLevelSequenceChangedHandle;
};
