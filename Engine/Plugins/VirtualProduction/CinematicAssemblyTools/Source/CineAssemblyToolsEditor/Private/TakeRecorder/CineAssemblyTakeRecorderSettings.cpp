// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblyTakeRecorderSettings.h"

#include "CineAssembly.h"
#include "TakePresetSettings.h"

void UCineAssemblyTakeRecorderSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UObject::PostEditChangeProperty(PropertyChangedEvent);
	SaveConfig();

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UCineAssemblyTakeRecorderSettings, AssemblySchema))
	{
		OnAssemblySchemaChangedDelegate.Broadcast();
	}
}

bool UCineAssemblyTakeRecorderSettings::CanEditAssemblySchema() const
{
	return UTakePresetSettings::Get()->GetTargetRecordClass() == UCineAssembly::StaticClass();
}
