// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "CineAssemblySchema.h"

#include "CineAssemblyTakeRecorderSettings.generated.h"

DECLARE_MULTICAST_DELEGATE(FOnAssemblySchemaChanged);

UCLASS(Config=EditorPerProjectUserSettings, DisplayName = "Take Recorder")
class UCineAssemblyTakeRecorderSettings : public UObject
{
	GENERATED_BODY()

public:
	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	/** Returns true if the Take Preset settings TargetRecordClass is set to CineAssembly */
	UFUNCTION()
	bool CanEditAssemblySchema() const;

	/** Multicast delegate fired whenever the AssemblySchema property is edited */
	FOnAssemblySchemaChanged& OnAssemblySchemaChanged() { return OnAssemblySchemaChangedDelegate; }

public:
	/** The Cine Assembly Schema to use as the base for the recorded Assembly */
	UPROPERTY(Config, EditAnywhere, Category = "Cine Assembly", meta=(EditCondition = "CanEditAssemblySchema", EditConditionHides))
	TSoftObjectPtr<UCineAssemblySchema> AssemblySchema;

private:
	FOnAssemblySchemaChanged OnAssemblySchemaChangedDelegate;
};
