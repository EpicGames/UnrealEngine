// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineAssemblyMetadataWidgets.h"

#include "Algo/Find.h"
#include "AssetRegistry/AssetData.h"
#include "CineAssembly.h"
#include "CineAssemblyNamingTokens.h"
#include "CineAssemblySchema.h"
#include "MovieSceneSubAssemblySection.h"
#include "PropertyCustomizationHelpers.h"
#include "SNamingTokensEditableTextBox.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CineAssemblyMetadataWidgets"

namespace UE::CineAssemblyTools::Private
{
	/** Applies an override for the given key on a section, or removes the entry if the new value matches the default (keeps the map sparse). */
	void ApplySectionOverride(UMovieSceneSubAssemblySection* Section, const FString& Key, const FString& NewStringValue, const FString& DefaultStringValue)
	{
		Section->Modify();
		if (NewStringValue == DefaultStringValue)
		{
			Section->MetadataOverrides.Remove(Key);
		}
		else
		{
			Section->MetadataOverrides.Add(Key, NewStringValue);
		}
	}

	/** Reads the current override value for the given key on a section, or the descriptor default if no override is set. */
	FString GetOverrideOrDefaultString(UMovieSceneSubAssemblySection* Section, const FAssemblyMetadataDesc& MetadataDesc)
	{
		const FString DefaultValue = MetadataDesc.DefaultValue.IsType<FString>() ? MetadataDesc.DefaultValue.Get<FString>() : FString();
		if (const FString* Override = Section->MetadataOverrides.Find(MetadataDesc.Key))
		{
			return *Override;
		}
		return DefaultValue;
	}

	/** Builds a String-typed value widget bound to the Assembly's metadata, choosing SEditableTextBox or SNamingTokensEditableTextBox based on the descriptor's bEvaluateTokens flag. */
	TSharedRef<SWidget> MakeStringValueWidget(UCineAssembly* Assembly, const FAssemblyMetadataDesc& MetadataDesc)
	{
		const FString Key = MetadataDesc.Key;

		if (!MetadataDesc.bEvaluateTokens)
		{
			return SNew(SEditableTextBox)
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.ClearKeyboardFocusOnCommit(false)
				.Text_Lambda([Assembly, Key]() -> FText
					{
						FString Value;
						Assembly->GetMetadataAsString(Key, Value);
						return FText::FromString(Value);
					})
				.OnTextCommitted_Lambda([Assembly, Key](const FText& InText, ETextCommit::Type)
					{
						Assembly->SetMetadataAsString(Key, InText.ToString());
					});
		}

		// Naming Tokens context pointer is captured by value to ensure it does not get GC'd during the widget's lifetime
		TStrongObjectPtr<UCineAssemblyNamingTokensContext> NamingTokenContext(NewObject<UCineAssemblyNamingTokensContext>());
		NamingTokenContext->Assembly = Assembly;

		FNamingTokenFilterArgs FilterArgs;
		FilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

		return SNew(SNamingTokensEditableTextBox)
			.Contexts({ NamingTokenContext.Get() })
			.FilterArgs(FilterArgs)
			.EvaluationFrequency(1.0f)
			.ShouldEvaluateTokens(true)
			.ShowUnsetTokenWarning(true)
			.Text_Lambda([Assembly, Key, NamingTokenContext]() -> FText
				{
					FTemplateString Value;
					Assembly->GetMetadataAsTokenString(Key, Value);
					return FText::FromString(Value.Template);
				})
			.OnTextCommitted_Lambda([Assembly, Key, NamingTokenContext](const FText& InText, ETextCommit::Type)
				{
					FTemplateString TemplateString;
					Assembly->GetMetadataAsTokenString(Key, TemplateString);
					TemplateString.Template = InText.ToString();
					Assembly->SetMetadataAsTokenString(Key, TemplateString);
				})
			.OnTokenizedTextEvaluated_Lambda([Assembly, Key, NamingTokenContext](const FText& InText)
				{
					FTemplateString TemplateString;
					Assembly->GetMetadataAsTokenString(Key, TemplateString);
					TemplateString.Resolved = InText;
					Assembly->SetMetadataAsTokenString(Key, TemplateString);
				});
	}

	/** Builds a value widget bound to the Assembly's metadata for the given descriptor. Reads via Get*, writes via Set*. */
	TSharedRef<SWidget> MakeMetadataValueWidget(UCineAssembly* Assembly, const FAssemblyMetadataDesc& MetadataDesc)
	{
		const FString Key = MetadataDesc.Key;

		switch (MetadataDesc.Type)
		{
		case ECineAssemblyMetadataType::String:
			return MakeStringValueWidget(Assembly, MetadataDesc);

		case ECineAssemblyMetadataType::AssetPath:
			return SNew(SObjectPropertyEntryBox)
				.AllowedClass(!MetadataDesc.AssetClass.IsNull() ? MetadataDesc.AssetClass.ResolveClass() : UObject::StaticClass())
				.AllowCreate(true)
				.ObjectPath_Lambda([Assembly, Key]() -> FString
					{
						FString Value;
						Assembly->GetMetadataAsString(Key, Value);
						return Value;
					})
				.OnObjectChanged_Lambda([Assembly, Key](const FAssetData& InAssetData)
					{
						Assembly->SetMetadataAsString(Key, InAssetData.GetObjectPathString());
					});

		case ECineAssemblyMetadataType::CineAssembly:
			return SNew(SObjectPropertyEntryBox)
				.AllowedClass(UCineAssembly::StaticClass())
				.AllowCreate(true)
				.OnShouldFilterAsset_Lambda([SchemaType = MetadataDesc.SchemaType](const FAssetData& InAssetData)
					{
						if (!SchemaType.IsValid())
						{
							return false;
						}
						const FAssetDataTagMapSharedView::FFindTagResult AssemblyType = InAssetData.TagsAndValues.FindTag(UCineAssembly::AssetRegistryTag_AssemblyType);
						if (AssemblyType.IsSet())
						{
							return !AssemblyType.GetValue().Equals(SchemaType.GetAssetName());
						}
						return true;
					})
				.ObjectPath_Lambda([Assembly, Key]() -> FString
					{
						FString Value;
						Assembly->GetMetadataAsString(Key, Value);
						return Value;
					})
				.OnObjectChanged_Lambda([Assembly, Key](const FAssetData& InAssetData)
					{
						Assembly->SetMetadataAsString(Key, InAssetData.GetObjectPathString());
					});

		case ECineAssemblyMetadataType::Bool:
			return SNew(SCheckBox)
				.IsChecked_Lambda([Assembly, Key]() -> ECheckBoxState
					{
						bool bValue = false;
						Assembly->GetMetadataAsBool(Key, bValue);
						return bValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([Assembly, Key](ECheckBoxState CheckBoxState)
					{
						Assembly->SetMetadataAsBool(Key, CheckBoxState == ECheckBoxState::Checked);
					});

		case ECineAssemblyMetadataType::Integer:
			return SNew(SNumericEntryBox<int32>)
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.Value_Lambda([Assembly, Key]() -> int32
					{
						int32 Value = 0;
						Assembly->GetMetadataAsInteger(Key, Value);
						return Value;
					})
				.OnValueCommitted_Lambda([Assembly, Key](int32 InValue, ETextCommit::Type)
					{
						Assembly->SetMetadataAsInteger(Key, InValue);
					});

		case ECineAssemblyMetadataType::Float:
			return SNew(SNumericEntryBox<float>)
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.Value_Lambda([Assembly, Key]() -> float
					{
						float Value = 0.0f;
						Assembly->GetMetadataAsFloat(Key, Value);
						return Value;
					})
				.OnValueCommitted_Lambda([Assembly, Key](float InValue, ETextCommit::Type)
					{
						Assembly->SetMetadataAsFloat(Key, InValue);
					});

		default:
			checkNoEntry();
		}

		return SNullWidget::NullWidget;
	}

	/** Builds a value widget bound to a SubAssemblySections's metadata override for the given descriptor. */
	TSharedRef<SWidget> MakeMetadataOverrideWidget(UMovieSceneSubAssemblySection* Section, const FAssemblyMetadataDesc& MetadataDesc)
	{
		switch (MetadataDesc.Type)
		{
		case ECineAssemblyMetadataType::String:
			return SNew(SEditableTextBox)
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.ClearKeyboardFocusOnCommit(false)
				.Text_Lambda([Section, MetadataDesc]() -> FText
					{
						return FText::FromString(GetOverrideOrDefaultString(Section, MetadataDesc));
					})
				.OnTextCommitted_Lambda([Section, MetadataDesc](const FText& InText, ETextCommit::Type)
					{
						const FString DefaultValue = MetadataDesc.DefaultValue.IsType<FString>() ? MetadataDesc.DefaultValue.Get<FString>() : FString();
						ApplySectionOverride(Section, MetadataDesc.Key, InText.ToString(), DefaultValue);
					});

		case ECineAssemblyMetadataType::AssetPath:
			return SNew(SObjectPropertyEntryBox)
				.AllowedClass(!MetadataDesc.AssetClass.IsNull() ? MetadataDesc.AssetClass.ResolveClass() : UObject::StaticClass())
				.AllowCreate(false)
				.ObjectPath_Lambda([Section, MetadataDesc]() -> FString
					{
						return GetOverrideOrDefaultString(Section, MetadataDesc);
					})
				.OnObjectChanged_Lambda([Section, MetadataDesc](const FAssetData& InAssetData)
					{
						const FString DefaultValue = MetadataDesc.DefaultValue.IsType<FString>() ? MetadataDesc.DefaultValue.Get<FString>() : FString();
						ApplySectionOverride(Section, MetadataDesc.Key, InAssetData.GetObjectPathString(), DefaultValue);
					});

		case ECineAssemblyMetadataType::CineAssembly:
			return SNew(SObjectPropertyEntryBox)
				.AllowedClass(UCineAssembly::StaticClass())
				.AllowCreate(false)
				.OnShouldFilterAsset_Lambda([SchemaType = MetadataDesc.SchemaType](const FAssetData& InAssetData)
					{
						if (!SchemaType.IsValid())
						{
							return false;
						}
						const FAssetDataTagMapSharedView::FFindTagResult AssemblyType = InAssetData.TagsAndValues.FindTag(UCineAssembly::AssetRegistryTag_AssemblyType);
						if (AssemblyType.IsSet())
						{
							return !AssemblyType.GetValue().Equals(SchemaType.GetAssetName());
						}
						return true;
					})
				.ObjectPath_Lambda([Section, MetadataDesc]() -> FString
					{
						return GetOverrideOrDefaultString(Section, MetadataDesc);
					})
				.OnObjectChanged_Lambda([Section, MetadataDesc](const FAssetData& InAssetData)
					{
						const FString DefaultValue = MetadataDesc.DefaultValue.IsType<FString>() ? MetadataDesc.DefaultValue.Get<FString>() : FString();
						ApplySectionOverride(Section, MetadataDesc.Key, InAssetData.GetObjectPathString(), DefaultValue);
					});

		case ECineAssemblyMetadataType::Bool:
			return SNew(SCheckBox)
				.IsChecked_Lambda([Section, MetadataDesc]() -> ECheckBoxState
					{
						const bool DefaultValue = MetadataDesc.DefaultValue.IsType<bool>() ? MetadataDesc.DefaultValue.Get<bool>() : false;

						bool bValue = DefaultValue;
						if (const FString* Override = Section->MetadataOverrides.Find(MetadataDesc.Key))
						{
							bValue = Override->ToBool();
						}
						return bValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([Section, MetadataDesc](ECheckBoxState CheckBoxState)
					{
						const bool DefaultValue = MetadataDesc.DefaultValue.IsType<bool>() ? MetadataDesc.DefaultValue.Get<bool>() : false;
						const bool bNewValue = (CheckBoxState == ECheckBoxState::Checked);
						ApplySectionOverride(Section, MetadataDesc.Key, bNewValue ? TEXT("true") : TEXT("false"), DefaultValue ? TEXT("true") : TEXT("false"));
					});

		case ECineAssemblyMetadataType::Integer:
			return SNew(SNumericEntryBox<int32>)
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.Value_Lambda([Section, MetadataDesc]() -> int32
					{
						const int32 DefaultValue = MetadataDesc.DefaultValue.IsType<int32>() ? MetadataDesc.DefaultValue.Get<int32>() : 0;
						if (const FString* Override = Section->MetadataOverrides.Find(MetadataDesc.Key))
						{
							return FCString::Atoi(**Override);
						}
						return DefaultValue;
					})
				.OnValueCommitted_Lambda([Section, MetadataDesc](int32 InValue, ETextCommit::Type)
					{
						const int32 DefaultValue = MetadataDesc.DefaultValue.IsType<int32>() ? MetadataDesc.DefaultValue.Get<int32>() : 0;
						ApplySectionOverride(Section, MetadataDesc.Key, LexToString(InValue), LexToString(DefaultValue));
					});

		case ECineAssemblyMetadataType::Float:
			return SNew(SNumericEntryBox<float>)
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.Value_Lambda([Section, MetadataDesc]() -> float
					{
						const float DefaultValue = MetadataDesc.DefaultValue.IsType<float>() ? MetadataDesc.DefaultValue.Get<float>() : 0.0f;
						if (const FString* Override = Section->MetadataOverrides.Find(MetadataDesc.Key))
						{
							return FCString::Atof(**Override);
						}
						return DefaultValue;
					})
				.OnValueCommitted_Lambda([Section, MetadataDesc](float InValue, ETextCommit::Type)
					{
						const float DefaultValue = MetadataDesc.DefaultValue.IsType<float>() ? MetadataDesc.DefaultValue.Get<float>() : 0.0f;
						ApplySectionOverride(Section, MetadataDesc.Key, LexToString(InValue), LexToString(DefaultValue));
					});

		default:
			checkNoEntry();
		}

		return SNullWidget::NullWidget;
	}

	/** Builds a single key/value row using the shared splitter layout used throughout the Edit panel. */
	TSharedRef<SWidget> MakeEditRow(const FText& KeyText, TSharedRef<SWidget> ValueWidget, const TSharedRef<float>& ColumnRatio)
	{
		return SNew(SBorder)
			.BorderBackgroundColor(FStyleColors::Recessed)
			.Padding(0.0f)
			[
				SNew(SSplitter)
				.PhysicalSplitterHandleSize(1.0f)

				+ SSplitter::Slot()
					.Value_Lambda([ColumnRatio]() { return *ColumnRatio; })
					.OnSlotResized_Lambda([ColumnRatio](float NewRatio) { *ColumnRatio = NewRatio; })
					[
						SNew(SBox)
							.VAlign(VAlign_Center)
							.Padding(FMargin(10.0f, 3.0f))
							[
								SNew(STextBlock)
									.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
									.Text(KeyText)
							]
					]

				+ SSplitter::Slot()
					.Value_Lambda([ColumnRatio]() { return 1.0f - *ColumnRatio; })
					[
						SNew(SBox)
							.VAlign(VAlign_Center)
							.Padding(FMargin(12.0f, 3.0f, 10.0f, 3.0f))
							[
								ValueWidget
							]
					]
			];
	}

	/** Builds a section to edit a label field */
	TSharedRef<SWidget> MakeLabelSection(const FText& InitialText, const TSharedRef<float>& ColumnRatio, TFunction<void(const FText&)> CommitLabel)
	{
		TSharedRef<SEditableTextBox> TextBox = SNew(SEditableTextBox)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
			.ClearKeyboardFocusOnCommit(false)
			.Text(InitialText)
			.OnTextChanged_Lambda([CommitLabel = MoveTemp(CommitLabel)](const FText& InText)
				{
					if (CommitLabel)
					{
						CommitLabel(InText);
					}
				});

		return SNew(SExpandableArea)
			.InitiallyCollapsed(false)
			.AreaTitle(LOCTEXT("LabelArea", "Label"))
			.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
			.BodyContent()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
					.AutoHeight()
					[
						MakeEditRow(LOCTEXT("LabelRowKey", "Label"), TextBox, ColumnRatio)
					]
			];
	}

	/** Builds a Metadata section with edit widgets for each of the schema's metadata fields */
	TSharedRef<SWidget> MakeMetadataSection(const UCineAssemblySchema* Schema, const TSharedRef<float>& ColumnRatio, TFunctionRef<TSharedRef<SWidget>(const FAssemblyMetadataDesc&)> BuildMetadataWidget)
	{
		TSharedRef<SVerticalBox> MetadataRows = SNew(SVerticalBox);
		for (const FAssemblyMetadataDesc& MetadataDesc : Schema->AssemblyMetadata)
		{
			TSharedRef<SWidget> MetadataWidget = BuildMetadataWidget(MetadataDesc);

			MetadataRows->AddSlot()
				.AutoHeight()
				[
					MakeEditRow(FText::FromString(MetadataDesc.Key), MetadataWidget, ColumnRatio)
				];
		}

		return SNew(SExpandableArea)
			.InitiallyCollapsed(false)
			.AreaTitle(LOCTEXT("MetadataArea", "Metadata"))
			.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
			.BodyContent()
			[
				MetadataRows
			];
	}
}

TSharedRef<SWidget> FCineAssemblyMetadataWidgets::MakeEditMenuForAssembly(UCineAssembly* InAssembly)
{
	using namespace UE::CineAssemblyTools::Private;

	if (!InAssembly)
	{
		return SNullWidget::NullWidget;
	}

	// Shared column ratio so the splitter handle on every row moves together
	const TSharedRef<float> ColumnRatio = MakeShared<float>(0.4f);

	// Define how label changes are saved
	TWeakObjectPtr<UCineAssembly> WeakAssembly = InAssembly;
	auto OnLabelTextChanged = [WeakAssembly](const FText& InText)
		{
			if (UCineAssembly* Assembly = WeakAssembly.Get())
			{
				Assembly->SetLabel(InText.IsEmptyOrWhitespace() ? NAME_None : FName(*InText.ToString()));
			}
		};

	// Add a section to the edit menu to edit the Assembly label
	TSharedRef<SVerticalBox> EditSections = SNew(SVerticalBox);
	EditSections->AddSlot()
		.AutoHeight()
		[
			MakeLabelSection(FText::FromName(InAssembly->GetLabel()), ColumnRatio, OnLabelTextChanged)
		];

	// Create the Metadata section with value widgets for each of the input Assembly's metadata fields (derived from its Schema)
	const UCineAssemblySchema* Schema = InAssembly->GetSchema();
	if (Schema && !Schema->AssemblyMetadata.IsEmpty())
	{
		auto MakeMetadataWidgetLambda = [InAssembly](const FAssemblyMetadataDesc& MetadataDesc) { return MakeMetadataValueWidget(InAssembly, MetadataDesc); };

		EditSections->AddSlot()
			.AutoHeight()
			[
				MakeMetadataSection(Schema, ColumnRatio, MakeMetadataWidgetLambda)
			];
	}

	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(0.0f)
		[
			SNew(SBox)
				.MinDesiredWidth(400.0f)
				.MaxDesiredHeight(400.0f)
				[
					SNew(SScrollBox)
					+ SScrollBox::Slot()
					[
						EditSections
					]
				]
		];
}

TSharedRef<SWidget> FCineAssemblyMetadataWidgets::MakeEditMenuForSection(UMovieSceneSubAssemblySection* InSubAssemblySection)
{
	using namespace UE::CineAssemblyTools::Private;

	if (!InSubAssemblySection)
	{
		return SNullWidget::NullWidget;
	}

	// Shared column ratio so the splitter handle on every row moves together
	const TSharedRef<float> ColumnRatio = MakeShared<float>(0.4f);

	// Define how label changes are saved
	TWeakObjectPtr<UMovieSceneSubAssemblySection> WeakSection = InSubAssemblySection;
	auto OnLabelTextChanged = [WeakSection](const FText& InText)
		{
			if (UMovieSceneSubAssemblySection* SubAssemblySection = WeakSection.Get())
			{
				SubAssemblySection->Modify();
				SubAssemblySection->Label = InText.IsEmptyOrWhitespace() ? NAME_None : FName(*InText.ToString());
			}
		};

	// Add a section to the edit menu to edit the SubAssembly label
	TSharedRef<SVerticalBox> EditSections = SNew(SVerticalBox);
	EditSections->AddSlot()
		.AutoHeight()
		[
			MakeLabelSection(FText::FromName(InSubAssemblySection->Label), ColumnRatio, OnLabelTextChanged)
		];

	// Create the Metadata section with value widgets for each of the Section's metadata override fields (derived from the template schema)
	const UCineAssemblySchema* Schema = InSubAssemblySection->GetTemplateSchema();
	if (Schema && !Schema->AssemblyMetadata.IsEmpty())
	{
		auto MakeMetadataWidgetLambda = [InSubAssemblySection](const FAssemblyMetadataDesc& MetadataDesc) { return MakeMetadataOverrideWidget(InSubAssemblySection, MetadataDesc); };

		EditSections->AddSlot()
			.AutoHeight()
			[
				MakeMetadataSection(Schema, ColumnRatio, MakeMetadataWidgetLambda)
			];
	}

	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(0.0f)
		[
			SNew(SBox)
				.MinDesiredWidth(400.0f)
				.MaxDesiredHeight(400.0f)
				[
					SNew(SScrollBox)
					+ SScrollBox::Slot()
					[
						EditSections
					]
				]
		];
}

TSharedRef<SWidget> FCineAssemblyMetadataWidgets::MakeEditMenuForAssociatedAsset(UCineAssembly* InAssembly, const FGuid& AssetID)
{
	using namespace UE::CineAssemblyTools::Private;

	if (!InAssembly)
	{
		return SNullWidget::NullWidget;
	}

	const FAssemblyAssociatedAssetDesc* AssetDesc = Algo::FindBy(InAssembly->AssociatedAssets, AssetID, &FAssemblyAssociatedAssetDesc::AssetID);
	if (!AssetDesc)
	{
		return SNullWidget::NullWidget;
	}

	const TSharedRef<float> ColumnRatio = MakeShared<float>(0.4f);

	// Define how label changes are saved
	TWeakObjectPtr<UCineAssembly> WeakAssembly = InAssembly;
	auto OnLabelTextChanged = [WeakAssembly, AssetID](const FText& InText)
		{
			if (UCineAssembly* Assembly = WeakAssembly.Get())
			{
				if (FAssemblyAssociatedAssetDesc* AssetDesc = Algo::FindBy(Assembly->AssociatedAssets, AssetID, &FAssemblyAssociatedAssetDesc::AssetID))
				{
					Assembly->Modify();
					AssetDesc->Label = InText.IsEmptyOrWhitespace() ? NAME_None : FName(*InText.ToString());
				}
			}
		};

	// Add a section to the edit menu to edit the Associated Asset label
	TSharedRef<SVerticalBox> EditSections = SNew(SVerticalBox);
	EditSections->AddSlot()
		.AutoHeight()
		[
			MakeLabelSection(FText::FromName(AssetDesc->Label), ColumnRatio, OnLabelTextChanged)
		];

	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(0.0f)
		[
			SNew(SBox)
				.MinDesiredWidth(400.0f)
				.MaxDesiredHeight(400.0f)
				[
					SNew(SScrollBox)
					+ SScrollBox::Slot()
					[
						EditSections
					]
				]
		];
}

#undef LOCTEXT_NAMESPACE
