// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class SWidget;
class UCineAssembly;
class UMovieSceneSubAssemblySection;
struct FGuid;

struct FCineAssemblyMetadataWidgets
{
	/** Edit panel for a UCineAssembly to edit its label and metadata */
	static TSharedRef<SWidget> MakeEditMenuForAssembly(UCineAssembly* InAssembly);

	/** Edit panel for a SubAssembly Section to edit its label and metadata overrides */
	static TSharedRef<SWidget> MakeEditMenuForSection(UMovieSceneSubAssemblySection* InSubAssemblySection);

	/** Edit panel for an Associated Asset descriptor identified by AssetID on the owning assembly */
	static TSharedRef<SWidget> MakeEditMenuForAssociatedAsset(UCineAssembly* InAssembly, const FGuid& AssetID);
};
