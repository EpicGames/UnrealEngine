// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCineAssemblyAssetTreeView.h"

#include "Algo/Contains.h"
#include "CineAssembly.h"
#include "CineAssemblyMetadataWidgets.h"
#include "CineAssemblyNamingTokens.h"
#include "CineAssemblyToolsStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneSubAssemblySection.h"
#include "MovieSceneSubAssemblyTrack.h"
#include "PropertyCustomizationHelpers.h"
#include "Styling/SlateIconFinder.h"
#include "UI/SAssemblyMetadataLink.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SCineAssemblyAssetTreeView"

struct ICineAssemblyTreeItem : TSharedFromThis<ICineAssemblyTreeItem>
{
	virtual ~ICineAssemblyTreeItem() {}

	virtual TSharedPtr<FCineAssemblyFolderItem> AsFolder() { return nullptr; }
	virtual TSharedPtr<FSubAssemblyItem> AsSubAssembly() { return nullptr; }
	virtual TSharedPtr<FSubAssemblySectionItem> AsSubAssemblySection() { return nullptr; }
	virtual TSharedPtr<FAssociatedAssetItem> AsAssociatedAsset() { return nullptr; }

	/** Returns true if this item should never be editable, regardless of the widget's read-only state */
	virtual bool IsReadOnly() const { return false; }

	/** Returns true if this item can be dragged to a new location in the tree */
	virtual bool CanDrag() const { return true; }

	/** Returns the assembly associated with this tree item */
	UCineAssembly* GetAssembly() { return AssociatedAssembly.Get(); }

	/** Sets the assembly associated with this tree item */
	void SetAssembly(UCineAssembly* InAssembly) { AssociatedAssembly = InAssembly; }

	/** Whether this item can accept an item being dropped onto it */
	virtual TOptional<EItemDropZone> CanAcceptDrop() const { return TOptional<EItemDropZone>(); }

	/** Return the name of the tree item */
	virtual FString GetName() const = 0;

	/** Set the name of the tree item */
	virtual void SetName(const FString& InName) = 0;

	/** Return the display name of the tree item */
	virtual FText GetDisplayName() const = 0;

	/** Set the display name of the tree item */
	virtual void SetDisplayName(const FText& InDisplayName) = 0;

	/** Return the path of the tree item (relative to the root folder) */
	virtual FString GetPath() const = 0;

	/** Set the path of the tree item (relative to the root folder) */
	virtual void SetPath(const FString& InPath) = 0;

	/** The parent folder of this item */
	TSharedPtr<FCineAssemblyFolderItem> ParentFolder;

	/** The text widget that displays this item's name and supports renaming */
	TSharedPtr<SNamingTokensEditableTextBox> NameWidget;

	/** The assembly that owns or is represented by this tree item */
	TWeakObjectPtr<UCineAssembly> AssociatedAssembly;

	/** Context object used to evaluate naming tokens displayed in this item's name widget */
	TStrongObjectPtr<UCineAssemblyNamingTokensContext> NamingTokenContext;
};

struct FSubAssemblyItem : ICineAssemblyTreeItem
{
	virtual TSharedPtr<FSubAssemblyItem> AsSubAssembly() override { return SharedThis(this); }

	//~ Begin ICineAssemblyTreeItem Interface
	virtual FString GetName() const override
	{
		return AssociatedAssembly.IsValid() ? AssociatedAssembly->AssemblyName.Template : FString();
	}

	virtual FText GetDisplayName() const override
	{
		return AssociatedAssembly.IsValid() ? FText::FromString(AssociatedAssembly->AssemblyName.Template) : FText();
	}

	virtual FString GetPath() const override
	{
		return AssociatedAssembly.IsValid() ? AssociatedAssembly->PathRelativeToRoot.Template : FString();
	}

	virtual void SetName(const FString& InName) override { ensureAlways(false); }
	virtual void SetDisplayName(const FText& InDisplayName) override { ensureAlways(false); }
	virtual void SetPath(const FString& InPath) override
	{
		if (AssociatedAssembly.IsValid())
		{
			AssociatedAssembly->PathRelativeToRoot.Template = FPaths::GetPath(InPath);
		}
	}
	virtual bool IsReadOnly() const override { return true; }
	//~ End ICineAssemblyTreeItem Interface
};

struct FSubAssemblySectionItem : ICineAssemblyTreeItem
{
	virtual TSharedPtr<FSubAssemblySectionItem> AsSubAssemblySection() override { return SharedThis(this); }

	//~ Begin ICineAssemblyTreeItem Interface
	virtual FString GetName() const override
	{
		return SubAssemblySection.IsValid() ? SubAssemblySection->GetSequenceName().ToString() : FString();
	}

	virtual void SetName(const FString& InName) override
	{
		if (SubAssemblySection.IsValid())
		{
			SubAssemblySection->SetSequenceName(FText::FromString(InName));
		}
	}

	virtual FText GetDisplayName() const override
	{
		return SubAssemblySection.IsValid() ? SubAssemblySection->GetSequenceName() : FText();
	}

	virtual void SetDisplayName(const FText& InDisplayName) override
	{
		if (SubAssemblySection.IsValid())
		{
			SubAssemblySection->SetSequenceName(InDisplayName);
		}
	}

	virtual FString GetPath() const override
	{
		return SubAssemblySection.IsValid() ? SubAssemblySection->GetSequencePath() : FString();
	}

	virtual void SetPath(const FString& InPath) override
	{
		if (SubAssemblySection.IsValid())
		{
			SubAssemblySection->SetSequencePath(FPaths::GetPath(InPath));
		}
	}
	//~ End ICineAssemblyTreeItem Interface

	/** The MovieScene section represented in the content tree by this asset item */
	TWeakObjectPtr<UMovieSceneSubAssemblySection> SubAssemblySection;
};

struct FAssociatedAssetItem : ICineAssemblyTreeItem
{
	virtual TSharedPtr<FAssociatedAssetItem> AsAssociatedAsset() override { return SharedThis(this); }

	/** Finds the associated asset definition by AssetID in the owning assembly's array */
	FAssemblyAssociatedAssetDesc* FindAssetDesc() const
	{
		if (AssociatedAssembly.IsValid())
		{
			return Algo::FindBy(AssociatedAssembly->AssociatedAssets, AssetID, &FAssemblyAssociatedAssetDesc::AssetID);
		}
		return nullptr;
	}

	//~ Begin ICineAssemblyTreeItem Interface
	virtual FString GetName() const override
	{
		if (const FAssemblyAssociatedAssetDesc* AssetDesc = FindAssetDesc())
		{
			return AssetDesc->AssetName.Template;
		}
		return FString();
	}

	virtual void SetName(const FString& InName) override
	{
		if (FAssemblyAssociatedAssetDesc* AssetDesc = FindAssetDesc())
		{
			AssetDesc->AssetName.Template = InName;
		}
	}

	virtual FText GetDisplayName() const override
	{
		return FText::FromString(GetName());
	}

	virtual void SetDisplayName(const FText& InDisplayName) override
	{
		SetName(InDisplayName.ToString());
	}

	virtual FString GetPath() const override
	{
		if (const FAssemblyAssociatedAssetDesc* AssetDesc = FindAssetDesc())
		{
			return AssetDesc->RelativePath.Template;
		}
		return FString();
	}

	virtual void SetPath(const FString& InPath) override
	{
		if (FAssemblyAssociatedAssetDesc* AssetDesc = FindAssetDesc())
		{
			AssetDesc->RelativePath.Template = FPaths::GetPath(InPath);
		}
	}
	//~ End ICineAssemblyTreeItem Interface

	/** The stable ID of the associated asset definition */
	FGuid AssetID;
};

struct FCineAssemblyFolderItem : ICineAssemblyTreeItem
{
	//~ Begin ICineAssemblyTreeItem Interface
	virtual TSharedPtr<FCineAssemblyFolderItem> AsFolder() override { return SharedThis(this); }
	virtual TOptional<EItemDropZone> CanAcceptDrop() const override { return EItemDropZone::OntoItem; }
	virtual bool IsReadOnly() const override { return ParentFolder == nullptr; }
	virtual bool CanDrag() const override { return ParentFolder != nullptr; }

	virtual FString GetName() const override
	{
		return FPaths::GetPathLeaf(FullPath);
	}

	virtual void SetName(const FString& InName) override
	{
		FullPath = FPaths::GetPath(FullPath) / InName;
	}

	virtual FText GetDisplayName() const override
	{
		return FText::FromString(GetName());
	}

	virtual void SetDisplayName(const FText& InDisplayName) override
	{
		FullPath = FPaths::GetPath(FullPath) / InDisplayName.ToString();
	}

	virtual FString GetPath() const override
	{
		return FullPath;
	}

	virtual void SetPath(const FString& InPath) override
	{
		FullPath = InPath;
	}
	//~ End ICineAssemblyTreeItem Interface

	TArray<TSharedPtr<FCineAssemblyFolderItem>> GetChildFolders() const
	{
		TArray<TSharedPtr<FCineAssemblyFolderItem>> ChildFolders;
		for (const TSharedPtr<ICineAssemblyTreeItem>& Child : Children)
		{
			if (TSharedPtr<FCineAssemblyFolderItem> ChildFolder = Child->AsFolder())
			{
				ChildFolders.Add(ChildFolder);
			}
		}

		return ChildFolders;
	}

	/** The fully qualified path of this item (relative to the root folder) */
	FString FullPath;

	/** The children of this folder item */
	TArray<TSharedPtr<ICineAssemblyTreeItem>> Children;
};

namespace UE::CineAssemblyTools::Private
{
	auto SortTreeItems = [](const TSharedPtr<ICineAssemblyTreeItem>& A, const TSharedPtr<ICineAssemblyTreeItem>& B)
	{
		const bool bAIsFolder = A->AsFolder().IsValid();
		const bool bBIsFolder = B->AsFolder().IsValid();
		if (bAIsFolder != bBIsFolder)
		{
			return bBIsFolder;
		}

		return A->GetName() < B->GetName();
	};

	void SortTreeItemsRecursive(const TSharedPtr<FCineAssemblyFolderItem>& FolderItem)
	{
		FolderItem->Children.Sort(SortTreeItems);
		for (const TSharedPtr<FCineAssemblyFolderItem>& ChildFolder : FolderItem->GetChildFolders())
		{
			SortTreeItemsRecursive(ChildFolder);
		}
	}

	/** Get the list of template folder names for the input Assembly */
	TArray<FString> GetFolderTemplates(UCineAssembly* Assembly)
	{
		if (!Assembly)
		{
			return TArray<FString>();
		}

		// Extract the template folder names from the input assembly's default folder list
		TArrayView<FTemplateString> DefaultFolderNames = Assembly->DefaultFolderNames;

		TArray<FString> TemplateFolderNames;
		TemplateFolderNames.Reserve(DefaultFolderNames.Num());

		Algo::Transform(DefaultFolderNames, TemplateFolderNames, [](const FTemplateString& TemplateString) { return TemplateString.Template; });

		return TemplateFolderNames;
	}

	/** Set the list of template folder names for the input Assembly */
	void SetFolderTemplates(UCineAssembly* Assembly, const TArray<FString>& InFolderTemplates)
	{
		if (!Assembly)
		{
			return;
		}

		TArray<FTemplateString>& DefaultFolderNames = Assembly->DefaultFolderNames;

		DefaultFolderNames.Reset(InFolderTemplates.Num());
		Algo::Transform(InFolderTemplates, DefaultFolderNames, [](const FString& TemplateString) { return FTemplateString(TemplateString, FText::GetEmpty()); });
	}

	/** Recursively get the path of every folder item in the tree view */
	void GetFolderListRecursive(const TSharedPtr<FCineAssemblyFolderItem>& FolderItem, TArray<FString>& OutFolderList)
	{
		for (const TSharedPtr<FCineAssemblyFolderItem>& ChildFolder : FolderItem->GetChildFolders())
		{
			OutFolderList.Add(ChildFolder->GetPath());
			GetFolderListRecursive(ChildFolder, OutFolderList);
		}
	}

	/** Returns the folder in the tree whose path matches the input path */
	TSharedPtr<FCineAssemblyFolderItem> FindFolderByPathRecursive(const TSharedPtr<FCineAssemblyFolderItem>& FolderItem, const FString& FolderPath)
	{
		// Normalize the path by removing any trailing slash before comparing
		FString NormalizedPath = FolderPath;
		NormalizedPath.RemoveFromEnd(TEXT("/"));

		if (FolderItem->GetPath().Equals(NormalizedPath))
		{
			return FolderItem;
		}

		for (const TSharedPtr<FCineAssemblyFolderItem>& ChildFolder : FolderItem->GetChildFolders())
		{
			if (const TSharedPtr<FCineAssemblyFolderItem>& MatchingFolder = FindFolderByPathRecursive(ChildFolder, NormalizedPath))
			{
				return MatchingFolder;
			}
		}

		return nullptr;
	}

	/** Modifies the path of the input folder item and all of its children (recursively) with the input new path */
	void SetChildrenPathRecursive(const TSharedPtr<FCineAssemblyFolderItem>& FolderItem, const FString& NewPath)
	{
		for (const TSharedPtr<ICineAssemblyTreeItem>& Child : FolderItem->Children)
		{
			const FString NewChildPath = NewPath / Child->GetName();
			Child->SetPath(NewChildPath);

			if (TSharedPtr<FCineAssemblyFolderItem> ChildFolder = Child->AsFolder())
			{
				SetChildrenPathRecursive(ChildFolder, NewChildPath);
			}
		}
	}

	/** Returns true if the input parent folder, or any of its child folders, contains the input item */
	bool ContainsRecursive(const TSharedPtr<FCineAssemblyFolderItem>& InParentFolder, const TSharedPtr<ICineAssemblyTreeItem>& InItem)
	{
		if (InParentFolder->Children.Contains(InItem))
		{
			return true;
		}

		for (const TSharedPtr<FCineAssemblyFolderItem>& ChildFolder : InParentFolder->GetChildFolders())
		{
			if (ContainsRecursive(ChildFolder, InItem))
			{
				return true;
			}
		}

		return false;
	}

	/** Removes a SubAssembly section from its owning Track (and remove the Track from its owning MovieScene if it would then be empty) */
	void RemoveSubAssemblySection(UMovieSceneSubAssemblySection* SubAssemblySection)
	{
		if (!SubAssemblySection)
		{
			return;
		}

		// Find the track that this section belongs to, and remove the section from that track. 
		// IF the track has no other sections after deleting the section, then remove the whole track.
		if (UMovieSceneSubAssemblyTrack* Track = SubAssemblySection->GetTypedOuter<UMovieSceneSubAssemblyTrack>())
		{
			Track->RemoveSection(*SubAssemblySection);
			if (Track->GetAllSections().IsEmpty())
			{
				if (UMovieScene* OwningMovieScene = Track->GetTypedOuter<UMovieScene>())
				{
					OwningMovieScene->RemoveTrack(*Track);
				}
			}
		}
	}

	void RemoveAssociatedAsset(const TSharedPtr<FAssociatedAssetItem>& AssetItem)
	{
		if (UCineAssembly* Assembly = AssetItem->GetAssembly())
		{
			Assembly->RemoveAssociatedAsset(AssetItem->AssetID);
		}
	}

	/** Ensures all intermediate folders in the given path exist in the tree, creating them if necessary. Returns the deepest folder. */
	TSharedPtr<FCineAssemblyFolderItem> FindOrCreateFolderByPath(const TSharedPtr<FCineAssemblyFolderItem>& ContentRoot, const FString& FolderPath, UCineAssembly* Assembly)
	{
		if (FolderPath.IsEmpty())
		{
			return ContentRoot;
		}

		// Normalize the path by removing any trailing slash
		FString NormalizedPath = FolderPath;
		NormalizedPath.RemoveFromEnd(TEXT("/"));

		// Check if the folder already exists
		if (TSharedPtr<FCineAssemblyFolderItem> ExistingFolder = FindFolderByPathRecursive(ContentRoot, NormalizedPath))
		{
			return ExistingFolder;
		}

		// The folder doesn't exist. Ensure the parent path exists first (recursively), then create this folder.
		const FString ParentPath = FPaths::GetPath(NormalizedPath);
		TSharedPtr<FCineAssemblyFolderItem> ParentFolder = FindOrCreateFolderByPath(ContentRoot, ParentPath, Assembly);

		if (!ParentFolder)
		{
			return nullptr;
		}

		TSharedPtr<FCineAssemblyFolderItem> NewFolder = MakeShared<FCineAssemblyFolderItem>();
		NewFolder->SetPath(NormalizedPath);
		NewFolder->ParentFolder = ParentFolder;
		NewFolder->SetAssembly(Assembly);
		ParentFolder->Children.Add(NewFolder);

		return NewFolder;
	}

	/** Generates a unique folder path name, assuming the input folder will be the parent folder */
	FString MakeUniqueFolderPath(const TSharedPtr<FCineAssemblyFolderItem>& FolderItem)
	{
		// This implementation is based on a similar utility in AssetTools for creating a unique asset name.
		const FString BaseName = FolderItem->GetPath() / TEXT("NewFolder");

		// Find the index in the string of the last non-numeric character
		int32 CharIndex = BaseName.Len() - 1;
		while (CharIndex >= 0 && BaseName[CharIndex] >= TEXT('0') && BaseName[CharIndex] <= TEXT('9'))
		{
			--CharIndex;
		}

		// Trim the numeric characters off the end of the BaseName string, but remember the integer that was trimmed off to increment and append to the output
		int32 IntSuffix = 1;
		FString TrimmedBaseName = BaseName;
		if (CharIndex >= 0 && CharIndex < BaseName.Len() - 1)
		{
			TrimmedBaseName = BaseName.Left(CharIndex + 1);

			const FString TrailingInteger = BaseName.RightChop(CharIndex + 1);
			IntSuffix = FCString::Atoi(*TrailingInteger) + 1;
		}

		FString WorkingName = TrimmedBaseName;

		while (Algo::ContainsBy(FolderItem->GetChildFolders(), WorkingName, &FCineAssemblyFolderItem::FullPath))
		{
			WorkingName = FString::Printf(TEXT("%s%d"), *TrimmedBaseName, IntSuffix);
			IntSuffix++;
		}

		return WorkingName;
	}
}

void SCineAssemblyAssetTreeView::Construct(const FArguments& InArgs, UCineAssembly* InAssembly)
{
	WeakAssembly = InAssembly;

	bIsReadOnly = InArgs._IsReadOnly;
	bDisplayHintText = InArgs._DisplayHintText;
	SelectionMode = InArgs._SelectionMode;
	bShouldEvaluateTokens = InArgs._ShouldEvaluateTokens;

	OnItemRemoved = InArgs._OnItemRemoved;

	InitializeContentTree(InAssembly, RootFolder, TopLevelAssemblyItem);
	TreeItems.Add(RootFolder);

	TreeView = SNew(STreeView<TSharedPtr<ICineAssemblyTreeItem>>)
		.TreeItemsSource(&TreeItems)
		.SelectionMode(SelectionMode)
		.OnGenerateRow(this, &SCineAssemblyAssetTreeView::OnGenerateTreeRow)
		.OnGetChildren(this, &SCineAssemblyAssetTreeView::OnGetChildren)
		.OnItemsRebuilt(this, &SCineAssemblyAssetTreeView::OnTreeItemsRebuilt)
		.OnContextMenuOpening(this, &SCineAssemblyAssetTreeView::MakeContentTreeContextMenu)
		.OnMouseButtonDoubleClick(this, &SCineAssemblyAssetTreeView::OnTreeViewDoubleClick)
		.OnKeyDownHandler(this, &SCineAssemblyAssetTreeView::OnTreeViewKeyDown);

	ExpandTreeRecursive(RootFolder);

	ChildSlot
	[
		TreeView.ToSharedRef()
	];
}

void SCineAssemblyAssetTreeView::InitializeContentTree(UCineAssembly* Assembly, TSharedPtr<FCineAssemblyFolderItem>& ContentRoot, TSharedPtr<FSubAssemblyItem>& TopLevelAssembly)
{
	using namespace UE::CineAssemblyTools::Private;

	// Create the content tree root
	ContentRoot = MakeShared<FCineAssemblyFolderItem>();
	ContentRoot->SetPath(TEXT(""));

	if (!Assembly)
	{
		return;
	}

	// Sort the list of folders so that paths are added to the tree in the proper order
	TArray<FString> FolderTemplates = GetFolderTemplates(Assembly);
	FolderTemplates.Sort();

	for (FString FolderPath : FolderTemplates)
	{
		FolderPath.RemoveFromEnd(TEXT("/"));
		const FString ParentPath = FPaths::GetPath(FolderPath);

		// Walk the tree until we find an item whose path matches the parent path. The new tree item will be created as one of its children
		TSharedPtr<FCineAssemblyFolderItem> ParentFolder = FindOrCreateFolderByPath(ContentRoot, ParentPath, Assembly);
		if (ParentFolder)
		{
			TSharedPtr<FCineAssemblyFolderItem> NewItem = MakeShared<FCineAssemblyFolderItem>();
			NewItem->SetPath(FolderPath);
			NewItem->ParentFolder = ParentFolder;
			NewItem->SetAssembly(Assembly);
			ParentFolder->Children.Add(NewItem);
		}
	}

	// Add the top-level assembly node
	TopLevelAssembly = MakeShared<FSubAssemblyItem>();

	const FString& TopLevelAssemblyPath = Assembly->PathRelativeToRoot.Template;
	TSharedPtr<FCineAssemblyFolderItem> ParentFolder = FindOrCreateFolderByPath(ContentRoot, TopLevelAssemblyPath, Assembly);
	if (ParentFolder)
	{
		TopLevelAssembly->ParentFolder = ParentFolder;
		TopLevelAssembly->SetAssembly(Assembly);
		ParentFolder->Children.Add(TopLevelAssembly);
	}

	// Add an item to the tree for each SubAssemblySection in the Assembly's MovieScene
	AddSubAssemblySectionsToTree(Assembly, ContentRoot);

	// Add an item to the tree for each child SubAssembly of this Assembly
	AddSubAssembliesToTree(Assembly, ContentRoot);

	// Add an item to the tree for each associated asset definition
	AddAssociatedAssetsToTree(Assembly, ContentRoot);

	SortTreeItemsRecursive(ContentRoot);
}

void SCineAssemblyAssetTreeView::AddSubAssemblySectionsToTree(UCineAssembly* Assembly, TSharedPtr<FCineAssemblyFolderItem>& ContentRoot)
{
	TArray<UMovieSceneSubSection*> SubSections;
	MovieSceneHelpers::GetDescendantSubSections(Assembly->GetMovieScene(), SubSections);

	for (UMovieSceneSubSection* SubSection : SubSections)
	{
		UMovieSceneSubAssemblySection* SubAssemblySection = Cast<UMovieSceneSubAssemblySection>(SubSection);

		if (SubAssemblySection && SubAssemblySection->IsTemplateSection())
		{
			// Find or create the parent folder for this section's path, creating intermediate folders as needed
			if (TSharedPtr<FCineAssemblyFolderItem> ParentFolder = UE::CineAssemblyTools::Private::FindOrCreateFolderByPath(ContentRoot, SubAssemblySection->GetSequencePath(), Assembly))
			{
				// Make a new tree item for this section and add it into the tree
				TSharedPtr<FSubAssemblySectionItem> NewItem = MakeShared<FSubAssemblySectionItem>();
				NewItem->ParentFolder = ParentFolder;
				NewItem->SubAssemblySection = SubAssemblySection;
				ParentFolder->Children.Add(NewItem);
			}
		}
	}
}

void SCineAssemblyAssetTreeView::AddSubAssembliesToTree(UCineAssembly* Assembly, TSharedPtr<FCineAssemblyFolderItem>& ContentRoot)
{
	// Each SubAssembly could have its own content tree that it will create.
	// To display the full tree, we build a temporary content tree for the SubAssembly, then re-parent its root-level items into the parent tree.
	for (UMovieSceneSubSection* SubSection : Assembly->SubAssemblies)
	{
		if (UCineAssembly* SubAssembly = Cast<UCineAssembly>(SubSection->GetSequence()))
		{
			SubAssembly->AssemblyName.Resolved = UCineAssemblyNamingTokens::GetResolvedText(SubAssembly->AssemblyName.Template, SubAssembly);

			// Generate the content tree for this SubAssembly
			TSharedPtr<FCineAssemblyFolderItem> SubAssemblyContentRoot;
			TSharedPtr<FSubAssemblyItem> SubAssemblyTreeItem;
			InitializeContentTree(SubAssembly, SubAssemblyContentRoot, SubAssemblyTreeItem);

			// Reparent the items from the SubAssembly content tree into the parent assembly's content tree
			if (TSharedPtr<FCineAssemblyFolderItem> ParentFolder = UE::CineAssemblyTools::Private::FindOrCreateFolderByPath(ContentRoot, SubAssembly->PathRelativeToParent.Template, Assembly))
			{
				for (const TSharedPtr<ICineAssemblyTreeItem>& Child : SubAssemblyContentRoot->Children)
				{
					ParentFolder->Children.Add(Child);
					Child->ParentFolder = ParentFolder;
				}
			}
		}
	}
}

void SCineAssemblyAssetTreeView::AddAssociatedAssetsToTree(UCineAssembly* Assembly, TSharedPtr<FCineAssemblyFolderItem>& ContentRoot)
{
	for (const FAssemblyAssociatedAssetDesc& AssetDesc : Assembly->AssociatedAssets)
	{
		const FString& RelativePath = AssetDesc.RelativePath.Template;

		if (TSharedPtr<FCineAssemblyFolderItem> ParentFolder = UE::CineAssemblyTools::Private::FindOrCreateFolderByPath(ContentRoot, RelativePath, Assembly))
		{
			TSharedPtr<FAssociatedAssetItem> NewItem = MakeShared<FAssociatedAssetItem>();
			NewItem->ParentFolder = ParentFolder;
			NewItem->SetAssembly(Assembly);
			NewItem->AssetID = AssetDesc.AssetID;
			ParentFolder->Children.Add(NewItem);
		}
	}
}

void SCineAssemblyAssetTreeView::Reinitialize()
{
	TreeItems.Empty();
	InitializeContentTree(WeakAssembly.Get(), RootFolder, TopLevelAssemblyItem);
	TreeItems.Add(RootFolder);

	ExpandTreeRecursive(RootFolder);
	TreeView->RequestTreeRefresh();
}

void SCineAssemblyAssetTreeView::AddFolder()
{
	using namespace UE::CineAssemblyTools::Private;

	TSharedPtr<FCineAssemblyFolderItem> NewFolder = MakeShared<FCineAssemblyFolderItem>();

	// Get the parent item for the new folder being added (this can be the root folder if no parent is currently selected)
	// The TreeView uses single selection mode, so at most one item can ever be selected by the user
	TSharedPtr<FCineAssemblyFolderItem> ParentFolder = RootFolder;

	TArray<TSharedPtr<ICineAssemblyTreeItem>> SelectedItems = TreeView->GetSelectedItems();
	if (SelectedItems.Num() == 1)
	{
		if (TSharedPtr<FCineAssemblyFolderItem> SelectedFolder = SelectedItems[0]->AsFolder())
		{
			ParentFolder = SelectedFolder;
		}
		else
		{
			ParentFolder = SelectedItems[0]->ParentFolder;
		}
	}

	NewFolder->SetPath(MakeUniqueFolderPath(ParentFolder));
	NewFolder->ParentFolder = ParentFolder;
	NewFolder->SetAssembly(WeakAssembly.Get());

	ParentFolder->Children.Add(NewFolder);

	// Sort the children alphabetically to maintain a good ordering with the new folder
	ParentFolder->Children.Sort(SortTreeItems);

	// Save a reference to this item so that when the tree is rebuilt, we can immediately start editing its name
	MostRecentlyAddedItem = NewFolder;

	UpdateFolderList();

	TreeView->SetItemExpansion(ParentFolder, true);
	TreeView->RequestTreeRefresh();
}

TSharedRef<ITableRow> SCineAssemblyAssetTreeView::OnGenerateTreeRow(TSharedPtr<ICineAssemblyTreeItem> TreeItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedPtr<SImage> Icon = SNew(SImage);

	if (TreeItem->AsFolder())
	{
		Icon->SetImage(FCineAssemblyToolsStyle::Get().GetBrush("Icons.Folder"));
		Icon->SetColorAndOpacity(FAppStyle::Get().GetSlateColor("ContentBrowser.DefaultFolderColor"));
	}
	else if (TSharedPtr<FAssociatedAssetItem> AssetItem = TreeItem->AsAssociatedAsset())
	{
		if (const FAssemblyAssociatedAssetDesc* AssetDesc = AssetItem->FindAssetDesc())
		{
			UClass* AssetClass = AssetDesc->AssetClass.LoadSynchronous();
			Icon->SetImage(FSlateIconFinder::FindIconBrushForClass(AssetClass));
		}
		Icon->SetColorAndOpacity(FLinearColor::White);
	}
	else
	{
		Icon->SetImage(FSlateIconFinder::FindIconBrushForClass(UCineAssembly::StaticClass()));
		Icon->SetColorAndOpacity(FLinearColor::White);
	}

	if (!TreeItem->NamingTokenContext)
	{
		TreeItem->NamingTokenContext = TStrongObjectPtr<UCineAssemblyNamingTokensContext>(NewObject<UCineAssemblyNamingTokensContext>());
	}
	TreeItem->NamingTokenContext->Assembly = TreeItem->GetAssembly();

	FNamingTokenFilterArgs FilterArgs;
	FilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

	// The template-asset picker is only meaningful for Associated Asset rows; other row types get an empty slot.
	TSharedRef<SWidget> TemplateAssetPicker = SNullWidget::NullWidget;
	if (TSharedPtr<FAssociatedAssetItem> AssetItem = TreeItem->AsAssociatedAsset())
	{
		TemplateAssetPicker = MakeTemplateAssetPicker(AssetItem);
	}

	return SNew(STableRow<TSharedPtr<ICineAssemblyTreeItem>>, OwnerTable)
		.ShowSelection(true)
		.Padding(FMargin(8.0f, 0.0f, 8.0f, 0.0f))
		.OnCanAcceptDrop_Lambda([this](const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<ICineAssemblyTreeItem> InItem) 
			{ 
				if (bIsReadOnly)
				{
					return TOptional<EItemDropZone>();
				}
				return InItem->CanAcceptDrop(); 
			})
		.OnAcceptDrop(this, &SCineAssemblyAssetTreeView::OnTreeRowAcceptDrop)
		.OnDragDetected(this, &SCineAssemblyAssetTreeView::OnTreeRowDragDetected)
		.Content()
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(0.0f)
				.ToolTipText(this, &SCineAssemblyAssetTreeView::GetTreeItemTooltipText, TreeItem)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 2.0f, 0.0f)
						[
							Icon.ToSharedRef()
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 8.0f, 0.0f)
						[
							SAssignNew(TreeItem->NameWidget, SNamingTokensEditableTextBox)
								.SupportInlineEdit(true)
								.ShouldEvaluateTokens(bShouldEvaluateTokens)
								.Contexts({ TreeItem->NamingTokenContext.Get() })
								.FilterArgs(FilterArgs)
								.EvaluationFrequency(1.0f)
								.ShowUnsetTokenWarning(bShouldEvaluateTokens)
								.TextBoxPadding(FMargin(4, 2, 0, 2))
								.IsReadOnly(bIsReadOnly || TreeItem->IsReadOnly())
								.DisplayTokenIcon(false)
								.DisplayBorderImage(false)
								.DisplayErrorMessage(!TreeItem->IsReadOnly())
								.Text(this, &SCineAssemblyAssetTreeView::GetTreeItemDisplayName, TreeItem)
								.OnTextCommitted(this, &SCineAssemblyAssetTreeView::OnTreeItemTextCommitted, TreeItem)
								.OnValidateTokenizedText(this, &SCineAssemblyAssetTreeView::OnValidateTreeItemName, TreeItem)
								.OnTokenizedTextEvaluated(this, &SCineAssemblyAssetTreeView::OnTreeItemTextResolved, TreeItem)
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
								.Text(this, &SCineAssemblyAssetTreeView::GetTreeItemLabelText, TreeItem)
								.Font(FCoreStyle::GetDefaultFontStyle("Italic", 10))
								.ColorAndOpacity(FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f, 0.3f)))
						]

					+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						[
							SNullWidget::NullWidget
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(4.0f, 0.0f, 0.0f, 0.0f)
						[
							TemplateAssetPicker
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(4.0f, 0.0f, 0.0f, 0.0f)
						[
							MakeMetadataLinkButton(TreeItem)
						]
				]
		];
}

TSharedRef<SWidget> SCineAssemblyAssetTreeView::MakeTemplateAssetPicker(TSharedPtr<FAssociatedAssetItem> AssetItem)
{
	// The picker is a Schema-editing affordance; in the read-only Assembly config view it has no purpose, so hide it entirely.
	if (bIsReadOnly)
	{
		return SNullWidget::NullWidget;
	}

	const FAssemblyAssociatedAssetDesc* AssetDesc = AssetItem ? AssetItem->FindAssetDesc() : nullptr;
	UClass* AssetClass = AssetDesc ? AssetDesc->AssetClass.LoadSynchronous() : nullptr;
	if (!AssetClass)
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SBox)
		.WidthOverride(220.0f)
		[
			SNew(SObjectPropertyEntryBox)
				.AllowedClass(AssetClass)
				.AllowClear(true)
				.DisplayThumbnail(false)
				.ToolTipText(LOCTEXT("TemplateAssetTooltip", "Template asset used when a CineAssembly is created from this schema. None creates a blank asset. Existing assemblies are not affected."))
				.ObjectPath(this, &SCineAssemblyAssetTreeView::GetTemplateAssetObjectPath, AssetItem)
				.OnObjectChanged(this, &SCineAssemblyAssetTreeView::OnTemplateAssetChanged, AssetItem)
		];
}

FString SCineAssemblyAssetTreeView::GetTemplateAssetObjectPath(TSharedPtr<FAssociatedAssetItem> AssetItem) const
{
	const FAssemblyAssociatedAssetDesc* AssetDesc = AssetItem ? AssetItem->FindAssetDesc() : nullptr;
	if (AssetDesc)
	{
		return AssetDesc->TemplateAsset.ToString();
	}
	return FString();
}

void SCineAssemblyAssetTreeView::OnTemplateAssetChanged(const FAssetData& AssetData, TSharedPtr<FAssociatedAssetItem> AssetItem)
{
	if (!AssetItem)
	{
		return;
	}

	UCineAssembly* Assembly = AssetItem->GetAssembly();
	FAssemblyAssociatedAssetDesc* AssetDesc = AssetItem->FindAssetDesc();
	if (Assembly && AssetDesc)
	{
		Assembly->Modify();
		AssetDesc->TemplateAsset = AssetData.GetSoftObjectPath();
	}
}

TSharedRef<SWidget> SCineAssemblyAssetTreeView::MakeMetadataLinkButton(TSharedPtr<ICineAssemblyTreeItem> TreeItem)
{
	if (bIsReadOnly)
	{
		return SNullWidget::NullWidget;
	}

	if (TSharedPtr<FAssociatedAssetItem> AssetItem = TreeItem->AsAssociatedAsset())
	{
		const FAssemblyAssociatedAssetDesc* AssetDesc = AssetItem->FindAssetDesc();
		if (!AssetDesc)
		{
			return SNullWidget::NullWidget;
		}
		return SNew(SAssemblyMetadataLink, AssetItem->GetAssembly(), AssetDesc->AssetID);
	}

	if (TSharedPtr<FSubAssemblySectionItem> SectionItem = TreeItem->AsSubAssemblySection())
	{
		if (SectionItem->SubAssemblySection.IsValid() && SectionItem->SubAssemblySection->IsTemplateSection())
		{
			return SNew(SAssemblyMetadataLink, WeakAssembly.Get(), SectionItem->SubAssemblySection->GetSectionID());
		}
	}

	return SNullWidget::NullWidget;
}

void SCineAssemblyAssetTreeView::OnGetChildren(TSharedPtr<ICineAssemblyTreeItem> TreeItem, TArray<TSharedPtr<ICineAssemblyTreeItem>>& OutNodes)
{
	// Children are pre-sorted by SortTreeItems (assets first, then folders, alphabetical within each group)
	if (TSharedPtr<FCineAssemblyFolderItem> FolderItem = TreeItem->AsFolder())
	{
		OutNodes.Append(FolderItem->Children);
	}
}

void SCineAssemblyAssetTreeView::OnTreeItemsRebuilt()
{
	// Upon regenerating the tree view, allow the user to immediately interact with the name widget of the newly added folder in order to rename it
	if (MostRecentlyAddedItem && MostRecentlyAddedItem->NameWidget)
	{
		MostRecentlyAddedItem->NameWidget->EnterEditingMode();
		MostRecentlyAddedItem.Reset();
	}
}

FText SCineAssemblyAssetTreeView::GetTreeItemDisplayName(TSharedPtr<ICineAssemblyTreeItem> TreeItem) const
{
	if (TreeItem == RootFolder)
	{
		return LOCTEXT("RootPathName", "Root Folder");
	}
	else if (TreeItem == TopLevelAssemblyItem)
	{
		return FText::FromString(TEXT("{assembly}"));
	}
	else
	{
		return TreeItem->GetDisplayName();
	}
}

FText SCineAssemblyAssetTreeView::GetTreeItemLabelText(TSharedPtr<ICineAssemblyTreeItem> TreeItem) const
{
	// SubAssembly items display the label of the associated assembly
	if (TreeItem->AsSubAssembly())
	{
		if (UCineAssembly* AssociatedAssembly = TreeItem->GetAssembly())
		{
			const FName AssemblyLabel = AssociatedAssembly->GetLabel();
			if (!AssemblyLabel.IsNone())
			{
				return FText::Format(LOCTEXT("LabelText", "[{0}]"), FText::FromName(AssemblyLabel));
			}
		}
	}

	// AssociatedAsset items display the label of the associated AssetDesc
	if (TSharedPtr<FAssociatedAssetItem> AssetItem = TreeItem->AsAssociatedAsset())
	{
		if (const FAssemblyAssociatedAssetDesc* AssetDesc = AssetItem->FindAssetDesc())
		{
			if (!AssetDesc->Label.IsNone())
			{
				return FText::Format(LOCTEXT("LabelText", "[{0}]"), FText::FromName(AssetDesc->Label));
			}
		}
	}

	// SubAssemblySection items show the label and origin text, both from the section properties
	if (TSharedPtr<FSubAssemblySectionItem> SectionItem = TreeItem->AsSubAssemblySection())
	{
		if (UMovieSceneSubAssemblySection* SubAssemblySection = SectionItem->SubAssemblySection.Get())
		{
			FText LabelText;
			if (!SubAssemblySection->Label.IsNone())
			{
				LabelText = FText::Format(LOCTEXT("LabelText", "[{0}]"), FText::FromName(SubAssemblySection->Label));
			}

			FText OriginText;
			if (UObject* TemplateObject = SubAssemblySection->GetAssemblyTemplate())
			{
				OriginText = FText::Format(LOCTEXT("TemplateOriginText", "from {0}"), FText::FromString(TemplateObject->GetName()));
			}

			if (!LabelText.IsEmpty() && !OriginText.IsEmpty())
			{
				return FText::Format(LOCTEXT("LabeledOrigin", "{0} {1}"), LabelText, OriginText);
			}
			return LabelText.IsEmpty() ? OriginText : LabelText;
		}
	}

	return FText::GetEmpty();
}

FText SCineAssemblyAssetTreeView::GetTreeItemTooltipText(TSharedPtr<ICineAssemblyTreeItem> TreeItem) const
{
	if (TreeItem->AsSubAssemblySection())
	{
		return FText::GetEmpty();
	}

	if (!bDisplayHintText)
	{
		return FText::GetEmpty();
	}

	UCineAssembly* AssociatedAssembly = TreeItem->GetAssembly();
	if (!AssociatedAssembly)
	{
		return FText::GetEmpty();
	}

	if (TreeItem->AsSubAssembly())
	{
		if (UCineAssembly* ParentAssembly = Cast<UCineAssembly>(AssociatedAssembly->ParentAssembly.TryLoad()))
		{
			return FText::Format(LOCTEXT("CreatedByText", "created by {0}"), ParentAssembly->AssemblyName.Resolved);
		}
		return FText::GetEmpty();
	}

	return FText::Format(LOCTEXT("CreatedByText", "created by {0}"), AssociatedAssembly->AssemblyName.Resolved);
}

void SCineAssemblyAssetTreeView::OnTreeItemTextCommitted(const FText& InText, ETextCommit::Type InCommitType, TSharedPtr<ICineAssemblyTreeItem> TreeItem)
{
	using namespace UE::CineAssemblyTools::Private;

	// Early-out if the name has not actually changed
	const FString OldName = TreeItem->GetName();
	if (OldName.Equals(InText.ToString()))
	{
		return;
	}

	TreeItem->SetDisplayName(InText);

	if (TSharedPtr<FCineAssemblyFolderItem> FolderItem = TreeItem->AsFolder())
	{
		// If this is a folder item, update the path of all of its children (recursively)
		SetChildrenPathRecursive(FolderItem, FolderItem->GetPath());
		UpdateFolderList();
	}

	TreeItem->ParentFolder->Children.Sort(SortTreeItems);

	TreeView->RequestTreeRefresh();
}

bool SCineAssemblyAssetTreeView::OnValidateTreeItemName(const FText& InText, FText& OutErrorMessage, TSharedPtr<ICineAssemblyTreeItem> TreeItem)
{
	if (InText.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("EmptyItemNameErrorMessage", "Please provide a name for this item");
		return false;
	}

	// Check for invalid characters
	FString InvalidCharacters = INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS;

	// These characters are actually valid, because we want to support naming tokens
	InvalidCharacters = InvalidCharacters.Replace(TEXT("{}"), TEXT(""));
	InvalidCharacters = InvalidCharacters.Replace(TEXT(":"), TEXT(""));

	if (!FName::IsValidXName(InText.ToString(), InvalidCharacters, &OutErrorMessage))
	{
		return false;
	}

	// SubAssemblies cannot use the {assembly} token in their name because it resolves to the SubAssembly name itself, causing infinite name recursion.
	// The {parent} token is the correct way to reference the top-level assembly name.
	if (TreeItem->AsSubAssemblySection())
	{
		const FString PotentialName = InText.ToString();
		if (PotentialName.Contains(TEXT("{assembly}")) || PotentialName.Contains(TEXT("{cat:assembly}")))
		{
			OutErrorMessage = LOCTEXT("SubAssemblyAssemblyTokenError", "The {assembly} token cannot be used in an assembly name because it is self-referential. Use {parent} instead to reference the top-level assembly name.");
			return false;
		}
	}

	// Check to see if the parent folder already contains an item of the same type with the same name
	// Note: We only check folder names, because asset names may contain tokens, and identical tokenized names for different assets is fine
	if (TreeItem->AsFolder())
	{
		const bool bIsDuplicateName = Algo::AnyOf(TreeItem->ParentFolder->GetChildFolders(), [TreeItem, InText](const TSharedPtr<ICineAssemblyTreeItem>& ChildItem)
			{
				return ChildItem && (ChildItem != TreeItem) && ChildItem->GetDisplayName().EqualTo(InText);
			});

		if (bIsDuplicateName)
		{
			OutErrorMessage = LOCTEXT("DuplicateItemNameErrorMessage", "An item already exists at this location with this name");
			return false;
		}
	}

	return true;
}

void SCineAssemblyAssetTreeView::OnTreeItemTextResolved(const FText& InText, TSharedPtr<ICineAssemblyTreeItem> TreeItem)
{
	if (TreeItem->AsSubAssembly())
	{
		if (UCineAssembly* Assembly = TreeItem->GetAssembly())
		{
			Assembly->AssemblyName.Resolved = InText;
		}
	}
}

void SCineAssemblyAssetTreeView::OnTreeViewDoubleClick(TSharedPtr<ICineAssemblyTreeItem> TreeItem)
{
	// Get the row for the input item and put its textblock in edit mode so the user can rename the item
	if (TreeItem && TreeItem->NameWidget)
	{
		if (!bIsReadOnly && !TreeItem->IsReadOnly())
		{
			TreeItem->NameWidget->EnterEditingMode();
		}
	}
}

FReply SCineAssemblyAssetTreeView::OnTreeViewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (bIsReadOnly)
	{
		return FReply::Unhandled();
	}

	if (InKeyEvent.GetKey() == EKeys::Delete)
	{
		// The TreeView uses single selection mode, so at most one item can ever be selected by the user
		TArray<TSharedPtr<ICineAssemblyTreeItem>> SelectedItems = TreeView->GetSelectedItems();
		if (SelectedItems.Num() == 1)
		{
			const TSharedPtr<ICineAssemblyTreeItem>& SelectedItem = SelectedItems[0];
			if (!SelectedItem->IsReadOnly())
			{
				DeleteTreeItem(SelectedItem);
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

TSharedPtr<SWidget> SCineAssemblyAssetTreeView::MakeContentTreeContextMenu()
{
	// The TreeView uses single selection mode, so at most one item can ever be selected by the user
	TArray<TSharedPtr<ICineAssemblyTreeItem>> SelectedItems = TreeView->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<ICineAssemblyTreeItem>& SelectedTreeItem = SelectedItems[0];

	constexpr bool bCloseAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterMenuSelection, nullptr);

	if (!bIsReadOnly)
	{
		if (TSharedPtr<FCineAssemblyFolderItem> FolderItem = SelectedTreeItem->AsFolder())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("AddChildFolderAction", "Add Child Folder"),
				FText::GetEmpty(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"),
				FUIAction(FExecuteAction::CreateLambda([this]() { AddFolder(); })),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}

		if (!SelectedTreeItem->IsReadOnly())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("RenameAction", "Rename"),
				FText::GetEmpty(),
				FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.AssetNaming"),
				FUIAction(FExecuteAction::CreateLambda([SelectedTreeItem]() { SelectedTreeItem->NameWidget->EnterEditingMode(); })),
				NAME_None,
				EUserInterfaceActionType::Button
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("DeleteAction", "Delete"),
				FText::GetEmpty(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
				FUIAction(FExecuteAction::CreateSP(this, &SCineAssemblyAssetTreeView::DeleteTreeItem, SelectedTreeItem)),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
	}

	// SubMenu for editing item labels and metadata
	TSharedRef<SWidget> EditPanel = SNullWidget::NullWidget;
	if (TSharedPtr<FSubAssemblyItem> SubAssemblyItem = SelectedTreeItem->AsSubAssembly())
	{
		EditPanel = FCineAssemblyMetadataWidgets::MakeEditMenuForAssembly(SubAssemblyItem->GetAssembly());
	}
	else if (TSharedPtr<FSubAssemblySectionItem> SectionItem = SelectedTreeItem->AsSubAssemblySection())
	{
		if (UMovieSceneSubAssemblySection* Section = SectionItem->SubAssemblySection.Get())
		{
			EditPanel = FCineAssemblyMetadataWidgets::MakeEditMenuForSection(Section);
		}
	}
	else if (TSharedPtr<FAssociatedAssetItem> AssetItem = SelectedTreeItem->AsAssociatedAsset())
	{
		EditPanel = FCineAssemblyMetadataWidgets::MakeEditMenuForAssociatedAsset(AssetItem->GetAssembly(), AssetItem->AssetID);
	}

	if (EditPanel != SNullWidget::NullWidget)
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("EditMenuLabel", "Edit"),
			LOCTEXT("EditMenuTooltip", "Edit this row's Label and per-instance Metadata."),
			FNewMenuDelegate::CreateLambda([EditPanel](FMenuBuilder& SubMenuBuilder) { SubMenuBuilder.AddWidget(EditPanel, FText::GetEmpty(), true); }),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"));
	}

	return MenuBuilder.MakeWidget();
}

void SCineAssemblyAssetTreeView::DeleteTreeItem(TSharedPtr<ICineAssemblyTreeItem> TreeItem)
{
	if (TSharedPtr<FCineAssemblyFolderItem> FolderItem = TreeItem->AsFolder())
	{
		DeleteFolderItem(FolderItem);
	}
	else if (TSharedPtr<FSubAssemblySectionItem> SectionItem = TreeItem->AsSubAssemblySection())
	{
		UE::CineAssemblyTools::Private::RemoveSubAssemblySection(SectionItem->SubAssemblySection.Get());
	}
	else if (TSharedPtr<FAssociatedAssetItem> AssetItem = TreeItem->AsAssociatedAsset())
	{
		UE::CineAssemblyTools::Private::RemoveAssociatedAsset(AssetItem);
	}

	TreeItem->ParentFolder->Children.Remove(TreeItem);

	UpdateFolderList();

	TreeView->RequestTreeRefresh();

	OnItemRemoved.ExecuteIfBound();
}

void SCineAssemblyAssetTreeView::DeleteFolderItem(const TSharedPtr<FCineAssemblyFolderItem>& FolderItem)
{
	// If the folder contains the top level assembly, reparent that item to the root so it is not lost
	if (FolderItem->Children.Contains(TopLevelAssemblyItem))
	{
		FolderItem->Children.Remove(TopLevelAssemblyItem);
		RootFolder->Children.Add(TopLevelAssemblyItem);
		TopLevelAssemblyItem->ParentFolder = RootFolder;
	}

	for (const TSharedPtr<ICineAssemblyTreeItem>& Child : FolderItem->Children)
	{
		if (TSharedPtr<FCineAssemblyFolderItem> ChildFolder = Child->AsFolder())
		{
			DeleteFolderItem(ChildFolder);
		}
		else if (TSharedPtr<FSubAssemblySectionItem> ChildSection = Child->AsSubAssemblySection())
		{
			UE::CineAssemblyTools::Private::RemoveSubAssemblySection(ChildSection->SubAssemblySection.Get());
		}
		else if (TSharedPtr<FAssociatedAssetItem> ChildAsset = Child->AsAssociatedAsset())
		{
			UE::CineAssemblyTools::Private::RemoveAssociatedAsset(ChildAsset);
		}
	}

	FolderItem->Children.Empty();
}

void SCineAssemblyAssetTreeView::UpdateFolderList()
{
	using namespace UE::CineAssemblyTools::Private;

	UCineAssembly* Assembly = WeakAssembly.Get();
	if (!Assembly)
	{
		return;
	}

	TArray<FString> CachedFolderList = GetFolderTemplates(Assembly);

	TArray<FString> FolderList;
	GetFolderListRecursive(RootFolder, FolderList);

	if (FolderList != CachedFolderList)
	{
		Assembly->Modify();
		SetFolderTemplates(Assembly, FolderList);
	}

	if (TopLevelAssemblyItem->ParentFolder)
	{
		Assembly->PathRelativeToRoot.Template = TopLevelAssemblyItem->ParentFolder->GetPath();
	}
}

void SCineAssemblyAssetTreeView::ExpandTreeRecursive(const TSharedPtr<FCineAssemblyFolderItem>& FolderItem) const
{
	TreeView->SetItemExpansion(FolderItem, true);

	for (const TSharedPtr<FCineAssemblyFolderItem>& ChildFolder : FolderItem->GetChildFolders())
	{
		ExpandTreeRecursive(ChildFolder);
	}
}

FReply SCineAssemblyAssetTreeView::OnTreeRowDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bIsReadOnly)
	{
		return FReply::Unhandled();
	}

	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		TArray<TSharedPtr<ICineAssemblyTreeItem>> SelectedItems = TreeView->GetSelectedItems();
		if (SelectedItems.Num() == 1)
		{
			TSharedPtr<ICineAssemblyTreeItem> SelectedTreeItem = SelectedItems[0];

			if (!SelectedTreeItem->CanDrag())
			{
				return FReply::Handled();
			}

			TSharedRef<FCineAssemblyTreeDragDrop> Operation = FCineAssemblyTreeDragDrop::New(SelectedTreeItem);

			// If the drop operation is not handled for some reason, add the selected tree item back to its original location
			Operation->OnDropNotHandled.BindLambda([this, SelectedTreeItem]()
				{
					SelectedTreeItem->ParentFolder->Children.Add(SelectedTreeItem);
					SelectedTreeItem->ParentFolder->Children.Sort(UE::CineAssemblyTools::Private::SortTreeItems);

					TreeView->RequestTreeRefresh();
				});

			SelectedTreeItem->ParentFolder->Children.Remove(SelectedTreeItem);
			TreeView->RequestTreeRefresh();

			return FReply::Handled().BeginDragDrop(Operation);
		}
	}

	return FReply::Unhandled();
}

FReply SCineAssemblyAssetTreeView::OnTreeRowAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<ICineAssemblyTreeItem> InItem)
{
	using namespace UE::CineAssemblyTools::Private;

	if (bIsReadOnly)
	{
		return FReply::Unhandled();
	}

	// Only Folder Items are valid drop targets
	TSharedPtr<FCineAssemblyFolderItem> TargetFolder = InItem->AsFolder();
	if (!TargetFolder)
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FCineAssemblyTreeDragDrop> Operation = InDragDropEvent.GetOperationAs<FCineAssemblyTreeDragDrop>();
	if (Operation.IsValid() && Operation->TreeItem.IsValid())
	{
		TSharedPtr<ICineAssemblyTreeItem> DroppedItem = Operation->TreeItem;

		// Ensure that we are not attempting to drop an item onto itself or one of its children
		if (DroppedItem == InItem)
		{
			return FReply::Unhandled();
		}
		else if (DroppedItem->AsFolder() && ContainsRecursive(DroppedItem->AsFolder(), InItem))
		{
			return FReply::Unhandled();
		}

		// If the dropped item is a folder, but the target folder already contains a subfolder with the same name, do not handle the drop (the dragged item will be reset to its original location)
		const FString DroppedItemPath = TargetFolder->GetPath() / DroppedItem->GetName();
		if (DroppedItem->AsFolder())
		{
			if (Algo::AnyOf(TargetFolder->GetChildFolders(), [DroppedItemPath](const TSharedPtr<FCineAssemblyFolderItem>& FolderPtr) { return FolderPtr && FolderPtr->GetPath() == DroppedItemPath; }))
			{
				return FReply::Unhandled();
			}
		}

		DroppedItem->SetPath(DroppedItemPath);
		DroppedItem->ParentFolder = TargetFolder;

		TargetFolder->Children.Add(DroppedItem);

		if (TSharedPtr<FCineAssemblyFolderItem> DroppedFolder = DroppedItem->AsFolder())
		{
			SetChildrenPathRecursive(DroppedFolder, DroppedFolder->GetPath());
		}

		TargetFolder->Children.Sort(SortTreeItems);

		UpdateFolderList();

		ExpandTreeRecursive(TargetFolder);
		TreeView->RequestTreeRefresh();
	}

	return FReply::Handled();
}

TSharedRef<FCineAssemblyTreeDragDrop> FCineAssemblyTreeDragDrop::New(const TSharedPtr<ICineAssemblyTreeItem>& InItem)
{
	TSharedRef<FCineAssemblyTreeDragDrop> DragDropOp = MakeShared<FCineAssemblyTreeDragDrop>();
	DragDropOp->TreeItem = InItem;
	DragDropOp->MouseCursor = EMouseCursor::GrabHandClosed;
	DragDropOp->Construct();

	return DragDropOp;
}

TSharedPtr<SWidget> FCineAssemblyTreeDragDrop::GetDefaultDecorator() const
{
	return SNew(SBorder)
		.Padding(8.0f)
		.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.PanelBackground"))
		[
			SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SImage).Image(FCineAssemblyToolsStyle::Get().GetBrush("Icons.Sequencer"))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock).Text(TreeItem->GetDisplayName())
				]
		];
}

void FCineAssemblyTreeDragDrop::OnDragged(const class FDragDropEvent& DragDropEvent)
{
	if (CursorDecoratorWindow.IsValid())
	{
		CursorDecoratorWindow->MoveWindowTo(DragDropEvent.GetScreenSpacePosition() - (CursorDecoratorWindow->GetSizeInScreen() * FVector2f(0.0f, 0.5f)));
	}
}

void FCineAssemblyTreeDragDrop::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	if (!bDropWasHandled)
	{
		OnDropNotHandled.ExecuteIfBound();
	}
}

#undef LOCTEXT_NAMESPACE
