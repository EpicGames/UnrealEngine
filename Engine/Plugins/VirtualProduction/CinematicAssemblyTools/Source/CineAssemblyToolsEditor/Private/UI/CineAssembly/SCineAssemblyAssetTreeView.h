// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CineAssembly.h"
#include "Framework/Views/ITypedTableView.h"
#include "SNamingTokensEditableTextBox.h"
#include "Widgets/Views/STreeView.h"

struct ICineAssemblyTreeItem;
struct FSubAssemblyItem;
struct FSubAssemblySectionItem;
struct FAssociatedAssetItem;
struct FCineAssemblyFolderItem;

class UMovieSceneSubAssemblySection;

/** Drag and Drop operation to handle dragging tree items in the CineAssembly AssetTreeView  */
class FCineAssemblyTreeDragDrop : public FDragDropOperation
{
public:

	DRAG_DROP_OPERATOR_TYPE(FCineAssemblyTreeDragDrop, FDragDropOperation)

	FCineAssemblyTreeDragDrop() = default;

	static TSharedRef<FCineAssemblyTreeDragDrop> New(const TSharedPtr<ICineAssemblyTreeItem>& InItem);

	// Begin FDragDropOperation overrides
	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;
	virtual void OnDragged(const class FDragDropEvent& DragDropEvent) override;
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	// End FDragDropOperation overrides

	/** The tree item being dragged. */
	TSharedPtr<ICineAssemblyTreeItem> TreeItem;

	/** Callback to handle the case where an item is dropped somewhere not on the tree view */
	FSimpleDelegate OnDropNotHandled;
};

/**
 * A tree view widget that displays the folder hierarchy and asset layout of a CineAssembly.
 */
class SCineAssemblyAssetTreeView : public SCompoundWidget
{
public:
	SCineAssemblyAssetTreeView() = default;

	SLATE_BEGIN_ARGS(SCineAssemblyAssetTreeView)
		: _SelectionMode(ESelectionMode::Type::Single)
		, _ShouldEvaluateTokens(false)
		, _IsReadOnly(false)
		, _DisplayHintText(false)
		{}

		/** Sets the selection mode for this tree view. Ignored when IsReadOnly is true (selection is forced off). */
		SLATE_ARGUMENT(ESelectionMode::Type, SelectionMode)

		/** Sets whether this tree view should evaluate and display resolved token strings, rather than raw templates */
		SLATE_ARGUMENT(bool, ShouldEvaluateTokens)

		/** When true, all editing interactions are disabled: renaming, deleting, drag/drop, and keyboard shortcuts */
		SLATE_ARGUMENT(bool, IsReadOnly)

		/** When true, displays additional context text next to tree items (e.g. which assembly created a sub-assembly) */
		SLATE_ARGUMENT(bool, DisplayHintText)

		/** Triggered when an item in the tree is removed */
		SLATE_EVENT(FSimpleDelegate, OnItemRemoved)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UCineAssembly* InAssembly);

	/** Adds a new child folder under the currently selected folder (or the root if nothing is selected) */
	void AddFolder();

	/** Rebuilds the content tree from the current assembly state and refreshes the tree view */
	void Reinitialize();

private:
	/** Build the content tree for the input Assembly, including its SubAssemblies and default folders */
	void InitializeContentTree(UCineAssembly* Assembly, TSharedPtr<FCineAssemblyFolderItem>& ContentRoot, TSharedPtr<FSubAssemblyItem>& TopLevelAssembly);

	/** Add SubAssemblySection items to the tree from the Assembly's MovieScene */
	void AddSubAssemblySectionsToTree(UCineAssembly* Assembly, TSharedPtr<FCineAssemblyFolderItem>& ContentRoot);

	/** Add SubAssembly items to the tree from the Assembly's SubAssembly list */
	void AddSubAssembliesToTree(UCineAssembly* Assembly, TSharedPtr<FCineAssemblyFolderItem>& ContentRoot);

	/** Add associated asset items to the tree from the Assembly's AssociatedAssets list */
	void AddAssociatedAssetsToTree(UCineAssembly* Assembly, TSharedPtr<FCineAssemblyFolderItem>& ContentRoot);

	/** Generates the row widget for an entry in the tree view */
	TSharedRef<ITableRow> OnGenerateTreeRow(TSharedPtr<ICineAssemblyTreeItem> TreeItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Gets the children of the input tree view item to build additional tree rows */
	void OnGetChildren(TSharedPtr<ICineAssemblyTreeItem> TreeItem, TArray<TSharedPtr<ICineAssemblyTreeItem>>& OutNodes);

	/** Callback when the content tree view rebuilds itself, used to make the textbox in the most recently added row editable */
	void OnTreeItemsRebuilt();

	/** Callback to get the display name of the input tree item */
	FText GetTreeItemDisplayName(TSharedPtr<ICineAssemblyTreeItem> TreeItem) const;

	/** Callback to get the semantic label of the input tree item */
	FText GetTreeItemLabelText(TSharedPtr<ICineAssemblyTreeItem> TreeItem) const;

	/** Callback to get the tooltip text of the input tree item */
	FText GetTreeItemTooltipText(TSharedPtr<ICineAssemblyTreeItem> TreeItem) const;

	/** Renames the tree item, and updates the paths of all of its children */
	void OnTreeItemTextCommitted(const FText& InText, ETextCommit::Type InCommitType, TSharedPtr<ICineAssemblyTreeItem> TreeItem);

	/** Validates a tree item's name, checking for empty names, invalid characters, and duplicate folder names */
	bool OnValidateTreeItemName(const FText& InText, FText& OutErrorMessage, TSharedPtr<ICineAssemblyTreeItem> TreeItem);

	/** Callback when naming tokens in a tree item's text are evaluated, used to update the assembly's resolved name */
	void OnTreeItemTextResolved(const FText& InText, TSharedPtr<ICineAssemblyTreeItem> TreeItem);

	/** Handles double clicks on the tree view to rename the item */
	void OnTreeViewDoubleClick(TSharedPtr<ICineAssemblyTreeItem> TreeItem);

	/** Handles key presses on the tree view */
	FReply OnTreeViewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/** Create the context menu when the content tree view is right-clicked */
	TSharedPtr<SWidget> MakeContentTreeContextMenu();

	/** Remove the input item and its children from the tree view */
	void DeleteTreeItem(TSharedPtr<ICineAssemblyTreeItem> TreeItem);

	/** Removes a folder and all of its children from the tree view */
	void DeleteFolderItem(const TSharedPtr<FCineAssemblyFolderItem>& FolderItem);

	/** Update the list of folders to create from the current set in the tree view */
	void UpdateFolderList();

	/** Recursively expands every folder in the tree view */
	void ExpandTreeRecursive(const TSharedPtr<FCineAssemblyFolderItem>& FolderItem) const;

	/** Creates the inline template asset picker widget for an Associated Asset row. Returns SNullWidget if AssetItem is null or its AssetClass can't be resolved. */
	TSharedRef<SWidget> MakeTemplateAssetPicker(TSharedPtr<FAssociatedAssetItem> AssetItem);

	/** Returns the current TemplateAsset path string for the row's descriptor (or empty if the descriptor is missing). */
	FString GetTemplateAssetObjectPath(TSharedPtr<FAssociatedAssetItem> AssetItem) const;

	/** Updates the row's descriptor TemplateAsset to the picker's new selection. */
	void OnTemplateAssetChanged(const FAssetData& AssetData, TSharedPtr<FAssociatedAssetItem> AssetItem);

	/** Creates the metadata link widget for associated asset and SubAssembly section items (returns SNullWidget read-only trees) */
	TSharedRef<SWidget> MakeMetadataLinkButton(TSharedPtr<ICineAssemblyTreeItem> TreeItem);

	/** Begins a drag and drop event to drag an item out of the content tree view */
	FReply OnTreeRowDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Callback when an item is dropped onto a folder in the content tree view to add it to the children of that row */
	FReply OnTreeRowAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<ICineAssemblyTreeItem> InItem);

private:
	/** The assembly that this tree view was constructed with */
	TWeakObjectPtr<UCineAssembly> WeakAssembly;

	/** The tree view of content folders and assets for the provided CineAssembly (and its SubAssemblies) */
	TSharedPtr<STreeView<TSharedPtr<ICineAssemblyTreeItem>>> TreeView;

	/** Items source for the tree view */
	TArray<TSharedPtr<ICineAssemblyTreeItem>> TreeItems;

	/** The root folder in the tree view */
	TSharedPtr<FCineAssemblyFolderItem> RootFolder;

	/** An item representing where the top-level assembly should be created */
	TSharedPtr<FSubAssemblyItem> TopLevelAssemblyItem;

	/** The most recently added tree item, used to allow the user to immediately rename the item after the tree view updates to show it */
	TSharedPtr<ICineAssemblyTreeItem> MostRecentlyAddedItem;

	/** Delegate broadcast when an item is removed from the tree */
	FSimpleDelegate OnItemRemoved;

	/** The selection mode for this tree view */
	ESelectionMode::Type SelectionMode = ESelectionMode::Single;

	/** Whether this tree view should evaluate and display resolved token strings */
	bool bShouldEvaluateTokens = false;

	/** When true, all editing interactions are disabled */
	bool bIsReadOnly = false;

	/** When true, displays additional context text next to tree items */
	bool bDisplayHintText = false;
};
