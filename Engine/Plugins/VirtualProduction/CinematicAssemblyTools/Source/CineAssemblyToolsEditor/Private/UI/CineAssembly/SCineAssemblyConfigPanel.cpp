// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCineAssemblyConfigPanel.h"

#include "AssetToolsModule.h"
#include "CineAssemblyToolsStyle.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/Input/SSegmentedControl.h"

#define LOCTEXT_NAMESPACE "SCineAssemblyConfigPanel"

void SCineAssemblyConfigPanel::Construct(const FArguments& InArgs, UCineAssembly* InAssembly)
{
	CineAssemblyToConfigure = InAssembly;

	TabSwitcher = SNew(SWidgetSwitcher)
		+ SWidgetSwitcher::Slot()
		[MakeDetailsWidget()]

		+ SWidgetSwitcher::Slot()
		[MakeHierarchyWidget()]

		+ SWidgetSwitcher::Slot()
		[MakeNotesWidget()];

	if (InArgs._HideSubAssemblies)
	{
		DetailsView->SetIsCustomRowVisibleDelegate(FIsCustomRowVisible::CreateSP(this, &SCineAssemblyConfigPanel::IsCustomRowVisible));
		DetailsView->SetObject(CineAssemblyToConfigure, true);
	}

	NamingTokenContext = TStrongObjectPtr<UCineAssemblyNamingTokensContext>(NewObject<UCineAssemblyNamingTokensContext>());
	NamingTokenContext->Assembly = InAssembly;

	FilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSegmentedControl<int32>)
						.Value(0)
						.OnValueChanged_Lambda([Switcher = TabSwitcher](int32 NewValue)
							{
								Switcher->SetActiveWidgetIndex(NewValue);
							})

					+ SSegmentedControl<int32>::Slot(0)
						.Text(LOCTEXT("DetailsTab", "Details"))
						.Icon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Details").GetIcon())

					+ SSegmentedControl<int32>::Slot(1)
						.Text(LOCTEXT("HierarchyTab", "Hierarchy"))
						.Icon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FolderClosed").GetIcon())

					+ SSegmentedControl<int32>::Slot(2)
						.Text(LOCTEXT("NotesTab", "Notes"))
						.Icon(FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.Notes").GetIcon())
				]

			+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.HAlign(HAlign_Fill)
				[
					TabSwitcher.ToSharedRef()
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
						.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.PanelBackground"))
						.Padding(16.0f)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(0.0f, 0.0f, 8.0f, 0.0f)
								[
									SNew(STextBlock).Text(LOCTEXT("AssemblyNameField", "Assembly Name"))
								]

							+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.VAlign(VAlign_Center)
								[
									SAssignNew(AssemblyNameTextBox, SNamingTokensEditableTextBox)
										.Contexts({ NamingTokenContext.Get() })
										.FilterArgs(FilterArgs)
										.EvaluationFrequency(1.0f)
										.ShowUnsetTokenWarning(true)
										.OnValidateTokenizedText(this, &SCineAssemblyConfigPanel::ValidateAssemblyName)
										.OnValidateResolvedText(this, &SCineAssemblyConfigPanel::ValidateAssemblyName)
										.Text_Lambda([&Assembly = CineAssemblyToConfigure]() -> FText
											{
												return FText::FromString(Assembly->AssemblyName.Template);
											})
										.OnTextCommitted_Lambda([&Assembly = CineAssemblyToConfigure](const FText& InText, ETextCommit::Type InCommitType)
											{
												Assembly->AssemblyName.Template = InText.ToString();
											})
										.OnTokenizedTextEvaluated_Lambda([&Assembly = CineAssemblyToConfigure](const FText& InText)
											{
												Assembly->AssemblyName.Resolved = InText;
											})
								]
						]
				]
		];
}

bool SCineAssemblyConfigPanel::ValidateAssemblyName(const FText& InText, FText& OutErrorMessage) const
{
	// Ensure that the name does not contain any characters that would be invalid for an asset name
	// This matches the validation that would happen if the user was renaming an asset in the content browser
	FString InvalidCharacters = INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS;

	// These characters are actually valid, because we want to support naming tokens
	InvalidCharacters = InvalidCharacters.Replace(TEXT("{}"), TEXT(""));
	InvalidCharacters = InvalidCharacters.Replace(TEXT(":"), TEXT(""));

	const FString PotentialName = InText.ToString();

	if (!FName::IsValidXName(PotentialName, InvalidCharacters, &OutErrorMessage))
	{
		return false;
	}

	if (PotentialName.Contains(TEXT("{assembly}")) || PotentialName.Contains(TEXT("{cat:assembly}")))
	{
		OutErrorMessage = LOCTEXT("RecursiveAssemblyTokenError", "The {assembly} token cannot be used in an assembly name because it is self-referential.");
		return false;
	}

	return true;
}

void SCineAssemblyConfigPanel::Refresh()
{
	AssemblyNameTextBox->EvaluateNamingTokens();

	// The details view needs to be redrawn to show the new metadata fields from the selected schema
	DetailsView->ForceRefresh();

	// Recreate the hierarchy tree items based on the selected schema
	TreeView->Reinitialize();
}

TSharedRef<SWidget> SCineAssemblyConfigPanel::MakeDetailsWidget()
{
	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bHideSelectionTip = true;

	DetailsView = PropertyEditor.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(CineAssemblyToConfigure, true);

	return SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
				.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("Borders.RecessedNoBorder"))
				.Padding(16.0f)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
						.Padding(0.0f, 0.0f, 0.0f, 16.0f)
						.AutoHeight()
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(0.0f, 0.0f, 8.0f, 0.0f)
								[
									SNew(SImage)
										.Image(this, &SCineAssemblyConfigPanel::GetSchemaThumbnail)
								]

							+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								[
									SNew(SVerticalBox)

									+ SVerticalBox::Slot()
										[
											SNew(STextBlock)
												.Text_Lambda([this]()
													{
														const UCineAssemblySchema* Schema = CineAssemblyToConfigure->GetSchema();
														return Schema ? FText::FromString(Schema->SchemaName) : LOCTEXT("NoSchemaName", "No Schema");
													})
										]

									+ SVerticalBox::Slot()
										[
											SNew(STextBlock)
												.Text(LOCTEXT("SchemClassName", "Cine Assembly Schema"))
												.ColorAndOpacity(FSlateColor::UseSubduedForeground())
										]
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(STextBlock)
								.AutoWrapText(true)
								.Text_Lambda([this]()
									{
										if (const UCineAssemblySchema* Schema = CineAssemblyToConfigure->GetSchema())
										{
											return !Schema->Description.IsEmpty() ? FText::FromString(Schema->Description) : LOCTEXT("EmptyDescription", "No description");
										}
										return LOCTEXT("SchemaInstructions", "Choose a schema to use as the base for configuring your Cine Assembly, or proceed with no schema.");
									})
						]
				]
		]

		+ SVerticalBox::Slot()
		.FillContentHeight(1.0f)
		[
			DetailsView.ToSharedRef()
		];
}

bool SCineAssemblyConfigPanel::IsCustomRowVisible(FName RowName, FName ParentName)
{
	if (ParentName == TEXT("ManagedAssets"))
	{
		return false;
	}
	return true;
}

const FSlateBrush* SCineAssemblyConfigPanel::GetSchemaThumbnail() const
{
	if (const UCineAssemblySchema* Schema = CineAssemblyToConfigure->GetSchema())
	{
		// Find the thumbnail brush associated with the selected schema
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
		TSharedPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(UCineAssemblySchema::StaticClass()).Pin();

		if (AssetTypeActions)
		{
			const FAssetData AssetData = FAssetData(Schema);
			const FName AssetClassName = AssetData.AssetClassPath.GetAssetName();

			return AssetTypeActions->GetThumbnailBrush(AssetData, AssetClassName);
		}
	}

	return FCineAssemblyToolsStyle::Get().GetBrush("ClassThumbnail.CineAssemblySchema");
}

TSharedRef<SWidget> SCineAssemblyConfigPanel::MakeHierarchyWidget()
{
	TreeView = SNew(SCineAssemblyAssetTreeView, CineAssemblyToConfigure)
		.IsReadOnly(true)
		.SelectionMode(ESelectionMode::Single)
		.DisplayHintText(true)
		.ShouldEvaluateTokens(true);

	return SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.Padding(16.0f)
		.AutoHeight()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("HierarchyInstructions", "The following content will be created when this assembly is finalized."))
				.AutoWrapText(true)
		]

		+ SVerticalBox::Slot()
		[
			SNew(SBorder)
				.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.RecessedBackground"))
				.Padding(8.0f)
				[
					TreeView.ToSharedRef()
				]
		];
}

TSharedRef<SWidget> SCineAssemblyConfigPanel::MakeNotesWidget()
{
	return SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.Padding(16.0f)
		.AutoHeight()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("NoteInstructions", "The following notes will be saved with the assembly. This can also be edited later."))
				.AutoWrapText(true)
		]

		+ SVerticalBox::Slot()
		[
			SNew(SBorder)
				.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("Borders.Background"))
				.Padding(16.0f)
				[
					SNew(SMultiLineEditableText)
						.HintText(LOCTEXT("NoteHintText", "Assembly Notes"))
						.Text_Lambda([this]()
							{
								return FText::FromString(CineAssemblyToConfigure->AssemblyNote);
							})
						.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type InCommitType)
							{
								CineAssemblyToConfigure->Modify();
								CineAssemblyToConfigure->AssemblyNote = InText.ToString();
							})
				]
		];
}

#undef LOCTEXT_NAMESPACE