// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CineAssembly.h"
#include "CineAssemblyNamingTokens.h"
#include "IDetailsView.h"
#include "NamingTokensEngineSubsystem.h"
#include "SCineAssemblyAssetTreeView.h"
#include "SNamingTokensEditableTextBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Views/STreeView.h"

/**
 * A widget that wraps details and config steps for setting up a cine assembly asset
 */
class SCineAssemblyConfigPanel : public SCompoundWidget
{
public:
	SCineAssemblyConfigPanel() = default;

	SLATE_BEGIN_ARGS(SCineAssemblyConfigPanel) 
		: _HideSubAssemblies(false)
		{}

		/** If set to true, the SubAssemblies section in the details view will be hidden */
		SLATE_ARGUMENT(bool, HideSubAssemblies)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UCineAssembly* InAssembly);

	/** Refreshes the details view and hierarchy tree view */
	void Refresh();

private:
	/** Creates the widget to display for the Overview tab */
	TSharedRef<SWidget> MakeDetailsWidget();

	/** Creates the widget to display for the Hierarchy tab */
	TSharedRef<SWidget> MakeHierarchyWidget();

	/** Creates the widget to display for the Notes tab */
	TSharedRef<SWidget> MakeNotesWidget();

	/** Validates the user input text for the assembly name */
	bool ValidateAssemblyName(const FText& InText, FText& OutErrorMessage) const;

	/** Filter used by the Details View to determine which custom rows to show */
	bool IsCustomRowVisible(FName RowName, FName ParentName);

	/** Get the thumbnail brush for the selected schema */
	const FSlateBrush* GetSchemaThumbnail() const;

private:
	/** Transient object used only by this UI to configure the properties of the new asset that will get created by the Factory */
	UCineAssembly* CineAssemblyToConfigure = nullptr;

	/** Switcher that controls which tab widget is currently visible */
	TSharedPtr<SWidgetSwitcher> TabSwitcher;

	/** Details View displaying the reflected properties of the Cine Assembly being configured */
	TSharedPtr<IDetailsView> DetailsView;

	/** A TreeView to display the list of other assets and folders that this Assembly will create */
	TSharedPtr<SCineAssemblyAssetTreeView> TreeView;

	/** Naming Token Context object, with a pointer to the assembly being configured */
	TStrongObjectPtr<UCineAssemblyNamingTokensContext> NamingTokenContext;

	/** Naming Token filter properties */
	FNamingTokenFilterArgs FilterArgs;

	/** TextBox displaying the name of the assembly being configured */
	TSharedPtr<SNamingTokensEditableTextBox> AssemblyNameTextBox;
};
