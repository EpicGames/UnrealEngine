// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCineAssemblyConfigWindow.h"

#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "CineAssemblyFactory.h"
#include "CineAssemblySchema.h"
#include "CineAssemblyToolsStyle.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneMetaData.h"
#include "ProductionSettings.h"
#include "UObject/Package.h"
#include "Settings/ContentBrowserSettings.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSplitter.h"

#define LOCTEXT_NAMESPACE "SCineAssemblyConfigWindow"

void SCineAssemblyConfigWindow::Construct(const FArguments& InArgs, const FString& InCreateAssetPath)
{
	CreateAssetPath = InCreateAssetPath;

	// Create a new transient CineAssembly to configure in the UI.
	CineAssemblyToConfigure = TStrongObjectPtr<UCineAssembly>(NewObject<UCineAssembly>(GetTransientPackage(), NAME_None, RF_Transient));
	CineAssemblyToConfigure->Initialize();

	// Initialize the Assembly's name to a good default
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
	const TOptional<FString> DefaultAssemblyName = AssetToolsModule.Get().GetDefaultAssetNameForClass(UCineAssembly::StaticClass());

	CineAssemblyToConfigure->AssemblyName.Template = DefaultAssemblyName.Get(TEXT("NewCineAssembly"));

	if (UMovieSceneMetaData* MetaDataToConfigure = CineAssemblyToConfigure->FindOrAddMetaData<UMovieSceneMetaData>())
	{
		// Default the MetaData for the CineAssembly being configured.
		MetaDataToConfigure->SetCreated(FDateTime::UtcNow());
		MetaDataToConfigure->SetAuthor(FApp::GetSessionOwner());
	}

	UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();
	CineAssemblyToConfigure->Level = FSoftObjectPath(CurrentWorld);

	const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
	TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
	
	if (ActiveProduction.IsSet())
	{
		CineAssemblyToConfigure->Production = ActiveProduction.GetValue().ProductionID;
		CineAssemblyToConfigure->ProductionName = ActiveProduction.GetValue().ProductionName;
	}

	const FVector2D DefaultWindowSize = FVector2D(1400, 750);

	SWindow::Construct(SWindow::FArguments()
		.Title(LOCTEXT("WindowTitleCreateNew", "Create Cine Assembly"))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(DefaultWindowSize)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			SNew(SBorder)
				.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.PanelBackground"))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
						[
							SNew(SSplitter)
								.Orientation(Orient_Horizontal)
								.PhysicalSplitterHandleSize(2.0f)

								+ SSplitter::Slot()
								.Value(0.7f)
								[
									MakeSchemaPanel()
								]

								+ SSplitter::Slot()
								.Value(0.3f)
								[
									SAssignNew(ConfigPanel, SCineAssemblyConfigPanel, CineAssemblyToConfigure.Get())
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SSeparator)
								.Orientation(Orient_Horizontal)
								.Thickness(2.0f)
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							MakeButtonsPanel()
						]
				]
		]);
}

SCineAssemblyConfigWindow::~SCineAssemblyConfigWindow()
{
	// Save the UI config settings for whether to display engine/plugin content
	if (UContentBrowserSettings* ContentBrowserSettings = GetMutableDefault<UContentBrowserSettings>())
	{
		const bool bShowEngineContent = ContentBrowserSettings->GetDisplayEngineFolder();
		const bool bShowPluginContent = ContentBrowserSettings->GetDisplayPluginFolders();

		if (GConfig)
		{
			GConfig->SetBool(TEXT("NewCineAssemblyUI"), TEXT("bShowEngineContent"), bShowEngineContent, GEditorPerProjectIni);
			GConfig->SetBool(TEXT("NewCineAssemblyUI"), TEXT("bShowPluginContent"), bShowPluginContent, GEditorPerProjectIni);
		}

		ContentBrowserSettings->SetDisplayEngineFolder(bShowEngineContentCached);
		ContentBrowserSettings->SetDisplayPluginFolders(bShowPluginContentCached);
	}
}

TSharedRef<SWidget> SCineAssemblyConfigWindow::MakeSchemaPanel()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	// The asset picker will only display Cine Assembly Schema assets
	FAssetPickerConfig Config;
	Config.Filter.ClassPaths.Add(UCineAssemblySchema::StaticClass()->GetClassPathName());
	Config.SelectionMode = ESelectionMode::Single;
	Config.InitialAssetViewType = EAssetViewType::Tile;
	Config.bFocusSearchBoxWhenOpened = false;
	Config.bShowBottomToolbar = false;
	Config.bAllowDragging = false;
	Config.bAllowRename = false;
	Config.bCanShowClasses = false;
	Config.bCanShowFolders = false;
	Config.bForceShowEngineContent = false;
	Config.bForceShowPluginContent = false;
	Config.bAddFilterUI = false;
	Config.bShowPathInColumnView = true;
	Config.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SCineAssemblyConfigWindow::OnSchemaSelected);

	const FString PackageName = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetMountedAssetPath();
	const FString NoSchemaName = TEXT("NoSchema");
	const FString PackagePath = PackageName / NoSchemaName;

	FAssetData NoSchemaAssetData = FAssetData(*PackagePath, *PackageName, *NoSchemaName, FTopLevelAssetPath());

	// Add a fake asset to the list (so that it appears as a tile in the asset picker) that represents a selection of no schema
	Config.OnGetCustomSourceAssets = FOnGetCustomSourceAssets::CreateLambda([NoSchemaAssetData](const FARFilter& Filter, TArray<FAssetData>& OutAssets)
		{
			OutAssets.Add(NoSchemaAssetData);
		});

	Config.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([NoSchemaAssetData](const FAssetData& AssetData) -> bool
		{
			const FName IsHiddenPropertyName = GET_MEMBER_NAME_CHECKED(UCineAssemblySchema, bIsHidden);

			bool bIsSchemaHidden = false;
			AssetData.GetTagValue(IsHiddenPropertyName, bIsSchemaHidden);

			// Filter out any schemas that should be hidden
			return (AssetData == NoSchemaAssetData) ? false : bIsSchemaHidden;
		});

	Config.InitialAssetSelection = NoSchemaAssetData;

	// The fake NoSchema asset should not display the normal asset tooltip, just a plain text-based tooltip describing what it is
	Config.OnIsAssetValidForCustomToolTip = FOnIsAssetValidForCustomToolTip::CreateLambda([NoSchemaAssetData](FAssetData& AssetData)
		{
			return (AssetData == NoSchemaAssetData) ? true : false;
		});

	Config.OnGetCustomAssetToolTip = FOnGetCustomAssetToolTip::CreateLambda([](FAssetData& AssetData) -> TSharedRef<SToolTip>
		{
			return SNew(SToolTip).Text(LOCTEXT("NoSchemaToolTip", "Create a new assembly with no schema"));
		});

	// Check the UI config settings to determine whether or not to display engine/plugin content by default in this window
	UContentBrowserSettings* ContentBrowserSettings = GetMutableDefault<UContentBrowserSettings>();

	bool bShowEngineContent = true;
	bool bShowPluginContent = true;
	GConfig->GetBool(TEXT("NewCineAssemblyUI"), TEXT("bShowEngineContent"), bShowEngineContent, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("NewCineAssemblyUI"), TEXT("bShowPluginContent"), bShowPluginContent, GEditorPerProjectIni);

	bShowEngineContentCached = ContentBrowserSettings->GetDisplayEngineFolder();
	bShowPluginContentCached = ContentBrowserSettings->GetDisplayPluginFolders();

	ContentBrowserSettings->SetDisplayEngineFolder(bShowEngineContent);
	ContentBrowserSettings->SetDisplayPluginFolders(bShowPluginContent);

	return SNew(SBorder)
		.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.PanelBackground"))
		.Padding(16.0f)
		[
			ContentBrowserModule.Get().CreateAssetPicker(Config)
		];
}

void SCineAssemblyConfigWindow::OnSchemaSelected(const FAssetData& AssetData)
{
	const FString NoSchemaName = TEXT("NoSchema");
	if (AssetData.AssetName == NoSchemaName)
	{
		CineAssemblyToConfigure->ChangeSchema(nullptr);
		CineAssemblyToConfigure->Initialize();

		// Re-initialize the Assembly's name to a good default
		FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
		const TOptional<FString> DefaultAssemblyName = AssetToolsModule.Get().GetDefaultAssetNameForClass(UCineAssembly::StaticClass());

		CineAssemblyToConfigure->AssemblyName.Template = DefaultAssemblyName.Get(TEXT("NewCineAssembly"));

		ConfigPanel->Refresh();
	}
	else if (UCineAssemblySchema* CineAssemblySchema = Cast<UCineAssemblySchema>(AssetData.GetAsset()))
	{
		CineAssemblyToConfigure->InitializeFromSchema(CineAssemblySchema);

		// If the schema does not override the default Level, reset to the current editor world
		if (!CineAssemblySchema->bOverrideDefaultLevel)
		{
			UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();
			CineAssemblyToConfigure->Level = FSoftObjectPath(CurrentWorld);
		}

		ConfigPanel->Refresh();
	}
}

bool SCineAssemblyConfigWindow::ValidateAssetName(FText& OutErrorMessage) const
{
	if (CineAssemblyToConfigure.IsValid() && CineAssemblyToConfigure.Get()->AssemblyName.Resolved.ToString().Contains(TEXT("/")))
	{
		OutErrorMessage = LOCTEXT("InvalidAssemblyNameErrorMessage", "Assembly names cannot contain '/'");
		return false;
	}
	const FString AssemblyPackageName = UCineAssemblyFactory::MakeAssemblyPackageName(CineAssemblyToConfigure.Get(), CreateAssetPath);
	return AssetViewUtils::IsValidPackageForCooking(AssemblyPackageName, OutErrorMessage);
}

bool SCineAssemblyConfigWindow::ValidateAssetName() const
{
	FText OutErrorMessage;
	return ValidateAssetName(OutErrorMessage);
}

TSharedRef<SWidget> SCineAssemblyConfigWindow::MakeButtonsPanel()
{
	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(16.0f)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SButton)
						.Text(this, &SCineAssemblyConfigWindow::GetCreateButtonText)
						.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
						.HAlign(HAlign_Center)
						.OnClicked(this, &SCineAssemblyConfigWindow::OnCreateAssetClicked)
						.IsEnabled_Raw(this, &SCineAssemblyConfigWindow::ValidateAssetName)
						.ToolTipText_Lambda([this]() -> FText
							{
								FText OutErrorMessage;
								return !ValidateAssetName(OutErrorMessage) ? OutErrorMessage : FText();
							})
				]

			+ SHorizontalBox::Slot()
				.MinWidth(118.0f)
				.MaxWidth(118.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
						.Text(LOCTEXT("CancelButton", "Cancel"))
						.HAlign(HAlign_Center)
						.OnClicked(this, &SCineAssemblyConfigWindow::OnCancelClicked)
				]
		];
}

FText SCineAssemblyConfigWindow::GetCreateButtonText() const
{
	if (const UCineAssemblySchema* Schema = CineAssemblyToConfigure->GetSchema())
	{
		if (!Schema->SchemaName.IsEmpty())
		{
			return FText::Format(LOCTEXT("CreateAssetButtonWithSchema", "Create {0}"), FText::FromString(Schema->SchemaName));
		}
	}
	return LOCTEXT("CreateAssetButton", "Create Assembly");
}

FReply SCineAssemblyConfigWindow::OnCreateAssetClicked()
{
	if (ValidateAssetName())
	{
		UCineAssemblyFactory::CreateConfiguredAssembly(CineAssemblyToConfigure.Get(), CreateAssetPath);
		UCineAssemblyFactory::DuplicateSubsequences(CineAssemblyToConfigure.Get());
		UCineAssemblyFactory::DuplicateAssociatedAssets(CineAssemblyToConfigure.Get());
		RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SCineAssemblyConfigWindow::OnCancelClicked()
{
	RequestDestroyWindow();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
