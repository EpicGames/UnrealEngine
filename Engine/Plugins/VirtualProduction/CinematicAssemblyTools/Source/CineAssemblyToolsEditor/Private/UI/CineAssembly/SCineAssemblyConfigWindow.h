// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CineAssembly.h"
#include "CineAssemblySchema.h"
#include "SCineAssemblyConfigPanel.h"
#include "Widgets/SWindow.h"

/**
 * A window for configuring the properties of a UCineAssembly asset
 */
class SCineAssemblyConfigWindow : public SWindow
{
public:
	SCineAssemblyConfigWindow() = default;
	~SCineAssemblyConfigWindow();

	SLATE_BEGIN_ARGS(SCineAssemblyConfigWindow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FString& InCreateAssetPath);

private:
	/** Creates the panel that displays the available Cine Assembly Schemas */
	TSharedRef<SWidget> MakeSchemaPanel();

	/** Creates the buttons on the bottom of the window */
	TSharedRef<SWidget> MakeButtonsPanel();

	/** Returns the text to display on the Create Asset button, based on the selected schema */
	FText GetCreateButtonText() const;

	/** Closes the window and indicates that a new asset should be created by the asset factory */
	FReply OnCreateAssetClicked();

	/** Closes the window and indicates that no assets should be created by the asset factory */
	FReply OnCancelClicked();

	/** Updates the UI and CineAssembly properties based on the selected schema */
	void OnSchemaSelected(const FAssetData& AssetData);

	/** Validate the asset name is acceptable. */
	bool ValidateAssetName(FText& OutErrorMessage) const;
	bool ValidateAssetName() const;

private:
	/** Transient object used only by this UI to configure the properties of the new asset that will get created by the Factory */
	TStrongObjectPtr<UCineAssembly> CineAssemblyToConfigure = nullptr;

	/** The widget that display all of the configuration properties for the Cine Assembly asset */
	TSharedPtr<SCineAssemblyConfigPanel> ConfigPanel;

	/** The root path where the configured assembly will be created */
	FString CreateAssetPath;

	/** Cached content browser settings, used to restore defaults when closing the window */
	bool bShowEngineContentCached = false;
	bool bShowPluginContentCached = false;
};
