// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCineAssemblyPropertyEditor.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetThumbnail.h"
#include "CineAssemblyToolsStyle.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SLevelSequenceFavoriteRating.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SCineAssemblyPropertyEditor"

void SCineAssemblyPropertyEditor::Construct(const FArguments& InArgs, UCineAssembly* InAssembly)
{
	CineAssembly = InAssembly;
	ChildSlot[BuildUI()];
}

void SCineAssemblyPropertyEditor::Construct(const FArguments& InArgs, FGuid InAssemblyGuid)
{
	// The UI will be temporary because no CineAssembly has been found yet
	ChildSlot[BuildUI()];

	// If the asset registry is still scanning assets, add a callback to find the assembly asset matching the input GUID and update this widget once the scan is finished.
	// Otherwise, we can find the assembly asset and update the UI immediately. 
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (AssetRegistryModule.Get().IsLoadingAssets())
	{
		AssetRegistryModule.Get().OnFilesLoaded().AddSP(this, &SCineAssemblyPropertyEditor::FindAssembly, InAssemblyGuid);
	}
	else
	{
		FindAssembly(InAssemblyGuid);
	}
}

SCineAssemblyPropertyEditor::~SCineAssemblyPropertyEditor()
{
	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry"))
	{
		AssetRegistryModule->Get().OnFilesLoaded().RemoveAll(this);
	}
}

TSharedRef<SWidget> SCineAssemblyPropertyEditor::BuildUI()
{
	// Build a temporary UI to display while waiting for the assembly to be loaded
	if (!CineAssembly)
	{
		return SNew(SBorder)
			.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("Borders.PanelNoBorder"))
			.Padding(8.0f)
			[
				SNew(STextBlock).Text(LOCTEXT("LoadingAssemblyText", "Loading Cine Assembly..."))
			];
	}

	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bHideSelectionTip = true;

	TSharedRef<IDetailsView> DetailsView = PropertyEditor.CreateDetailView(DetailsViewArgs);
	DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SCineAssemblyPropertyEditor::IsPropertyVisible));
	DetailsView->SetIsCustomRowVisibleDelegate(FIsCustomRowVisible::CreateSP(this, &SCineAssemblyPropertyEditor::IsCustomRowVisible));

	DetailsView->SetObject(CineAssembly, true);

	TabSwitcher = SNew(SWidgetSwitcher)
		+ SWidgetSwitcher::Slot()
		[MakeOverviewWidget()]

		+ SWidgetSwitcher::Slot()
		[DetailsView];

	return SNew(SBorder)
		.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("Borders.PanelNoBorder"))
		.Padding(8.0f)
		[
			SNew(SScrollBox)
				.Orientation(Orient_Vertical)

			+ SScrollBox::Slot()
				.AutoSize()
				.HAlign(HAlign_Center)
				.Padding(0.0f, 0.0f, 0.0f, 8.0f)
				[
					SNew(SSegmentedControl<int32>)
						.Value(0)
						.OnValueChanged_Lambda([Switcher = TabSwitcher](int32 NewValue)
							{
								Switcher->SetActiveWidgetIndex(NewValue);
							})

					+ SSegmentedControl<int32>::Slot(0)
						.Text(LOCTEXT("OverviewTab", "Overview"))
						.Icon(FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.Animation").GetIcon())

					+ SSegmentedControl<int32>::Slot(1)
						.Text(LOCTEXT("DetailsTab", "Details"))
						.Icon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Details").GetIcon())
				]

			+ SScrollBox::Slot()
				.FillSize(1.0f)
				.HAlign(HAlign_Fill)
				[
					TabSwitcher.ToSharedRef()
				]
		];
}

bool SCineAssemblyPropertyEditor::IsPropertyVisible(const FPropertyAndParent& PropertyAndParent)
{
	const FName PropertyName = PropertyAndParent.Property.GetFName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UCineAssembly, InstanceMetadata))
	{
		return false;
	}
	return true;
}

bool SCineAssemblyPropertyEditor::IsCustomRowVisible(FName RowName, FName ParentName)
{
	if (RowName == GET_MEMBER_NAME_CHECKED(UCineAssembly, SubAssemblyNames))
	{
		return false;
	}
	return true;
}

void SCineAssemblyPropertyEditor::FindAssembly(FGuid AssemblyID)
{
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");

	// The only search criterion for the asset search is for an asset with an AssemblyID matching the input GUID
	const TMultiMap<FName, FString> TagValues = { { UCineAssembly::AssemblyGuidPropertyName, AssemblyID.ToString() } };

	TArray<FAssetData> AssemblyAssets;
	AssetRegistryModule.Get().GetAssetsByTagValues(TagValues, AssemblyAssets);

	// The Assembly ID is unique, so at most one asset should ever be found
	if (AssemblyAssets.Num() > 0)
	{
		CineAssembly = Cast<UCineAssembly>(AssemblyAssets[0].GetAsset());

		// Update the widget's UI
		ChildSlot.DetachWidget();
		ChildSlot.AttachWidget(BuildUI());
	}
}

FString SCineAssemblyPropertyEditor::GetAssemblyName()
{
	if (CineAssembly)
	{
		FString AssemblyName;
		CineAssembly->GetName(AssemblyName);
		return AssemblyName;
	}
	return TEXT("CineAssembly");
}

bool SCineAssemblyPropertyEditor::HasRenderedThumbnail()
{
	const FObjectThumbnail* ObjectThumbnail = nullptr;
	if (CineAssembly)
	{
		const FName FullAssetName = *CineAssembly->GetFullName();

		FThumbnailMap ThumbnailMap;
		ThumbnailTools::ConditionallyLoadThumbnailsForObjects({ FullAssetName }, ThumbnailMap);
		ObjectThumbnail = ThumbnailMap.Find(FullAssetName);
	}

	const bool bHasRenderedThumbnail = ObjectThumbnail && !ObjectThumbnail->IsEmpty();
	return bHasRenderedThumbnail;
}

TSharedRef<SWidget> SCineAssemblyPropertyEditor::MakeOverviewWidget()
{
	const UCineAssemblySchema* Schema = CineAssembly->GetSchema();

	auto GetVisbility = [Schema]() -> EVisibility
		{
			return Schema ? EVisibility::Visible : EVisibility::Collapsed;
		};

	TSharedRef<FAssetThumbnail> AssetThumbnail = MakeShared<FAssetThumbnail>(CineAssembly, 256, 256, UThumbnailManager::Get().GetSharedThumbnailPool());
	FAssetThumbnailConfig ThumbnailConfig;
	ThumbnailConfig.AssetTypeColorOverride = FLinearColor::Transparent;

	return SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 0.0f, 0.0f, 16.0f)
		[
			SNew(SBorder)
				.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.RecessedBackground"))
				.Padding(4.0f)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 8.0f, 0.0f, 8.0f)
						.HAlign(HAlign_Center)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									AssetThumbnail->MakeThumbnailWidget(ThumbnailConfig)
								]

							+ SHorizontalBox::Slot()
								.Padding(16.0f, 0.0, 0.0, 0.0f)
								.VAlign(VAlign_Center)
								.FillContentWidth(1.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("ThumbnailHintText", "This assembly does not currently have a preview thumbnail. Open this asset in Sequencer and save it to render a preview to display here."))
										.AutoWrapText(true)
										.Visibility_Lambda([this]() { return HasRenderedThumbnail() ? EVisibility::Collapsed : EVisibility::Visible; })
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Center)
						.Padding(0.0f, 0.0f, 0.0f, 4.0f)
						[
							SNew(STextBlock).Text_Lambda([this]() {	return FText::FromString(GetAssemblyName()); })
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Center)
						.Padding(0.0f, 0.0f, 0.0f, 8.0f)
						[
							SNew(STextBlock)
								.Text_Lambda([Schema]() { return Schema ? FText::FromString(Schema->SchemaName) : LOCTEXT("NoSchemaName", "No Schema"); })
								.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Center)
						.Padding(0.0f, 0.0f, 0.0f, 8.0f)
						[
							SNew(SLevelSequenceFavoriteRating)
								.LevelSequence(CineAssembly)
						]
				]
		]

		+ SVerticalBox::Slot()
		.MinHeight(300.0f)
		.FillContentHeight(1.0f)
		[
			SNew(SBorder)
				.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("Borders.Background"))
				.Padding(16.0f)
				[
					SNew(SMultiLineEditableText)
						.HintText(LOCTEXT("NoteHintText", "Assembly Notes"))
						.Text_Lambda([this]()
							{
								return FText::FromString(CineAssembly->AssemblyNote);
							})
						.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type InCommitType)
							{
								if (CineAssembly->AssemblyNote != InText.ToString())
								{
									CineAssembly->Modify();
									CineAssembly->AssemblyNote = InText.ToString();
								}
							})
				]
		];
}

#undef LOCTEXT_NAMESPACE
