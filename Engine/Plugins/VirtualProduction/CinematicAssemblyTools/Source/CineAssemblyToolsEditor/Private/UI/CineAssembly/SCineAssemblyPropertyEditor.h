// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CineAssembly.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

struct FPropertyAndParent;

/** 
 * A panel that displays the editable properties of a Cine Assembly asset 
 */
class SCineAssemblyPropertyEditor : public SCompoundWidget
{
public:
	SCineAssemblyPropertyEditor() = default;
	~SCineAssemblyPropertyEditor();

	SLATE_BEGIN_ARGS(SCineAssemblyPropertyEditor) {}
	SLATE_END_ARGS()

	/** Widget construction, initialized with the assembly asset being edited */
	void Construct(const FArguments& InArgs, UCineAssembly* InAssembly);

	/**
	 * Widget construction, initialized with the GUID of the assembly to be edited
	 * The widget will search the asset registry to find the assembly asset with the matching GUID,
	 * and then update the widget contents accordingly.
	 */
	void Construct(const FArguments& InArgs, FGuid InAssemblyGuid);

	/** Returns the name of the assembly asset being edited */
	FString GetAssemblyName();

	/** Searches the asset registry for a Cine Assembly matching the input ID and updates the UI */
	void FindAssembly(FGuid AssemblyID);

	/** Returns true if the Assembly asset has a rendered thumbnail (such as from the Sequencer preview) */
	bool HasRenderedThumbnail();

private:
	/** Constructs the main UI for the widget */
	TSharedRef<SWidget> BuildUI();

	/** Creates the widget to display for the Overview tab */
	TSharedRef<SWidget> MakeOverviewWidget();

	/** Filter used by the Details View to determine which properties to show */
	bool IsPropertyVisible(const FPropertyAndParent& PropertyAndParent);

	/** Filter used by the Details View to determine which custom rows to show */
	bool IsCustomRowVisible(FName RowName, FName ParentName);

private:
	/** The assembly asset whose properties are displayed by this widget */
	UCineAssembly* CineAssembly = nullptr;

	/** Switcher that controls which tab widget is currently visible */
	TSharedPtr<SWidgetSwitcher> TabSwitcher;
};
