// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCineAssemblySchemaWindow.h"

#include "Algo/Contains.h"
#include "Algo/Find.h"
#include "AssetDefinitionRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "CineAssemblyEditorFunctionLibrary.h"
#include "CineAssemblyNamingTokens.h"
#include "CineAssemblySchemaFactory.h"
#include "CineAssemblyToolsStyle.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "ISequencerModule.h"
#include "IStructureDetailsView.h"
#include "LevelSequenceEditorSubsystem.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneFolder.h"
#include "MovieSceneSubAssemblySection.h"
#include "MovieSceneSubAssemblyTrack.h"
#include "ProductionSettings.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "Sections/MovieSceneSubSection.h"
#include "Sequencer/CineAssemblySequencerUtilities.h"
#include "Sequencer/CineAssemblySchemaSpawnRegister.h"
#include "SequencerSettings.h"
#include "Settings/ContentBrowserSettings.h"
#include "SPositiveActionButton.h"
#include "Styling/StyleColors.h"
#include "TimerManager.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Tracks/MovieSceneCVarTrack.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "UObject/Package.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SAssetMenuIcon.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SCineAssemblySchemaWindow"

void SCineAssemblySchemaWindow::Construct(const FArguments& InArgs, const FString& InCreateAssetPath)
{
	// Create a new transient CineAssemblySchema to configure in the UI.
	// If the configuration is successful, this will turn into the persistent object created by the factory.
	NewSchemaObject.Reset(NewObject<UCineAssemblySchema>(GetTransientPackage(), NAME_None, RF_Transient));
	SchemaToConfigure = NewSchemaObject.Get();
	Mode = ESchemaConfigMode::CreateNew;

	CreateAssetPath = InCreateAssetPath;

	ChildSlot [ BuildUI() ];
}

void SCineAssemblySchemaWindow::Construct(const FArguments& InArgs, UCineAssemblySchema* InSchema)
{
	SchemaToConfigure = InSchema;
	Mode = ESchemaConfigMode::Edit;

	ChildSlot [ BuildUI() ];
}

void SCineAssemblySchemaWindow::Construct(const FArguments& InArgs, FGuid InSchemaGuid)
{
	Mode = ESchemaConfigMode::Edit;

	// The UI will be temporary because no CineAssemblySchema has been found yet
	ChildSlot [ BuildUI() ];

	// If the asset registry is still scanning assets, add a callback to find the schema asset matching the input GUID and update this widget once the scan is finished.
	// Otherwise, we can find the schema asset and update the UI immediately. 
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (AssetRegistryModule.Get().IsLoadingAssets())
	{
		AssetRegistryModule.Get().OnFilesLoaded().AddSP(this, &SCineAssemblySchemaWindow::FindSchema, InSchemaGuid);
	}
	else
	{
		FindSchema(InSchemaGuid);
	}
}

TSharedRef<SWidget> SCineAssemblySchemaWindow::BuildUI()
{
	// Build a temporary UI to display while waiting for the schema to be loaded
	if (!SchemaToConfigure)
	{
		return SNew(SBorder)
			.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("Borders.PanelNoBorder"))
			.Padding(8.0f)
			[
				SNew(STextBlock).Text(LOCTEXT("LoadingSchemaText", "Loading Cine Assembly Schema..."))
			];
	}

	// Register a delegate to clean up this window before the schema asset being edited is destroyed.
	// This is only a real concern when editing an existing Schema asset, not creating a new one.
	// Guarded against re-registration in case BuildUI is ever called more than once with a valid schema (e.g. a future reload path).
	if (!OnAssetsPreDeleteHandle.IsValid())
	{
		OnAssetsPreDeleteHandle = FEditorDelegates::OnAssetsPreDelete.AddSP(this, &SCineAssemblySchemaWindow::OnAssetsPreDelete);
	}

	// Check the UI config settings to determine whether or not to display engine/plugin content by default in this window
	UContentBrowserSettings* ContentBrowserSettings = GetMutableDefault<UContentBrowserSettings>();

	bool bShowEngineContent = true;
	bool bShowPluginContent = true;
	GConfig->GetBool(TEXT("NewCineAssemblySchemaUI"), TEXT("bShowEngineContent"), bShowEngineContent, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("NewCineAssemblySchemaUI"), TEXT("bShowPluginContent"), bShowPluginContent, GEditorPerProjectIni);

	bShowEngineContentCached = ContentBrowserSettings->GetDisplayEngineFolder();
	bShowPluginContentCached = ContentBrowserSettings->GetDisplayPluginFolders();

	ContentBrowserSettings->SetDisplayEngineFolder(bShowEngineContent);
	ContentBrowserSettings->SetDisplayPluginFolders(bShowPluginContent);
	
	// Validate and fix any invalid bindings or tracks in the Schema's TemplateSequence before opening it in the Sequencer widget
	ValidateTemplateSequence();

	// Construct the Sequencer widget to view and modify the Schema's TemplateSequence
	InitializeSequencer();

	return SNew(SBorder)
		.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.PanelBackground"))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
				[
					SNew(SSplitter)
						.Orientation(Orient_Horizontal)
						.PhysicalSplitterHandleSize(2.0f)

					+ SSplitter::Slot()
						.Value(0.2f)
						[
							MakeMenuPanel()
						]

					+ SSplitter::Slot()
						.Value(0.8f)
						[
							MakeContentPanel()
						]
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator)
						.Orientation(Orient_Horizontal)
						.Thickness(2.0f)
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					MakeButtonsPanel()
				]
		];
}

SCineAssemblySchemaWindow::~SCineAssemblySchemaWindow()
{
	FEditorDelegates::OnAssetsPreDelete.Remove(OnAssetsPreDeleteHandle);

	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry"))
	{
		AssetRegistryModule->Get().OnFilesLoaded().RemoveAll(this);
	}

	// Save the UI config settings for whether to display engine/plugin content
	if (UContentBrowserSettings* ContentBrowserSettings = GetMutableDefault<UContentBrowserSettings>())
	{
		const bool bShowEngineContent = ContentBrowserSettings->GetDisplayEngineFolder();
		const bool bShowPluginContent = ContentBrowserSettings->GetDisplayPluginFolders();
		
		if (GConfig)
		{
			GConfig->SetBool(TEXT("NewCineAssemblySchemaUI"), TEXT("bShowEngineContent"), bShowEngineContent, GEditorPerProjectIni);
			GConfig->SetBool(TEXT("NewCineAssemblySchemaUI"), TEXT("bShowPluginContent"), bShowPluginContent, GEditorPerProjectIni);
		}

		ContentBrowserSettings->SetDisplayEngineFolder(bShowEngineContentCached);
		ContentBrowserSettings->SetDisplayPluginFolders(bShowPluginContentCached);
	}
}

void SCineAssemblySchemaWindow::InitializeSequencer()
{
	TSharedRef<FCineAssemblySchemaSpawnRegister> SpawnRegister = MakeShared<FCineAssemblySchemaSpawnRegister>();

	FSequencerInitParams SequencerInitParams;
	SequencerInitParams.RootSequence = SchemaToConfigure->TemplateSequence;
	SequencerInitParams.bEditWithinLevelEditor = false;
	SequencerInitParams.SpawnRegister = SpawnRegister;

	SequencerInitParams.ViewParams.UniqueName = TEXT("CineAssemblySchemaTemplateSequencer");

	SequencerInitParams.ViewParams.ToolbarExtender = MakeShared<FExtender>();
	SequencerInitParams.ViewParams.ToolbarExtender->AddToolBarExtension("BaseCommands", EExtensionHook::First, nullptr, FToolBarExtensionDelegate::CreateRaw(this, &SCineAssemblySchemaWindow::ExtendSequencerToolbarBaseCommands));
	SequencerInitParams.ViewParams.ToolbarExtender->AddToolBarExtension("Snapping", EExtensionHook::After, nullptr, FToolBarExtensionDelegate::CreateSP(this, &SCineAssemblySchemaWindow::AddSequencerToolbarExtension));

	SequencerInitParams.HostCapabilities.bSupportsAddFromContentBrowser = true;
	SequencerInitParams.HostCapabilities.bSupportsDragAndDrop = true;
	SequencerInitParams.HostCapabilities.bSupportsSidebar = true;
	SequencerInitParams.HostCapabilities.bSupportsSimpleView = false;

	Sequencer = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer").CreateSequencer(SequencerInitParams);
	Sequencer->OnActorAddedToSequencer().AddSP(this, &SCineAssemblySchemaWindow::OnActorAddedToSequencer);
	Sequencer->OnMovieSceneDataChanged().AddSP(this, &SCineAssemblySchemaWindow::OnMovieSceneDataChanged);

	// Force bAutoSetTrackDefaults to true
	if (USequencerSettings* SequencerSettings = Sequencer->GetSequencerSettings())
	{
		if (!SequencerSettings->GetAutoSetTrackDefaults())
		{
			SequencerSettings->SetAutoSetTrackDefaults(true);
		}
	}
}

void SCineAssemblySchemaWindow::FindSchema(FGuid SchemaID)
{
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");

	// The only search criterion for the asset search is for an asset with an SchemaID matching the input GUID
	const TMultiMap<FName, FString> TagValues = { { UCineAssemblySchema::SchemaGuidPropertyName, SchemaID.ToString() } };

	TArray<FAssetData> SchemaAssets;
	AssetRegistryModule.Get().GetAssetsByTagValues(TagValues, SchemaAssets);

	// The Schema ID is unique, so at most one asset should ever be found
	if (SchemaAssets.Num() > 0)
	{
		SchemaToConfigure = Cast<UCineAssemblySchema>(SchemaAssets[0].GetAsset());

		// Update the widget's UI
		ChildSlot.DetachWidget();
		ChildSlot.AttachWidget(BuildUI());
	}
}

FString SCineAssemblySchemaWindow::GetSchemaName()
{
	if (SchemaToConfigure)
	{
		FString SchemaName;
		SchemaToConfigure->GetName(SchemaName);
		return SchemaName;
	}
	return TEXT("CineAssemblySchema");
}

void SCineAssemblySchemaWindow::OnWindowClosed(const TSharedRef<SWindow>& InWindow)
{
	if (Sequencer)
	{
		Sequencer->Close();
	}
}

void SCineAssemblySchemaWindow::OnAssetsPreDelete(const TArray<UObject*>& AssetsToDelete)
{
	// Defensive checks to be sure we are editing an existing Schema asset, and that asset is about to be deleted
	if (Mode != ESchemaConfigMode::Edit || !SchemaToConfigure || !AssetsToDelete.Contains(SchemaToConfigure))
	{
		return;
	}

	// Close the Sequencer, because it currently holds a reference to the Schema's TemplateSequence, which will prevent it from being deleted cleanly.
	if (Sequencer)
	{
		Sequencer->Close();
		Sequencer.Reset();
	}

	TreeView.Reset();
	SchemaToConfigure = nullptr;

	// Detach the remaining UI widgets which may try to update the Schema properties after it is deleted, and replace with placeholder text until the window is automatically closed.
	ChildSlot.DetachWidget();
	ChildSlot.AttachWidget(
		SNew(SBorder)
			.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("Borders.PanelNoBorder"))
			.Padding(8.0f)
			[
				SNew(STextBlock).Text(LOCTEXT("ClosingSchemaText", "Closing Cine Assembly Schema..."))
			]
	);
}

bool SCineAssemblySchemaWindow::DoesSchemaExistWithName(const FString& SchemaName) const
{
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
	TArray<FAssetData> SchemaAssets;
	AssetRegistryModule.Get().GetAssetsByClass(UCineAssemblySchema::StaticClass()->GetClassPathName(), SchemaAssets);

	return Algo::ContainsBy(SchemaAssets, *SchemaName, &FAssetData::AssetName);
}

bool SCineAssemblySchemaWindow::ValidateSchemaName(const FText& InText, FText& OutErrorMessage) const
{
	if (InText.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("EmptyNameErrorMessage", "Please provide a name for the schema");
		return false;
	}

	if (!SchemaToConfigure)
	{
		return false;
	}

	const FString DesiredPackageName = !CreateAssetPath.IsEmpty()
		? CreateAssetPath / InText.ToString()
		: FPackageName::GetLongPackagePath(SchemaToConfigure->GetPackage()->GetPathName()) / InText.ToString();

	if (!AssetViewUtils::IsValidPackageForCooking(DesiredPackageName, OutErrorMessage))
	{
		return false;
	}

	// It is valid if the input text matches the schema's current name
	if (SchemaToConfigure->SchemaName == InText.ToString())
	{
		return true;
	}

	if (DoesSchemaExistWithName(InText.ToString()))
	{
		OutErrorMessage = LOCTEXT("DuplicateNameErrorMessage", "A schema with that name already exists");
		return false;
	}

	return FName::IsValidXName(InText.ToString(), INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, &OutErrorMessage);
}

TSharedRef<SWidget> SCineAssemblySchemaWindow::MakeMenuPanel()
{
	return SNew(SBorder)
		.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.RecessedBackground"))
		.Padding(8.0f)
		.VAlign(VAlign_Top)
		[
			SNew(SSegmentedControl<int32>)
				.Style(&FCineAssemblyToolsStyle::Get().GetWidgetStyle<FSegmentedControlStyle>("PrimarySegmentedControl"))
				.MaxSegmentsPerLine(1)
				.Value_Lambda([this]()
				{ 
					return MenuTabSwitcher->GetActiveWidgetIndex();
				})
				.OnValueChanged_Lambda([this](int32 NewValue)
				{
					MenuTabSwitcher->SetActiveWidgetIndex(NewValue);
				})

				+ SSegmentedControl<int32>::Slot(0)
				.Text(LOCTEXT("DetailsTab", "Details"))
				.Icon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Details").GetIcon())
				.HAlign(HAlign_Left)

				+ SSegmentedControl<int32>::Slot(1)
				.Text(LOCTEXT("MetadataTab", "Metadata"))
				.Icon(FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.DataAsset").GetIcon())
				.HAlign(HAlign_Left)

				+ SSegmentedControl<int32>::Slot(2)
				.Text(LOCTEXT("TimelineTab", "Timeline Template"))
				.Icon(FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.Sequencer").GetIcon())
				.HAlign(HAlign_Left)
				.Visibility_Lambda([this]() { return SchemaToConfigure && SchemaToConfigure->bIsDataOnly ? EVisibility::Collapsed : EVisibility::Visible; })

				+ SSegmentedControl<int32>::Slot(3)
				.Text(LOCTEXT("HierarchyTab", "Content Hierarchy"))
				.Icon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FolderClosed").GetIcon())
				.HAlign(HAlign_Left)
		];
}

TSharedRef<SWidget> SCineAssemblySchemaWindow::MakeContentPanel()
{
	MenuTabSwitcher = SNew(SWidgetSwitcher)
		+ SWidgetSwitcher::Slot()
		[
			MakeDetailsTabContent()
		]

		+ SWidgetSwitcher::Slot()
		[
			MakeMetadataTabContent()
		]

		+ SWidgetSwitcher::Slot()
		[
			Sequencer->GetSequencerWidget()
		]

		+ SWidgetSwitcher::Slot()
		[
			MakeHierarchyTabContent()
		];

	return SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			MenuTabSwitcher.ToSharedRef()
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
				.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.PanelBackground"))
				.Padding(16.0f)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
						.FillWidth(0.8)
						[
							SNullWidget::NullWidget
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 8.0f, 0.0f)
						[
							SNew(STextBlock).Text(LOCTEXT("SchemaNameField", "Schema Name"))
						]

					+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.FillWidth(0.2)
						[
							SNew(SEditableTextBox)
								.Text_Lambda([this]() -> FText 
									{ 
										return SchemaToConfigure ? FText::FromString(SchemaToConfigure->SchemaName) : FText::GetEmpty(); 
									})
								.OnVerifyTextChanged(this, &SCineAssemblySchemaWindow::ValidateSchemaName)
								.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type InCommitType)
									{
										if (SchemaToConfigure)
										{
											SchemaToConfigure->RenameAsset(InText.ToString());
										}
									})
						]
				]
		];
}

TSharedRef<SWidget> SCineAssemblySchemaWindow::MakeButtonsPanel()
{
	if (Mode == ESchemaConfigMode::CreateNew)
	{
		return SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(16.0f)
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, 8.0f, 0.0f)
					[
						SNew(SButton)
							.Text(LOCTEXT("CreateAssetButton", "Create Schema"))
							.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
							.HAlign(HAlign_Center)
							.OnClicked(this, &SCineAssemblySchemaWindow::OnCreateAssetClicked)
							.IsEnabled_Lambda([this]() -> bool
								{
									FText OutErrorMessage;
									const FString SchemaPackageName = CreateAssetPath / SchemaToConfigure->SchemaName;
									return AssetViewUtils::IsValidPackageForCooking(SchemaPackageName, OutErrorMessage);
								})
							.ToolTipText_Lambda([this]() -> FText
								{
									FText OutErrorMessage;
									const FString SchemaPackageName = CreateAssetPath / SchemaToConfigure->SchemaName;
									return !AssetViewUtils::IsValidPackageForCooking(SchemaPackageName, OutErrorMessage) ? OutErrorMessage : FText();
								})
					]

				+ SHorizontalBox::Slot()
					.MinWidth(118.0f)
					.MaxWidth(118.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
							.Text(LOCTEXT("CancelButton", "Cancel"))
							.HAlign(HAlign_Center)
							.OnClicked(this, &SCineAssemblySchemaWindow::OnCancelClicked)
					]
			];
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SCineAssemblySchemaWindow::MakeDetailsWidget(bool bShowMetadata)
{
	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bHideSelectionTip = true;

	TSharedRef<IDetailsView> DetailsView = PropertyEditor.CreateDetailView(DetailsViewArgs);

	DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SCineAssemblySchemaWindow::IsPropertyVisible, bShowMetadata));

	DetailsView->SetObject(SchemaToConfigure, true);
	DetailsView->OnFinishedChangingProperties().AddSP(this, &SCineAssemblySchemaWindow::OnSchemaPropertiesChanged);

	return DetailsView;
}

bool SCineAssemblySchemaWindow::IsPropertyVisible(const FPropertyAndParent& PropertyAndParent, bool bShowMetadata)
{
	if (PropertyAndParent.Property.GetFName() == GET_MEMBER_NAME_CHECKED(UCineAssemblySchema, DefaultLevel))
	{
		return bShowMetadata;
	}
	if (PropertyAndParent.Property.GetFName() == GET_MEMBER_NAME_CHECKED(UCineAssemblySchema, AssemblyMetadata))
	{
		return bShowMetadata;
	}
	else if ((PropertyAndParent.ParentProperties.Num() > 0) && (PropertyAndParent.ParentProperties[0]->GetFName() == GET_MEMBER_NAME_CHECKED(UCineAssemblySchema, AssemblyMetadata)))
	{
		return bShowMetadata;
	}
	return !bShowMetadata;
}

TSharedRef<SWidget> SCineAssemblySchemaWindow::MakeDetailsTabContent()
{
	return SNew(SBorder)
		.Padding(16.0f)
		.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.PanelBackground"))
		[
			SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 0.0f, 0.0f, 8.0f)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("DetailsTitle", "Schema Details"))
						.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.HeadingFont"))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 0.0f, 0.0f, 24.0f)
				[
					SNew(STextBlock)
						.AutoWrapText(true)
						.Text(LOCTEXT("SchemaDetailsInstruction", "Configure the properties which will be inherited by every Cine Assembly asset created from this schema."))
				]

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					MakeDetailsWidget(false)
				]
		];
}

TSharedRef<SWidget> SCineAssemblySchemaWindow::MakeMetadataTabContent()
{
	return SNew(SBorder)
		.Padding(16.0f)
		.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.PanelBackground"))
		[
			SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 0.0f, 0.0f, 8.0f)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("MetadataTitle", "Schema Metadata"))
						.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.HeadingFont"))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 0.0f, 0.0f, 24.0f)
				[
					SNew(STextBlock)
						.AutoWrapText(true)
						.Text(LOCTEXT("SchemaMetadataInstruction", "Configure the metadata that should be associated with Cine Assemblies created from this schema. "
							"For each metadata field, choose the value type, metadata key, and optionally a default value."))
				]

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					MakeDetailsWidget(true)
				]
		];
}

TSharedRef<SWidget> SCineAssemblySchemaWindow::MakeHierarchyTabContent()
{
	TreeView = SNew(SCineAssemblyAssetTreeView, SchemaToConfigure->TemplateSequence)
		.SelectionMode(ESelectionMode::Single)
		.ShouldEvaluateTokens(false)
		.OnItemRemoved_Lambda([this]() { Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemRemoved); });

	TSharedRef<SWidget> ContentHierarchyWidget = SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Left)
		.Padding(0.0f, 0.0f, 0.0f, 8.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SPositiveActionButton)
						.Text(LOCTEXT("AddAssetButton", "Add Asset"))
						.OnGetMenuContent(this, &SCineAssemblySchemaWindow::OnGetAddAssetMenuContent)
				]

				+ SHorizontalBox::Slot()
				[
					SNew(SButton)
						.ContentPadding(FMargin(2.0f))
						.OnClicked(this, &SCineAssemblySchemaWindow::OnAddFolderClicked)
						[
							SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(0.0f, 0.0f, 4.0f, 0.0f)
								[
									SNew(SImage)
										.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
										.ColorAndOpacity(FStyleColors::AccentGreen)
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(STextBlock).Text(LOCTEXT("AddFolderButton", "Add Folder"))
								]
						]
				]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SBorder)
				.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.RecessedBackground"))
				[
					TreeView.ToSharedRef()
				]
		];

	return SNew(SBorder)
		.Padding(16.0f)
		.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.PanelBackground"))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 0.0f, 0.0f, 8.0f)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("ContentHierarchyTitle", "Content Hierarchy"))
						.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.HeadingFont"))
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 0.0f, 0.0f, 24.0f)
				[
					SNew(STextBlock)
						.AutoWrapText(true)
						.Text(LOCTEXT("AssetListInstructions", "The assets and folders displayed in this content tree will be created each time a new Cine Assembly is created using this Schema."))
				]

			+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					ContentHierarchyWidget
				]
		];
}

TSharedRef<SWidget> SCineAssemblySchemaWindow::OnGetAddAssetMenuContent()
{
	constexpr bool bCloseAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterMenuSelection, nullptr);

	// When adding a new Cine Assembly, add a new SubAssemblyTrack and Section to the TemplateSequence
	AddAssetMenuRow(MenuBuilder, UCineAssembly::StaticClass(), FExecuteAction::CreateLambda([this]()
		{
			UCineAssemblyEditorFunctionLibrary::AddSubAssemblyTemplate(SchemaToConfigure, ESubAssemblyTrackType::SubsequenceTrack, TEXT("NewCineAssembly"));
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
			TreeView->Reinitialize();
		})
	);

	MenuBuilder.AddSubMenu(MakeAssetMenuRowWidget(UWorld::StaticClass()), FNewMenuDelegate::CreateSP(this, &SCineAssemblySchemaWindow::PopulateAddLevelSubMenu));

	return MenuBuilder.MakeWidget();
}

void SCineAssemblySchemaWindow::PopulateAddLevelSubMenu(FMenuBuilder& MenuBuilder)
{
	// Quick-pick entries point at the engine's stock template assets at known fixed paths.
	// The existence check guards against an unlikely future engine relocation; we don't try to honor
	// project-configured template overrides here (use the asset picker below for any other template).
	auto AddQuickPickEntry = [this, &MenuBuilder](const TCHAR* PackagePath, const TCHAR* AssetName, const FText& EntryLabel)
	{
		if (!FPackageName::DoesPackageExist(PackagePath))
		{
			return;
		}

		const FSoftObjectPath SoftPath(FString::Printf(TEXT("%s.%s"), PackagePath, AssetName));
		const TSoftObjectPtr<UObject> TemplatePath(SoftPath);

		MenuBuilder.AddMenuEntry(
			EntryLabel,
			FText::Format(LOCTEXT("AddQuickPickTemplateTooltip", "Add a new Level by duplicating the {0} template"), EntryLabel),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this, TemplatePath]()
			{
				AddAssociatedAsset(UWorld::StaticClass(), TemplatePath);
			})),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	};

	MenuBuilder.BeginSection(TEXT("AddLevelSection"), LOCTEXT("AddLevelSectionHeader", "Add New Level"));
	{
		AddQuickPickEntry(TEXT("/Engine/Maps/Templates/Template_Default"), TEXT("Template_Default"), LOCTEXT("NewBasicLevel", "New Basic Level"));
		AddQuickPickEntry(TEXT("/Engine/Maps/Templates/OpenWorld"), TEXT("OpenWorld"), LOCTEXT("NewOpenWorldLevel", "New Open World Level"));
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("SelectFromTemplateSection"), LOCTEXT("SelectFromTemplateSectionHeader", "Select From Template"));
	{
		MenuBuilder.AddWidget(BuildLevelTemplateAssetPicker(), FText::GetEmpty());
	}
	MenuBuilder.EndSection();
}

TSharedRef<SWidget> SCineAssemblySchemaWindow::BuildLevelTemplateAssetPicker()
{
	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bAddFilterUI = true;
	AssetPickerConfig.bShowTypeInColumnView = false;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.Filter.ClassPaths = { UWorld::StaticClass()->GetClassPathName() };
	AssetPickerConfig.SaveSettingsName = TEXT("CineAssemblySchemaLevelTemplatePicker");

	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([this](const FAssetData& AssetData)
	{
		AddAssociatedAsset(UWorld::StaticClass(), AssetData.GetAsset());
		FSlateApplication::Get().DismissAllMenus();
	});

	AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateLambda([this](const TArray<FAssetData>& AssetDataArray)
	{
		if (!AssetDataArray.IsEmpty())
		{
			AddAssociatedAsset(UWorld::StaticClass(), AssetDataArray[0].GetAsset());
		}
		FSlateApplication::Get().DismissAllMenus();
	});

	constexpr float WidthOverride = 500.0f;
	constexpr float HeightOverride = 400.0f;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	return SNew(SBox)
		.WidthOverride(WidthOverride)
		.HeightOverride(HeightOverride)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];
}

void SCineAssemblySchemaWindow::AddAssetMenuRow(FMenuBuilder& MenuBuilder, const UClass* AssetClass, FExecuteAction OnClickAction)
{
	if (!OnClickAction.IsBound())
	{
		OnClickAction = FExecuteAction::CreateLambda([this, AssetClass]() { AddAssociatedAsset(AssetClass); });
	}


	const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(AssetClass);
	const FText DisplayName = AssetDefinition ? AssetDefinition->GetAssetDisplayName() : AssetClass->GetDisplayNameText();
	const FText Tooltip = FText::Format(LOCTEXT("AddAssetToSchemaTooltip", "Add a new {0} to the schema"), DisplayName);

	MenuBuilder.AddMenuEntry(FUIAction(OnClickAction), MakeAssetMenuRowWidget(AssetClass), NAME_None, Tooltip);
}

TSharedRef<SWidget> SCineAssemblySchemaWindow::MakeAssetMenuRowWidget(const UClass* AssetClass)
{
	const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(AssetClass);
	const FText DisplayName = AssetDefinition ? AssetDefinition->GetAssetDisplayName() : AssetClass->GetDisplayNameText();

	return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 8.0f, 0.0f)
			[
				SNew(SAssetMenuIcon, AssetClass)
					.IconContainerSize(FVector2D(24.0f, 24.0f))
					.IconSize(FVector2D(20.0f, 20.0f))
			]

		+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(DisplayName)
			];
}

void SCineAssemblySchemaWindow::AddAssociatedAsset(const UClass* AssetClass, TSoftObjectPtr<UObject> InTemplateAsset)
{
	if (!SchemaToConfigure->TemplateSequence)
	{
		return;
	}

	FAssemblyAssociatedAssetDesc NewAssetDesc;
	NewAssetDesc.AssetClass = AssetClass;
	NewAssetDesc.AssetID = FGuid::NewGuid();
	NewAssetDesc.TemplateAsset = InTemplateAsset;
	NewAssetDesc.AssetName.Template = UCineAssemblyEditorFunctionLibrary::GetDefaultAssetNameForClass(AssetClass);
	NewAssetDesc.Label = UCineAssemblyEditorFunctionLibrary::MakeDefaultAssociatedAssetLabel(SchemaToConfigure, AssetClass);

	SchemaToConfigure->TemplateSequence->Modify();
	SchemaToConfigure->TemplateSequence->AssociatedAssets.Add(NewAssetDesc);
	TreeView->Reinitialize();
}

FReply SCineAssemblySchemaWindow::OnAddFolderClicked()
{
	TreeView->AddFolder();
	return FReply::Handled();
}

void SCineAssemblySchemaWindow::OnSchemaPropertiesChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!SchemaToConfigure)
	{
		return;
	}

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FAssemblyMetadataDesc, Key))
	{
		UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();
		UCineAssemblyNamingTokens* CineAssemblyNamingTokens = Cast<UCineAssemblyNamingTokens>(NamingTokensSubsystem->GetNamingTokens(UCineAssemblyNamingTokens::TokenNamespace));

		for (const FAssemblyMetadataDesc& MetadataDesc : SchemaToConfigure->AssemblyMetadata)
		{
			CineAssemblyNamingTokens->AddMetadataToken(MetadataDesc.Key);
		}
	}
	else if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UCineAssemblySchema, AssemblyMetadata) && PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayRemove)
	{
		// Validate the current metadata links in case the metadata field that was removed had an active link
		if (SchemaToConfigure->TemplateSequence)
		{
			SchemaToConfigure->TemplateSequence->ValidateMetadataLinks();
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UCineAssemblySchema, DefaultLevel))
	{
		// When bOverrideDefaultLevel is unchecked, remove any existing Level link
		if (!SchemaToConfigure->bOverrideDefaultLevel && SchemaToConfigure->TemplateSequence)
		{
			const FString& LevelKey = UCineAssemblySchema::DefaultLevelMetadataKey;
			if (SchemaToConfigure->TemplateSequence->MetadataLinks.Contains(LevelKey))
			{
				SchemaToConfigure->TemplateSequence->Modify();
				SchemaToConfigure->TemplateSequence->MetadataLinks.Remove(LevelKey);
			}
		}
	}
}

FReply SCineAssemblySchemaWindow::OnCreateAssetClicked()
{
	UCineAssemblySchemaFactory::CreateConfiguredSchema(SchemaToConfigure, CreateAssetPath);

	TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SCineAssemblySchemaWindow::OnCancelClicked()
{
	TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
	}

	return FReply::Handled();
}

void SCineAssemblySchemaWindow::OnMovieSceneDataChanged(EMovieSceneDataChangeType InChangeType)
{
	MuteTracks();

	// Validate the current metadata links in case a SubAssembly track/section was recently modified/removed
	if (SchemaToConfigure->TemplateSequence)
	{
		SchemaToConfigure->TemplateSequence->ValidateMetadataLinks();
		ValidateSubAssemblyLinks();
	}

	if (TreeView)
	{
		TreeView->Reinitialize();
	}

	// Workaround for some outliner rows not appearing on freshly-added tracks/sections (like the channel groups on a new Transform track).
	// MarkAsChanged() will trigger a layout refresh for each track/section to update the outliner rows.
	if (InChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemAdded && SchemaToConfigure->TemplateSequence)
	{
		if (const UMovieScene* MovieScene = SchemaToConfigure->TemplateSequence->GetMovieScene())
		{
			for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
			{
				for (UMovieSceneTrack* Track : Binding.GetTracks())
				{
					Track->MarkAsChanged();
				}
			}

			for (UMovieSceneTrack* Track : MovieScene->GetTracks())
			{
				Track->MarkAsChanged();
			}
		}
	}
}

void SCineAssemblySchemaWindow::ValidateSubAssemblyLinks()
{
	UCineAssembly* TemplateSequence = SchemaToConfigure->TemplateSequence;
	if (!TemplateSequence || TemplateSequence->MetadataLinks.IsEmpty())
	{
		return;
	}

	UMovieScene* MovieScene = TemplateSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	for (auto It = TemplateSequence->MetadataLinks.CreateIterator(); It; ++It)
	{
		// Only check CineAssembly-type metadata fields with a schema filter
		const FAssemblyMetadataDesc* MetadataDesc = Algo::FindBy(SchemaToConfigure->AssemblyMetadata, It->Key, &FAssemblyMetadataDesc::Key);
		if (!MetadataDesc || MetadataDesc->Type != ECineAssemblyMetadataType::CineAssembly || MetadataDesc->SchemaType.IsNull())
		{
			continue;
		}

		// Find the linked SubAssembly section and check if its schema still matches
		for (UMovieSceneSection* Section : MovieScene->GetAllSections())
		{
			if (UMovieSceneSubAssemblySection* SubAssemblySection = Cast<UMovieSceneSubAssemblySection>(Section))
			{
				if (SubAssemblySection->GetSectionID() == It->Value)
				{
					const UCineAssemblySchema* TemplateSchema = SubAssemblySection->GetTemplateSchema();
					if (!TemplateSchema || MetadataDesc->SchemaType != FSoftObjectPath(TemplateSchema))
					{
						TemplateSequence->Modify();
						It.RemoveCurrent();
					}
					break;
				}
			}
		}
	}
}

void SCineAssemblySchemaWindow::OnActorAddedToSequencer(AActor* InActor, const FGuid InBinding)
{
	// Possessables in our template sequence may not actually bind to any actors in the world
	if (InActor && InBinding.IsValid())
	{
		if (ULevelSequenceEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<ULevelSequenceEditorSubsystem>())
		{
			Subsystem->AddDefaultTracksForActor(Sequencer.ToSharedRef(), *InActor, InBinding);
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
		}

		// There are likely other delegates that are bound to OnActorAddedToSequencer, but we cannot control the order in which they execute.
		// It might be dangerous to remove the actor from the newly created binding too early, before other delegates have a chance to do something with that binding.
		// Therefore, we delay our action until the next tick to ensure that this do execute AFTER any other bound delegates. 
		GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateLambda([this, InBinding]()
			{
				if (SchemaToConfigure && SchemaToConfigure->TemplateSequence)
				{
					SchemaToConfigure->TemplateSequence->UnbindPossessableObjects(InBinding);
				}
			}));
	}
}

void SCineAssemblySchemaWindow::ValidateTemplateSequence()
{
	UMovieScene* MovieScene = SchemaToConfigure->TemplateSequence ? SchemaToConfigure->TemplateSequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}

	MuteTracks();

	// Any bindings (other than spawnables) should be broken so that the template sequence cannot reference actors in the world
	if (const FMovieSceneBindingReferences* BindingRefs = SchemaToConfigure->TemplateSequence->GetBindingReferences())
	{
		for (int32 PossessableIndex = 0; PossessableIndex < MovieScene->GetPossessableCount(); ++PossessableIndex)
		{
			const FGuid& BindingID = MovieScene->GetPossessable(PossessableIndex).GetGuid();
			if (Algo::AnyOf(BindingRefs->GetReferences(BindingID), [](const FMovieSceneBindingReference& Ref) { return Ref.CustomBinding && Ref.CustomBinding->IsA<UMovieSceneSpawnableBindingBase>(); }))
			{
				continue;
			}

			SchemaToConfigure->TemplateSequence->UnbindPossessableObjects(BindingID);
		}
	}

	// Gather all folders in the template sequence MovieScene so we can find tracks in them
	TArray<UMovieSceneFolder*> AllFolders;
	GetMovieSceneFoldersRecursive(MovieScene->GetRootFolders(), AllFolders);

	// Associate each track found in a folder for quick lookup later if we need to remove or replace them
	TMap<UMovieSceneTrack*, UMovieSceneFolder*> TracksInFolders;
	for (UMovieSceneFolder* Folder : AllFolders)
	{
		for (UMovieSceneTrack* Track : Folder->GetChildTracks())
		{
			if (Track)
			{
				TracksInFolders.Add(Track, Folder);
			}
		}
	}

	// Find unsupported tracks
	// SubTracks and ShotTracks will get replaced with SubAssemblyTracks
	TArray<UMovieSceneEventTrack*> TracksToRemove;
	TArray<UMovieSceneSubTrack*> SubTracks;
	for (UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		if (UMovieSceneEventTrack* EventTrack = Cast<UMovieSceneEventTrack>(Track))
		{
			TracksToRemove.Add(EventTrack);
		}

		// This will catch SubTracks and ShotTracks, but not SubAssemblyTracks
		if (UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
		{
			if (!Track->IsA<UMovieSceneSubAssemblyTrack>())
			{
				SubTracks.Add(SubTrack);
			}
		}
	}

	// Remove unsupported tracks, either from the root or from their owning folder
	for (UMovieSceneEventTrack* Track : TracksToRemove)
	{
		if (UMovieSceneFolder* OwningFolder = TracksInFolders.FindRef(Track))
		{
			OwningFolder->RemoveChildTrack(Track);
		}
		MovieScene->RemoveTrack(*Track);
	}

	// Convert all SubTracks and ShotTracks to SubAssemblyTracks
	for (UMovieSceneSubTrack* SubTrack : SubTracks)
	{
		UMovieSceneSubAssemblyTrack* NewTrack = MovieScene->AddTrack<UMovieSceneSubAssemblyTrack>();

		if (SubTrack->IsA<UMovieSceneCinematicShotTrack>())
		{
			NewTrack->TrackType = ESubAssemblyTrackType::CinematicShotTrack;
		}
		else
		{
			NewTrack->TrackType = ESubAssemblyTrackType::SubsequenceTrack;
		}

		for (UMovieSceneSection* Section : SubTrack->GetAllSections())
		{
			UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
			if (!SubSection)
			{
				continue;
			}

			UMovieSceneSubAssemblySection* NewSection = CastChecked<UMovieSceneSubAssemblySection>(NewTrack->CreateNewSection());
			NewSection->SectionType = ESubAssemblySectionType::Reference;
			NewSection->SetRange(SubSection->GetRange());
			NewSection->SetSequence(SubSection->GetSequence());
			NewSection->Parameters = SubSection->Parameters;
			NewTrack->AddSection(*NewSection);
		}

		// If the original SubTrack belonged to a folder, remove it and add the new SubAssemblyTrack to that folder instead. 
		if (UMovieSceneFolder* OwningFolder = TracksInFolders.FindRef(SubTrack))
		{
			OwningFolder->RemoveChildTrack(SubTrack);
			OwningFolder->AddChildTrack(NewTrack);
		}

		MovieScene->RemoveTrack(*SubTrack);
	}
}

void SCineAssemblySchemaWindow::ExtendSequencerToolbarBaseCommands(FToolBarBuilder& Builder)
{
	Builder.BeginStyleOverride("SequencerToolbar");

	Builder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateLambda([this]() 
			{
				FCineAssemblySequencerUtilities::CreateCamera(Sequencer.ToSharedRef());
			})),
		"CreateCamera",
		FText::GetEmpty(),
		LOCTEXT("CreateCameraTooltipText", "Create a new camera and set it as the current camera cut"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.CreateCamera"),
		EUserInterfaceActionType::Button
	);

	Builder.AddSeparator();

	Builder.EndStyleOverride();
}

void SCineAssemblySchemaWindow::AddSequencerToolbarExtension(FToolBarBuilder& Builder)
{
	constexpr bool bShouldHaveSeparator = false;
	Builder.BeginSection("CineAssemblySchemaExtensions", bShouldHaveSeparator);
	Builder.BeginStyleOverride("SequencerToolbar");

	Builder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SCineAssemblySchemaWindow::OnImportSequenceClicked)),
		"ImportSequence",
		FText::GetEmpty(),
		LOCTEXT("ImportSequenceTooltipText", "Import an existing Level Sequence or Cine Assembly to use as a template"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Import")
	);
	Builder.EndStyleOverride();

	TSharedPtr<SWidget> DisplayRateWarningWidget = SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ToolTipText(LOCTEXT("DisplayRateWarning", "New Cine Assemblies will be initialized with the Active Production's display rate, not the display rate of this template."))
		.Visibility(this, &SCineAssemblySchemaWindow::GetDisplayRateWarningVisibility)
		[
			SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
				.ColorAndOpacity(FStyleColors::Warning)
		];

	Builder.AddWidget(DisplayRateWarningWidget.ToSharedRef());
	Builder.EndSection();
}

EVisibility SCineAssemblySchemaWindow::GetDisplayRateWarningVisibility() const
{
	if (const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>())
	{
		TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
		if (ActiveProduction.IsSet() && SchemaToConfigure && SchemaToConfigure->TemplateSequence && SchemaToConfigure->TemplateSequence->MovieScene)
		{
			const FFrameRate TemplateDisplayRate = SchemaToConfigure->TemplateSequence->MovieScene->GetDisplayRate();
			const FFrameRate ActiveProductionDisplayRate = ActiveProduction.GetValue().DefaultDisplayRate;

			if (TemplateDisplayRate != ActiveProductionDisplayRate)
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

void SCineAssemblySchemaWindow::MuteTracks()
{
	UMovieScene* MovieScene = SchemaToConfigure->TemplateSequence ? SchemaToConfigure->TemplateSequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}

	for (UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		// Mute relevant track types to prevent their evaluation from affecting the world
		if (Track->IsA<UMovieSceneCVarTrack>() || Track->IsA<UMovieSceneSubAssemblyTrack>())
		{
			Track->SetLocalEvalDisabled(true);
		}
	}
}

void SCineAssemblySchemaWindow::OnImportSequenceClicked()
{
	FOpenAssetDialogConfig OpenAssetConfig;
	OpenAssetConfig.DialogTitleOverride = LOCTEXT("ImportSequenceDialogTitle", "Import Sequence");
	OpenAssetConfig.bAllowMultipleSelection = false;
	OpenAssetConfig.DefaultPath = TEXT("/Game");
	OpenAssetConfig.AssetClassNames.Add(ULevelSequence::StaticClass()->GetClassPathName());
	OpenAssetConfig.AssetClassNames.Add(UCineAssembly::StaticClass()->GetClassPathName());

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	TArray<FAssetData> SelectedAssets = ContentBrowserModule.Get().CreateModalOpenAssetDialog(OpenAssetConfig);

	if (SelectedAssets.IsEmpty())
	{
		return;
	}

	ULevelSequence* SourceSequence = Cast<ULevelSequence>(SelectedAssets[0].GetAsset());
	if (!SourceSequence || !SourceSequence->GetMovieScene())
	{
		return;
	}

	const FText ConfirmMessage = LOCTEXT("ImportSequenceConfirmMessage", 
		"Importing will overwrite the current template with the contents of the selected sequence's MovieScene. This action cannot be undone.\n\nDo you want to continue?");

	if (FMessageDialog::Open(EAppMsgType::YesNo, ConfirmMessage) != EAppReturnType::Yes)
	{
		return;
	}

	// The import process will trigger a refresh of the Sequencer widget, but the Import button is part of the Sequencer Toolbar.
	// Defer the import and the Sequencer refresh to the next tick so that Slate can properly release its widget path references.
	GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SCineAssemblySchemaWindow::ImportSequence, SourceSequence));
}

void SCineAssemblySchemaWindow::ImportSequence(ULevelSequence* SourceSequence)
{
	if (!SourceSequence || !SourceSequence->GetMovieScene() || !SchemaToConfigure)
	{
		return;
	}

	SchemaToConfigure->TemplateSequence->Modify();
	SchemaToConfigure->TemplateSequence->InitializeFromTemplate(SourceSequence);
	ValidateTemplateSequence();

	// The MovieScene pointer of the focused sequence in the Sequencer widget changed as a result of the import.
	// Close the current Sequencer, then recreate it so that the UI refreshes to show the freshly imported MovieScene contents. 
	const int32 ActiveWidgetIndex = MenuTabSwitcher->GetActiveWidgetIndex();

	MenuTabSwitcher->RemoveSlot(Sequencer->GetSequencerWidget());
	Sequencer->Close();
	Sequencer.Reset();

	InitializeSequencer();

	MenuTabSwitcher->AddSlot(ActiveWidgetIndex)
	[
		Sequencer->GetSequencerWidget()
	];

	MenuTabSwitcher->SetActiveWidgetIndex(ActiveWidgetIndex);
}

#undef LOCTEXT_NAMESPACE
