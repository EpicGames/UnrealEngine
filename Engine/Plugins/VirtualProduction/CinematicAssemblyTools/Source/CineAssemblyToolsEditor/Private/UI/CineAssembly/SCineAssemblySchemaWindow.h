// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CineAssemblySchema.h"
#include "ISequencer.h"
#include "SCineAssemblyAssetTreeView.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

struct FPropertyAndParent;

/** Mode that configures the UI based on the intended user interactions with the Cine Assembly Schema asset */
enum class ESchemaConfigMode : uint8
{
	CreateNew,
	Edit
};

/**
 * A window for configuring the properties of a UCineAssemblySchema
 */
class SCineAssemblySchemaWindow : public SCompoundWidget
{
public:
	SCineAssemblySchemaWindow() = default;
	~SCineAssemblySchemaWindow();

	SLATE_BEGIN_ARGS(SCineAssemblySchemaWindow) {}
	SLATE_END_ARGS()

	/** Widget construction, initialized with the path where a new schema asset will be created */
	void Construct(const FArguments& InArgs, const FString& InCreateAssetPath);

	/** Widget construction, initialized with the schema asset being edited */
	void Construct(const FArguments& InArgs, UCineAssemblySchema* InSchema);

	/**
	 * Widget construction, initialized with the GUID of the schema to be edited
	 * The widget will search the asset registry to find the schema asset with the matching GUID,
	 * and then update the widget contents accordingly.
	 */
	void Construct(const FArguments& InArgs, FGuid InSchemaGuid);

	/** Returns the name of the schema asset being edited */
	FString GetSchemaName();

	/** Searches the asset registry for a Cine Assembly Schema matching the input ID and updates the UI */
	void FindSchema(FGuid SchemaID);

	/** Callback to cleanup this widget when the containing window is closed */
	void OnWindowClosed(const TSharedRef<SWindow>& InWindow);

private:
	/** Constructs the main UI for the widget */
	TSharedRef<SWidget> BuildUI();

	/** Creates the panel that displays the tab menu */
	TSharedRef<SWidget> MakeMenuPanel();

	/** Creates the panel that displays the content for each tab */
	TSharedRef<SWidget> MakeContentPanel();

	/** Creates the buttons on the bottom of the window */
	TSharedRef<SWidget> MakeButtonsPanel();

	/** Creates the content for the Details tab */
	TSharedRef<SWidget> MakeDetailsTabContent();

	/** Creates the content for the Metadata tab */
	TSharedRef<SWidget> MakeMetadataTabContent();

	/** Creates the content for the Hierarchy tab */
	TSharedRef<SWidget> MakeHierarchyTabContent();

	/** Creates the content for the Details and Data tabs (metadata properties are only shown in the Data tab) */
	TSharedRef<SWidget> MakeDetailsWidget(bool bShowMetadata);

	/** Creates the Sequencer for editing the TemplateSequence */
	void InitializeSequencer();

	/** Returns true if a schema already exists with the input name */
	bool DoesSchemaExistWithName(const FString& SchemaName) const;

	/** Validates the user input text for the schema name */
	bool ValidateSchemaName(const FText& InText, FText& OutErrorMessage) const;

	/** Filter used by the Details Views to determine which schema properties to display */
	bool IsPropertyVisible(const FPropertyAndParent& PropertyAndParent, bool bShowMetadata);

	/** Closes the window and indicates that a new asset should be created by the asset factory */
	FReply OnCreateAssetClicked();

	/** Closes the window and indicates that no assets should be created by the asset factory */
	FReply OnCancelClicked();

	/** Callback used to tear-down the Sequencer and other object references in this window before the Schema asset it is editing is deleted */
	void OnAssetsPreDelete(const TArray<UObject*>& AssetsToDelete);

	/** Returns the menu content for the Add Asset button */
	TSharedRef<SWidget> OnGetAddAssetMenuContent();

	/** Adds a menu entry to the Add Asset menu for the input asset class */
	void AddAssetMenuRow(FMenuBuilder& MenuBuilder, const UClass* AssetClass, FExecuteAction OnClickAction = FExecuteAction());

	/** Builds the icon-and-label row widget used by Add Asset entries (and shared with the Level sub-menu's top-level entry) */
	TSharedRef<SWidget> MakeAssetMenuRowWidget(const UClass* AssetClass);

	/** Populates the Add Asset > Level sub-menu: quick-pick template entries (Basic, Open World) above a UWorld asset picker */
	void PopulateAddLevelSubMenu(FMenuBuilder& MenuBuilder);

	/** Builds the embedded UWorld asset picker shown inside the Add Asset > Level sub-menu */
	TSharedRef<SWidget> BuildLevelTemplateAssetPicker();

	/** Adds a new associated asset definition to the template sequence, with an optional template asset */
	void AddAssociatedAsset(const UClass* AssetClass, TSoftObjectPtr<UObject> InTemplateAsset = nullptr);

	/** Adds a new folder item to the content tree view */
	FReply OnAddFolderClicked();

	/** Callback when one of the properties of the schema being configured changes */
	void OnSchemaPropertiesChanged(const FPropertyChangedEvent& PropertyChangedEvent);

	/** Callback when an actor is added to the template sequence, used to remove the actor(s) from the binding */
	void OnActorAddedToSequencer(AActor* InActor, const FGuid InBinding);

	/** Callback whenever MovieScene data in the template sequence is changed */
	void OnMovieSceneDataChanged(EMovieSceneDataChangeType InChangeType);

	/** Validates all CineAssembly metadata links */
	void ValidateSubAssemblyLinks();

	/** Extends the sequencer widget's toolbar */
	void AddSequencerToolbarExtension(FToolBarBuilder& Builder);

	/** Opens an asset picker to import a LevelSequence or CineAssembly to initialize the TemplateSequence */
	void OnImportSequenceClicked();

	/** Initializes the TemplateSequence with the input Sequence */
	void ImportSequence(ULevelSequence* SourceSequence);

	/** Validates the schema's TemplateSequence and fixes / removes incorrect bindings and tracks */
	void ValidateTemplateSequence();

	/** Extends the Sequencer Toolbar with custom entries for this specific Sequencer */
	void ExtendSequencerToolbarBaseCommands(FToolBarBuilder& Builder);

	/** Returns the visibility of the display rate warning toolbar widget */
	EVisibility GetDisplayRateWarningVisibility() const;

	/** Mutes all tracks in the template sequence that we do not want to evaluate locally in this Sequencer widget */
	void MuteTracks();

	/** Cached content browser settings, used to restore defaults when closing the window */
	bool bShowEngineContentCached = false;
	bool bShowPluginContentCached = false;

private:
	/** Switcher that controls which menu tab is currently visible */
	TSharedPtr<SWidgetSwitcher> MenuTabSwitcher;

	/** Strong reference to a new, transient Schema object, held so that it cannot be GC'd while the user is still configuring it. Not used when editing existing schemas. */
	TStrongObjectPtr<UCineAssemblySchema> NewSchemaObject;

	/** Schema object being configured by this window. */
	UCineAssemblySchema* SchemaToConfigure = nullptr;

	/** Mode that configures the UI based on the intended user interactions with the Cine Assembly schema asset */
	ESchemaConfigMode Mode = ESchemaConfigMode::CreateNew;

	/** The root path where the configured schema will be created */
	FString CreateAssetPath;

	/** The sequencer used to display / edit the template sequence */
	TSharedPtr<ISequencer> Sequencer;

	/** A TreeView to display the list of other assets and folders that Assembly's made from this Schema will create */
	TSharedPtr<SCineAssemblyAssetTreeView> TreeView;

	/** Handle for OnAssetsPreDelete */
	FDelegateHandle OnAssetsPreDeleteHandle;
};
