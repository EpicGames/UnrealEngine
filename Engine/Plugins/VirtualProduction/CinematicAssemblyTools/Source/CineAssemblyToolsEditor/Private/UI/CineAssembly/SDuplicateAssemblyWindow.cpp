// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDuplicateAssemblyWindow.h"

#include "Algo/Find.h"
#include "AssetToolsModule.h"
#include "CineAssemblyEditorSubsystem.h"
#include "CineAssemblyFactory.h"
#include "CineAssemblyNamingTokens.h"
#include "CineAssemblyToolsStyle.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "Misc/ConfigCacheIni.h"
#include "MovieScene.h"
#include "MovieSceneFolder.h"
#include "ObjectTools.h"
#include "SCineAssemblyConfigPanel.h"
#include "Sections/MovieSceneSubSection.h"
#include "Sequencer/CineAssemblySequencerUtilities.h"
#include "SNamingTokensEditableTextBox.h"
#include "Styling/SlateIconFinder.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDuplicateAssemblyWindow"

const FName SDuplicateAssemblyWindow::OriginalNameColumn = "OriginalName";
const FName SDuplicateAssemblyWindow::DuplicationModeColumn = "DuplicationMode";
const FName SDuplicateAssemblyWindow::DuplicateNameColumn = "DuplicateName";

const FString SDuplicateAssemblyWindow::DuplicationPreferenceSection = TEXT("CinematicAssemblyDuplication_");

UCineAssembly* FSequenceTreeItem::GetDuplicateSubAssembly() const
{
	if (bIsManagedSubAssembly)
	{
		return Cast<UCineAssembly>(SubSection->GetSequence());
	}
	return nullptr;
}

void SDuplicateAssemblyWindow::Construct(const FArguments& InArgs, UCineAssembly* InAssembly)
{
	OriginalAssembly = InAssembly;

	// Duplicate the original assembly into a new transient assembly that can be configured in the UI
	// Note: all of the subsequence tracks/sections will be duplicated, but will reference the sequences in the original assembly
	// So if the user chooses MaintainReference as the duplication mode for any subsequence, no further work is needed for that subsequence
	DuplicateAssembly.Reset(Cast<UCineAssembly>(StaticDuplicateObject(OriginalAssembly, GetTransientPackage(), NAME_None, RF_Transient)));

	// Record the source as persistent lineage metadata on the duplicate
	if (DuplicateAssembly.IsValid())
	{
		DuplicateAssembly->SourceAssembly = OriginalAssembly;
	}

	// Get the path and root folder of the original assembly and initialize the duplication path to be the same root folder as the original
	OriginalAssembly->GetAssetPathAndRootFolder(OriginalAssemblyPath, OriginalAssemblyRootFolder);
	DuplicationPath = OriginalAssemblyRootFolder;

	const FVector2D DefaultWindowSize = FVector2D(1200, 750);

	SWindow::Construct(SWindow::FArguments()
		.Title(LOCTEXT("WindowTitle", "Duplicate Assembly"))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(DefaultWindowSize)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			SNew(SBorder)
				.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.PanelBackground"))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
						[
							SNew(SSplitter)
								.Orientation(Orient_Horizontal)
								.PhysicalSplitterHandleSize(2.0f)

							+ SSplitter::Slot()
								.Value(0.65f)
								[
									MakeSubsequencesPanel()
								]

							+ SSplitter::Slot()
								.Value(0.35f)
								[
									SNew(SCineAssemblyConfigPanel, DuplicateAssembly.Get())
										.HideSubAssemblies(true)
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SSeparator)
								.Orientation(Orient_Horizontal)
								.Thickness(2.0f)
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							MakeButtonsPanel()
						]
				]
		]);
}

TSharedRef<SWidget> SDuplicateAssemblyWindow::MakeSubsequencesPanel()
{
	UMovieScene* DuplicateAssemblyMovieScene = DuplicateAssembly->GetMovieScene();
	if (DuplicateAssemblyMovieScene)
	{
		// In order to duplicate subsequences, the MovieScene needs to be mutable. The original read-only state will be restored when we are done.
		const bool bRootIsReadOnly = DuplicateAssemblyMovieScene->IsReadOnly();
		DuplicateAssemblyMovieScene->SetReadOnly(false);

		// Walk the duplicate assembly to recursively find every subsequence and build up the subsequence tree
		PopulateSubsequenceTreeRecursive(DuplicateAssembly.Get(), nullptr, OriginalAssemblyPath);

		DuplicateAssemblyMovieScene->SetReadOnly(bRootIsReadOnly);
	}

	TreeView = SNew(STreeView<TSharedPtr<FSequenceTreeItem>>)
		.TreeItemsSource(&TreeItems)
		.SelectionMode(ESelectionMode::None)
		.OnGenerateRow(this, &SDuplicateAssemblyWindow::OnGenerateTreeRow)
		.OnGetChildren(this, &SDuplicateAssemblyWindow::OnGetChildren)
		.HeaderRow
		(
			SNew(SHeaderRow)

			+ SHeaderRow::Column(OriginalNameColumn)
			.DefaultLabel(LOCTEXT("OriginalNameColumn", "Original Name"))
			.FillWidth(0.3f)
			.HeaderContentPadding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))

			+ SHeaderRow::Column(DuplicationModeColumn)
			.DefaultLabel(LOCTEXT("DuplicationModeColumn", "Duplication Mode"))
			.FillWidth(0.35f)
			.HeaderContentPadding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))

			+ SHeaderRow::Column(DuplicateNameColumn)
			.DefaultLabel(LOCTEXT("DuplicateNameColumn", "Duplicate Name"))
			.FillWidth(0.35f)
			.HeaderContentPadding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
		);

	// Start with the entire subsequence tree expanded
	for (TSharedPtr<FSequenceTreeItem>& Item : TreeItems)
	{
		ExpandTreeItem(Item);
	}

	// Create a path picker widget to let the user choose where the duplicate assembly should be created
	const FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>("ContentBrowser");

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.DefaultPath = OriginalAssemblyRootFolder;
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateLambda([this](const FString& NewPath)
		{
			DuplicationPath = NewPath;
		});

	TSharedRef<SWidget> PathPicker = ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig);

	return SNew(SBorder)
		.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.PanelBackground"))
		.Padding(16.0f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 0.0f, 0.0f, 8.0f)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("DuplicateAssemblyTitle", "Subassemblies and Associated Assets"))
						.Font(FCoreStyle::GetDefaultFontStyle("Normal", 14))
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 0.0f, 0.0f, 16.0f)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("DuplicateAssemblyInstructions", "Choose how each Subassembly and Associated Asset will be duplicated. The path for the top-level duplicate assembly can be set below, and the properties and metadata of the duplicate assembly can be set in the panel on the right."))
						.Font(FCoreStyle::GetDefaultFontStyle("Normal", 10))
						.AutoWrapText(true)
				]

			+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.Padding(0.0f, 0.0f, 0.0f, 16.0f)
				[
					TreeView.ToSharedRef()
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 16.0f, 0.0f)
						[
							SNew(STextBlock).Text(LOCTEXT("DuplicationPath", "Duplication Path"))
						]

					+ SHorizontalBox::Slot()
						.FillContentWidth(1.0f)
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 8.0f, 0.0f)
						[
							SNew(SEditableTextBox)
								.ToolTipText(LOCTEXT("DuplicationPathTooltip", "Destination folder for the top-level duplicate assembly. Duplicated subassemblies and associated assets are placed relative to this folder using their existing relative paths."))
								.Text_Lambda([this]() { return FText::FromString(DuplicationPath); })
								.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type InCommitType) { DuplicationPath = InText.ToString(); })
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SComboButton)
								.MenuContent()
								[
									SNew(SVerticalBox)

									+ SVerticalBox::Slot()
										.MaxHeight(300.0f)
										[
											PathPicker
										]
								]
								.ButtonContent()
								[
									SNew(SImage).Image(FCineAssemblyToolsStyle::Get().GetBrush("Icons.Folder"))
								]
						]
				]
		];
}

TSharedRef<SWidget> SDuplicateAssemblyWindow::MakeButtonsPanel()
{
	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(16.0f)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.MinWidth(118.0f)
				.MaxWidth(118.0f)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SButton)
						.Text(LOCTEXT("DuplicateButton", "Duplicate"))
						.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
						.HAlign(HAlign_Center)
						.OnClicked(this, &SDuplicateAssemblyWindow::OnDuplicateClicked)
				]

			+ SHorizontalBox::Slot()
				.MinWidth(118.0f)
				.MaxWidth(118.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
						.Text(LOCTEXT("CancelButton", "Cancel"))
						.HAlign(HAlign_Center)
						.OnClicked(this, &SDuplicateAssemblyWindow::OnCancelClicked)
				]
		];
}

FReply SDuplicateAssemblyWindow::OnDuplicateClicked()
{
	SaveDuplicationPreferences();

	// Managed SubAssemblies were pre-duplicated during tree construction. Restore the original sequence in any SubSections that are not set to DuplicateOriginal
	for (TSharedPtr<FSequenceTreeItem>& Item : TreeItems)
	{
		RestoreOriginalSubAssembliesRecursive(Item);
	}

	// Flatten the duplication data from the subsequence tree into a map
	for (TSharedPtr<FSequenceTreeItem>& Item : TreeItems)
	{
		BuildDuplicationMapRecursive(Item);
	}

	// Create valid packages for the Duplicate Assembly and all of its duplicated SubAssemblies
	UCineAssemblyFactory::CreateConfiguredAssembly(DuplicateAssembly.Get(), DuplicationPath);

	// Duplicate the remaining subsequences (those that are not Assemblies) that are set to DuplicateOriginal
	DuplicateSubsequencesRecursive(DuplicateAssembly.Get());

	// Duplicate, maintain, or clear the associated assets
	for (TSharedPtr<FSequenceTreeItem>& Item : TreeItems)
	{
		ProcessAssociatedAssets(Item);
	}

	// AssociatedAsset CreatedAsset references just changed, so re-resolve MetadataLinks to push the new
	// references into linked properties (e.g. Level) and metadata fields on the duplicate and its sub-assemblies.
	UCineAssemblyFactory::PopulateLinkedMetadataRecursive(DuplicateAssembly.Get());

	// Remove any subsequences that are set to Remove
	RemoveSubsequencesRecursive(DuplicateAssembly.Get());

	// Broadcast the duplication event with the source assembly
	if (UCineAssemblyEditorSubsystem* EditorSubsystem = GEditor ? GEditor->GetEditorSubsystem<UCineAssemblyEditorSubsystem>() : nullptr)
	{
		EditorSubsystem->OnAssemblyDuplicated.Broadcast(DuplicateAssembly.Get(), OriginalAssembly);
	}

	RequestDestroyWindow();
	return FReply::Handled();
}

FReply SDuplicateAssemblyWindow::OnCancelClicked()
{
	RequestDestroyWindow();
	return FReply::Handled();
}

TSharedRef<ITableRow> SDuplicateAssemblyWindow::OnGenerateTreeRow(TSharedPtr<FSequenceTreeItem> TreeItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SSubsequenceDuplicationRow, OwnerTable, TreeItem, DuplicateAssembly.Get());
}

void SDuplicateAssemblyWindow::OnGetChildren(TSharedPtr<FSequenceTreeItem> TreeItem, TArray<TSharedPtr<FSequenceTreeItem>>& OutNodes)
{
	OutNodes.Append(TreeItem->Children);
}

void SDuplicateAssemblyWindow::PopulateSubsequenceTreeRecursive(UMovieSceneSequence* InSequence, TSharedPtr<FSequenceTreeItem> InItem, const FString& SourceBasePath)
{
	UMovieScene* MovieScene = InSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	UCineAssembly* OwningAssembly = Cast<UCineAssembly>(InSequence);

	// Loop through all of the sections to find every subsequence in the input sequence
	for (UMovieSceneSection* Section : MovieScene->GetAllSections())
	{
		if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
		{
			if (UMovieSceneSequence* Subsequence = SubSection->GetSequence())
			{
				TSharedPtr<FSequenceTreeItem> NewItem = MakeShared<FSequenceTreeItem>();
				NewItem->SubSection = SubSection;
				NewItem->OriginalSequence = Subsequence;
				NewItem->Parent = InItem;

				NewItem->DuplicationData.DuplicateName = Subsequence->IsA<UCineAssembly>() ? Cast<UCineAssembly>(Subsequence)->AssemblyName.Template : Subsequence->GetDisplayName().ToString();

				InItem ? InItem->Children.Add(NewItem) : TreeItems.Add(NewItem);

				// We handle managed SubAssemblies differently than other Subsequences found in the input Sequence's MovieScene
				if (OwningAssembly && OwningAssembly->SubAssemblies.Contains(SubSection))
				{
					// Pre-duplicate this SubAssembly such that it can be configured by the user before assets are created
					// If the user chooses to duplicate this, a valid asset will be created by the asset Factory
					// Otherwise, the original SubSequence will be restored and this duplicate will be thrown out
					UCineAssembly* OriginalSubAssembly = Cast<UCineAssembly>(Subsequence);
					UCineAssembly* DuplicateSubAssembly = Cast<UCineAssembly>(StaticDuplicateObject(Subsequence, GetTransientPackage(), NAME_None, RF_Transient));

					DuplicateSubAssembly->ParentAssembly = OwningAssembly;

					// Remap MetadataLinks entries from the original SubAssembly's GUID to the duplicate's new GUID
					if (OriginalSubAssembly)
					{
						const FGuid OriginalGuid = OriginalSubAssembly->GetAssemblyGuid();
						const FGuid DuplicateGuid = DuplicateSubAssembly->GetAssemblyGuid();
						for (TPair<FString, FGuid>& MetadataLink : OwningAssembly->MetadataLinks)
						{
							if (MetadataLink.Value == OriginalGuid)
							{
								MetadataLink.Value = DuplicateGuid;
							}
						}
					}

					FCineAssemblySequencerUtilities::ReplaceSubSequence(SubSection, DuplicateSubAssembly);

					NewItem->OwningAssembly = OwningAssembly;
					NewItem->bIsManagedSubAssembly = true;
					NewItem->DuplicationData.DuplicationMode = GetDuplicationPreference(DuplicateSubAssembly->AssemblyName.Template);
				}
				else
				{
					// Compute the path relative to the parent Assembly so that a duplicate SubSequence can be created in the correct relative folder structure
					FString SubsequencePath = FPaths::GetPath(Subsequence->GetPathName());
					const FString RelativeParentPath = SourceBasePath + TEXT('/');
					FPaths::MakePathRelativeTo(SubsequencePath, *RelativeParentPath);
					NewItem->DuplicationData.RelativePath = SubsequencePath;
				}

				// Query for the current SubSequence again (which could be the new duplicate SubAssembly) to ensure we recurse into the correct sequence
				UMovieSceneSequence* SequenceToRecurse = SubSection->GetSequence();
				if (UMovieScene* SubSequenceMovieScene = SequenceToRecurse->GetMovieScene())
				{
					// In order to duplicate subsequences, the MovieScene needs to be mutable. The original read-only state will be restored when we are done.
					const bool bMovieSceneIsReadOnly = SubSequenceMovieScene->IsReadOnly();
					SubSequenceMovieScene->SetReadOnly(false);

					// Use the path of the original Subsequence (even if it was duplicated already) because the duplicate does not yet have a valid path
					const FString ChildSourceBasePath = FPaths::GetPath(Subsequence->GetPathName());

					PopulateSubsequenceTreeRecursive(SequenceToRecurse, NewItem, ChildSourceBasePath);

					SubSequenceMovieScene->SetReadOnly(bMovieSceneIsReadOnly);
				}
			}
		}
	}

	// Add associated asset items for any assembly that has created assets
	if (OwningAssembly)
	{
		for (const FAssemblyAssociatedAssetDesc& AssetDesc : OwningAssembly->AssociatedAssets)
		{
			if (AssetDesc.CreatedAsset.IsNull())
			{
				continue;
			}

			TSharedPtr<FSequenceTreeItem> NewItem = MakeShared<FSequenceTreeItem>();
			NewItem->AssociatedAssetID = AssetDesc.AssetID;
			NewItem->OwningAssembly = OwningAssembly;
			NewItem->DuplicationData.DuplicateName = AssetDesc.AssetName.Template;
			NewItem->Parent = InItem;

			InItem ? InItem->Children.Add(NewItem) : TreeItems.Add(NewItem);
		}
	}
}

void SDuplicateAssemblyWindow::DuplicateSubsequencesRecursive(UMovieSceneSequence* InSequence)
{
	UMovieScene* MovieScene = InSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	// In order to duplicate subsequences, the MovieScene needs to be mutable.
	// The original read-only state will be restored when we are done.
	const bool bIsReadOnly = MovieScene->IsReadOnly();
	MovieScene->SetReadOnly(false);

	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();

	// Loop through all of the sections to find every subsequence in the input sequence
	for (UMovieSceneSection* Section : MovieScene->GetAllSections())
	{
		if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
		{
			if (UMovieSceneSequence* Subsequence = SubSection->GetSequence())
			{
				// Only duplicate subsequences that are in the map and set to DuplicateOriginal.
				// Note: Managed SubAssemblies that were already duplicated were not added to the map to avoid double duplication
				if (const FSubsequenceDuplicationData* DuplicationData = SubsequenceDuplicationData.Find(Subsequence))
				{
					if (DuplicationData->DuplicationMode == ECineAssemblyDuplicationMode::DuplicateOriginal)
					{
						const FString ResolvedName = UCineAssemblyNamingTokens::GetResolvedText(DuplicationData->DuplicateName, DuplicateAssembly.Get()).ToString();

						// Use the current parent sequence's path as the base, so that subsequences inside a renamed SubAssembly are placed relative to the SubAssembly's new folder
						const FString ParentAssetPath = FPaths::GetPath(InSequence->GetPathName());
						FString DesiredAssetName = ParentAssetPath / DuplicationData->RelativePath / ResolvedName;
						FPaths::CollapseRelativeDirectories(DesiredAssetName);

						FString UniquePackageName;
						FString UniqueAssetName;
						AssetTools.CreateUniqueAssetName(DesiredAssetName, TEXT(""), UniquePackageName, UniqueAssetName);

						const FString DestinationFolder = FPaths::GetPath(UniquePackageName);
						UMovieSceneSequence* DuplicateSubsequence = Cast<UMovieSceneSequence>(AssetTools.DuplicateAsset(UniqueAssetName, DestinationFolder, Subsequence));

						if (DuplicateSubsequence)
						{
							FCineAssemblySequencerUtilities::ReplaceSubSequence(SubSection, DuplicateSubsequence);

							SubsequenceDuplicationData.Add(DuplicateSubsequence, *DuplicationData);
							SubsequenceDuplicationData.Remove(Subsequence);
						}
					}
				}

				DuplicateSubsequencesRecursive(SubSection->GetSequence());
			}
		}
	}

	// Restore the original read-only state
	MovieScene->SetReadOnly(bIsReadOnly);
}

void SDuplicateAssemblyWindow::RemoveSubsequencesRecursive(UMovieSceneSequence* Sequence)
{
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	TArray<UMovieSceneTrack*> TracksToRemove;

	// Loop through all of the tracks and all of the sections to find every subsequence in the input sequence
	for (UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		if (UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
		{
			TArray<UMovieSceneSection*> SectionsToRemove;

			for (UMovieSceneSection* Section : SubTrack->GetAllSections())
			{
				if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
				{
					if (UMovieSceneSequence* Subsequence = SubSection->GetSequence())
					{
						const FSubsequenceDuplicationData* DuplicationData = SubsequenceDuplicationData.Find(Subsequence);
						if (!DuplicationData)
						{
							continue;
						}
						if (DuplicationData->DuplicationMode == ECineAssemblyDuplicationMode::Remove)
						{
							SectionsToRemove.Add(SubSection);
						}
						else
						{
							RemoveSubsequencesRecursive(Subsequence);
						}
					}
				}
			}

			for (UMovieSceneSection* Section : SectionsToRemove)
			{
				SubTrack->Modify();
				SubTrack->RemoveSection(*Section);
			}

			// If we have now removed every section from the subsequence track, we can remove the entire track as well
			if (SubTrack->IsEmpty())
			{
				TracksToRemove.Add(SubTrack);
			}
		}
	}

	for (UMovieSceneTrack* Track : TracksToRemove)
	{
		MovieScene->RemoveTrack(*Track);
	}

	TArray<UMovieSceneFolder*> RootFolders;
	MovieScene->GetRootFolders(RootFolders);

	RemoveTracksFromFoldersRecursive(RootFolders, TracksToRemove);
}

void SDuplicateAssemblyWindow::RemoveTracksFromFoldersRecursive(TArrayView<UMovieSceneFolder* const> Folders, const TArray<UMovieSceneTrack*>& TracksToRemove)
{
	for (UMovieSceneFolder* Folder : Folders)
	{
		for (UMovieSceneTrack* Track : TracksToRemove)
		{
			Folder->RemoveChildTrack(Track);
		}

		RemoveTracksFromFoldersRecursive(Folder->GetChildFolders(), TracksToRemove);
	}
}

void SDuplicateAssemblyWindow::RestoreOriginalSubAssembliesRecursive(const TSharedPtr<FSequenceTreeItem>& InItem)
{
	if (InItem->bIsManagedSubAssembly && InItem->DuplicationData.DuplicationMode != ECineAssemblyDuplicationMode::DuplicateOriginal && InItem->OriginalSequence.IsValid())
	{
		FCineAssemblySequencerUtilities::ReplaceSubSequence(InItem->SubSection, InItem->OriginalSequence.Get());
	}

	for (const TSharedPtr<FSequenceTreeItem>& Child : InItem->Children)
	{
		RestoreOriginalSubAssembliesRecursive(Child);
	}
}

void SDuplicateAssemblyWindow::BuildDuplicationMapRecursive(const TSharedPtr<FSequenceTreeItem> InItem)
{
	// SubAssemblies have already been duplicated so they should not be added to the map to avoid being duplicated again
	// Associated assets are also handled separately and don't belong in the subsequence duplication map
	const bool bIsAlreadyDuplicated = InItem->bIsManagedSubAssembly && (InItem->DuplicationData.DuplicationMode == ECineAssemblyDuplicationMode::DuplicateOriginal);
	if (!bIsAlreadyDuplicated && !InItem->AssociatedAssetID.IsValid() && InItem->OriginalSequence.IsValid())
	{
		SubsequenceDuplicationData.Add(InItem->OriginalSequence.Get(), InItem->DuplicationData);
	}

	for (const TSharedPtr<FSequenceTreeItem>& Child : InItem->Children)
	{
		BuildDuplicationMapRecursive(Child);
	}
}

void SDuplicateAssemblyWindow::ExpandTreeItem(TSharedPtr<FSequenceTreeItem> InItem) const
{
	TreeView->SetItemExpansion(InItem, true);

	for (TSharedPtr<FSequenceTreeItem>& Child : InItem->Children)
	{
		ExpandTreeItem(Child);
	}
}

void SDuplicateAssemblyWindow::SaveDuplicationPreferences() const
{
	const UCineAssemblySchema* Schema = DuplicateAssembly->GetSchema();
	if (Schema && GConfig)
	{
		const FString SectionName = DuplicationPreferenceSection + Schema->SchemaName;

		for (const TSharedPtr<FSequenceTreeItem>& Item : TreeItems)
		{
			SaveDuplicationPreferencesRecursive(Item, SectionName);
		}
	}
}

void SDuplicateAssemblyWindow::SaveDuplicationPreferencesRecursive(const TSharedPtr<FSequenceTreeItem>& InItem, const FString& SectionName) const
{
	if (UCineAssembly* DuplicateSubAssembly = InItem->GetDuplicateSubAssembly())
	{
		// The config strips off these token characters (for some reason), so we save and restore the values without them
		FString TemplateName = DuplicateSubAssembly->AssemblyName.Template;
		TemplateName = TemplateName.Replace(TEXT("{"), TEXT(""));
		TemplateName = TemplateName.Replace(TEXT("}"), TEXT(""));

		const int32 Value = static_cast<int32>(InItem->DuplicationData.DuplicationMode);
		GConfig->SetInt(*SectionName, *TemplateName, Value, GEditorPerProjectIni);
	}

	for (const TSharedPtr<FSequenceTreeItem>& Child : InItem->Children)
	{
		SaveDuplicationPreferencesRecursive(Child, SectionName);
	}
}

ECineAssemblyDuplicationMode SDuplicateAssemblyWindow::GetDuplicationPreference(const FString& SubAssemblyName) const
{
	const UCineAssemblySchema* Schema = DuplicateAssembly->GetSchema();
	if (Schema && GConfig)
	{
		const FString SectionName = DuplicationPreferenceSection + Schema->SchemaName;

		// The config strips off these token characters (for some reason), so we save and restore the values without them
		FString TemplateName = SubAssemblyName;
		TemplateName = TemplateName.Replace(TEXT("{"), TEXT(""));
		TemplateName = TemplateName.Replace(TEXT("}"), TEXT(""));

		int32 Value = 0;
		GConfig->GetInt(*SectionName, *TemplateName, Value, GEditorPerProjectIni);

		return static_cast<ECineAssemblyDuplicationMode>(Value);
	}

	return ECineAssemblyDuplicationMode::DuplicateOriginal;
}

void SDuplicateAssemblyWindow::ProcessAssociatedAssets(const TSharedPtr<FSequenceTreeItem>& InItem)
{
	if (InItem->AssociatedAssetID.IsValid())
	{
		UCineAssembly* Assembly = InItem->OwningAssembly.Get();
		if (!Assembly)
		{
			return;
		}

		FAssemblyAssociatedAssetDesc* AssetDesc = Algo::FindBy(Assembly->AssociatedAssets, InItem->AssociatedAssetID, &FAssemblyAssociatedAssetDesc::AssetID);
		if (!AssetDesc)
		{
			return;
		}

		switch (InItem->DuplicationData.DuplicationMode)
		{
		case ECineAssemblyDuplicationMode::DuplicateOriginal:
			UCineAssemblyFactory::DuplicateAssociatedAsset(Assembly, *AssetDesc);
			break;

		case ECineAssemblyDuplicationMode::MaintainReference:
			// No-op: keep the existing CreatedAsset reference
			break;

		case ECineAssemblyDuplicationMode::Remove:
			AssetDesc->CreatedAsset = nullptr;
			break;

		default:
			checkNoEntry();
		}
	}

	for (const TSharedPtr<FSequenceTreeItem>& Child : InItem->Children)
	{
		ProcessAssociatedAssets(Child);
	}
}

void SSubsequenceDuplicationRow::Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedPtr<FSequenceTreeItem>& InTreeItem, UCineAssembly* DuplicateAssembly)
{
	TreeItem = InTreeItem;

	if (TreeItem->AssociatedAssetID.IsValid() && TreeItem->OwningAssembly.IsValid())
	{
		if (const FAssemblyAssociatedAssetDesc* AssetDesc = Algo::FindBy(TreeItem->OwningAssembly->AssociatedAssets, TreeItem->AssociatedAssetID, &FAssemblyAssociatedAssetDesc::AssetID))
		{
			ItemClass = AssetDesc->AssetClass.LoadSynchronous();
		}
	}

	NamingTokenContext = TStrongObjectPtr<UCineAssemblyNamingTokensContext>(NewObject<UCineAssemblyNamingTokensContext>());

	if (UCineAssembly* DuplicateSubAssembly = TreeItem->GetDuplicateSubAssembly())
	{
		NamingTokenContext->Assembly = DuplicateSubAssembly;
	}
	else if (TreeItem->AssociatedAssetID.IsValid())
	{
		NamingTokenContext->Assembly = TreeItem->OwningAssembly.Get();
	}
	else
	{
		NamingTokenContext->Assembly = DuplicateAssembly;
	}

	FilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

	FSuperRowType::FArguments StyleArguments = FSuperRowType::FArguments()
		.Padding(FMargin(4.0f, 8.0f))
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow"));

	SMultiColumnTableRow<TSharedPtr<FSequenceTreeItem>>::Construct(StyleArguments, OwnerTableView);
}

TSharedRef<SWidget> SSubsequenceDuplicationRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == SDuplicateAssemblyWindow::OriginalNameColumn)
	{
		return MakeOriginalNameWidget();
	}
	else if (ColumnName == SDuplicateAssemblyWindow::DuplicationModeColumn)
	{
		return MakeDuplicationModeWidget();
	}
	else if (ColumnName == SDuplicateAssemblyWindow::DuplicateNameColumn)
	{
		return MakeDuplicateNameWidget();
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SSubsequenceDuplicationRow::MakeOriginalNameWidget()
{
	return SNew(SHorizontalBox)

		// The first column gets the expander / indent for children items in the tree view
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SExpanderArrow, SharedThis(this)).IndentAmount(12)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(8.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SImage).Image(this, &SSubsequenceDuplicationRow::GetOriginalNameIcon)
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(8.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock).Text(this, &SSubsequenceDuplicationRow::GetOriginalDisplayName)
		];
}

TSharedRef<SWidget> SSubsequenceDuplicationRow::MakeDuplicationModeWidget()
{
	FMenuBuilder DuplicationModeMenu(true, nullptr);
	{
		auto AddEntryForDuplicationMode = [this, &DuplicationModeMenu](ECineAssemblyDuplicationMode DuplicationMode, const FText& Tooltip)
			{
				DuplicationModeMenu.AddMenuEntry(
					GetDuplicationModeDisplayName(DuplicationMode),
					Tooltip,
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SSubsequenceDuplicationRow::SetDuplicationModeRecursive, TreeItem, DuplicationMode)),
					NAME_None,
					EUserInterfaceActionType::Button
				);
			};

		AddEntryForDuplicationMode(ECineAssemblyDuplicationMode::DuplicateOriginal,
			LOCTEXT("DuplicateOriginalTooltip", "Create a new asset duplicated from the original."));
		AddEntryForDuplicationMode(ECineAssemblyDuplicationMode::MaintainReference,
			LOCTEXT("MaintainReferenceTooltip", "Reference the original asset (no new asset is created)."));
		AddEntryForDuplicationMode(ECineAssemblyDuplicationMode::Remove,
			LOCTEXT("RemoveTooltip", "Do not include this item in the duplicated assembly."));
	}

	return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(4.0f, 0.0f, 16.0f, 0.0f)
		[
			SNew(SComboButton)
				.MenuContent()
				[
					DuplicationModeMenu.MakeWidget()
				]
				.ButtonContent()
				[
					SNew(STextBlock).Text_Lambda([this]() { return GetDuplicationModeDisplayName(TreeItem->DuplicationData.DuplicationMode); })
				]
				.IsEnabled_Lambda([this]()
					{
						// Disable the duplication mode widget if the parent is MaintainReference or Remove (since the value should be locked to match the parent)
						if (TSharedPtr<FSequenceTreeItem> ParentPinned = TreeItem->Parent.Pin())
						{
							return ParentPinned->DuplicationData.DuplicationMode == ECineAssemblyDuplicationMode::DuplicateOriginal;
						}
						return true;
					})
				.ToolTipText_Lambda([this]()
					{
						if (TSharedPtr<FSequenceTreeItem> ParentPinned = TreeItem->Parent.Pin())
						{
							if (ParentPinned->DuplicationData.DuplicationMode != ECineAssemblyDuplicationMode::DuplicateOriginal)
							{
								return LOCTEXT("DuplicationModeLockedTooltip", "Locked to match parent's duplication mode.");
							}
						}
						return FText::GetEmpty();
					})
		];
}

TSharedRef<SWidget> SSubsequenceDuplicationRow::MakeDuplicateNameWidget()
{
	return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(10.0f, 0.0f, 16.0f, 0.0f)
		[
			SNew(SNamingTokensEditableTextBox)
				.Contexts({ NamingTokenContext.Get() })
				.FilterArgs(FilterArgs)
				.EvaluationFrequency(1.0f)
				.ShowUnsetTokenWarning(true)
				.IsReadOnly_Lambda([this]()
					{
						return TreeItem->DuplicationData.DuplicationMode != ECineAssemblyDuplicationMode::DuplicateOriginal;
					})
				.Visibility_Lambda([this]()
					{
						return (TreeItem->DuplicationData.DuplicationMode != ECineAssemblyDuplicationMode::Remove) ? EVisibility::Visible : EVisibility::Hidden;
					})
				.Text(this, &SSubsequenceDuplicationRow::GetDuplicateDisplayName)
				.OnTextCommitted(this, &SSubsequenceDuplicationRow::OnSubsequenceNameCommitted)
				.OnTokenizedTextEvaluated(this, &SSubsequenceDuplicationRow::OnSubsequenceTokenTextEvaluated)
				.OnValidateTokenizedText(this, &SSubsequenceDuplicationRow::ValidateSubsequenceName)
				.OnValidateResolvedText(this, &SSubsequenceDuplicationRow::ValidateSubsequenceName)
		];
}

FText SSubsequenceDuplicationRow::GetDuplicationModeDisplayName(ECineAssemblyDuplicationMode DuplicationMode) const
{
	switch (DuplicationMode)
	{
	case ECineAssemblyDuplicationMode::DuplicateOriginal:
		return LOCTEXT("DuplicateOriginalDisplayName", "Duplicate Original");
	case ECineAssemblyDuplicationMode::MaintainReference:
		return LOCTEXT("MaintainReferenceDisplayName", "Maintain Reference");
	case ECineAssemblyDuplicationMode::Remove:
		return LOCTEXT("RemoveDisplayName", "Remove");
	}

	return FText::GetEmpty();
}

void SSubsequenceDuplicationRow::SetDuplicationModeRecursive(TSharedPtr<FSequenceTreeItem> InItem, ECineAssemblyDuplicationMode Mode)
{
	InItem->DuplicationData.DuplicationMode = Mode;

	if (Mode != ECineAssemblyDuplicationMode::DuplicateOriginal)
	{
		for (TSharedPtr<FSequenceTreeItem>& Child : InItem->Children)
		{
			SetDuplicationModeRecursive(Child, Mode);
		}
	}
}

const FSlateBrush* SSubsequenceDuplicationRow::GetOriginalNameIcon() const
{
	if (ItemClass)
	{
		return FSlateIconFinder::FindIconBrushForClass(ItemClass);
	}
	return FCineAssemblyToolsStyle::Get().GetBrush("Icons.Sequencer");
}

FText SSubsequenceDuplicationRow::GetOriginalDisplayName() const
{
	if (TreeItem->AssociatedAssetID.IsValid() && TreeItem->OwningAssembly.IsValid())
	{
		if (const FAssemblyAssociatedAssetDesc* AssetDesc = Algo::FindBy(TreeItem->OwningAssembly->AssociatedAssets, TreeItem->AssociatedAssetID, &FAssemblyAssociatedAssetDesc::AssetID))
		{
			return FText::FromString(AssetDesc->CreatedAsset.GetAssetName());
		}
		return FText::GetEmpty();
	}

	if (TreeItem->OriginalSequence.IsValid())
	{
		return TreeItem->OriginalSequence->GetDisplayName();
	}

	return FText::GetEmpty();
}

FText SSubsequenceDuplicationRow::GetDuplicateDisplayName() const
{
	if (TreeItem->DuplicationData.DuplicationMode == ECineAssemblyDuplicationMode::DuplicateOriginal)
	{
		return FText::FromString(TreeItem->DuplicationData.DuplicateName);
	}

	if (TreeItem->AssociatedAssetID.IsValid() && TreeItem->OwningAssembly.IsValid())
	{
		if (const FAssemblyAssociatedAssetDesc* AssetDesc = Algo::FindBy(TreeItem->OwningAssembly->AssociatedAssets, TreeItem->AssociatedAssetID, &FAssemblyAssociatedAssetDesc::AssetID))
		{
			return FText::FromString(AssetDesc->AssetName.Template);
		}
		return FText::GetEmpty();
	}

	if (TreeItem->OriginalSequence.IsValid())
	{
		return TreeItem->OriginalSequence->GetDisplayName();
	}

	return FText::GetEmpty();
}

void SSubsequenceDuplicationRow::OnSubsequenceNameCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	TreeItem->DuplicationData.DuplicateName = InText.ToString();

	if (UCineAssembly* DuplicateSubAssembly = TreeItem->GetDuplicateSubAssembly())
	{
		DuplicateSubAssembly->AssemblyName.Template = InText.ToString();
	}
	else if (TreeItem->AssociatedAssetID.IsValid() && TreeItem->OwningAssembly.IsValid())
	{
		// Update the associated asset desc so any linked metadata widgets reflect the current duplicate name
		if (FAssemblyAssociatedAssetDesc* AssetDesc = Algo::FindBy(TreeItem->OwningAssembly->AssociatedAssets, TreeItem->AssociatedAssetID, &FAssemblyAssociatedAssetDesc::AssetID))
		{
			AssetDesc->AssetName.Template = InText.ToString();
		}
	}
}

void SSubsequenceDuplicationRow::OnSubsequenceTokenTextEvaluated(const FText& InText)
{
	if (UCineAssembly* DuplicateSubAssembly = TreeItem->GetDuplicateSubAssembly())
	{
		DuplicateSubAssembly->AssemblyName.Resolved = InText;
	}
}

bool SSubsequenceDuplicationRow::ValidateSubsequenceName(const FText& InText, FText& OutErrorMessage) const
{
	// Ensure that the name does not contain any characters that would be invalid for an asset name
	// This matches the validation that would happen if the user was renaming an asset in the content browser
	FString InvalidCharacters = INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS;

	// These characters are actually valid, because we want to support naming tokens
	InvalidCharacters = InvalidCharacters.Replace(TEXT("{}"), TEXT(""));
	InvalidCharacters = InvalidCharacters.Replace(TEXT(":"), TEXT(""));

	const FString PotentialName = InText.ToString();
	if (!FName::IsValidXName(PotentialName, InvalidCharacters, &OutErrorMessage))
	{
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
