// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"

#include "CineAssembly.h"
#include "CineAssemblyNamingTokens.h"
#include "MovieSceneSequence.h"
#include "NamingTokensEngineSubsystem.h"
#include "Widgets/Views/STreeView.h"

class UMovieSceneFolder;
class UMovieSceneSubSection;

/** List of available duplication modes for a subsequence */
enum class ECineAssemblyDuplicationMode : uint8
{
	DuplicateOriginal, // Duplicates the original assembly's subsequence track/section AND duplicates the original subsequence, updating the reference
	MaintainReference, // Duplicates the original assembly's subsequence track/section BUT maintains the reference to the original subsequence
	Remove // Does not duplicate the original assembly's subsequence track/section
};

/** The data required to duplicate a specific subsequence */
struct FSubsequenceDuplicationData
{
	/** The duplication mode to use */
	ECineAssemblyDuplicationMode DuplicationMode = ECineAssemblyDuplicationMode::DuplicateOriginal;

	/** The folder path relative to the parent sequence */
	FString RelativePath;

	/** The name to use for the duplicated subsequence (if the mode is DuplicateOriginal) */
	FString DuplicateName;
};

/** An item in the subsequence tree view, associating a subsequence with its duplication data and child subsequences */
struct FSequenceTreeItem
{
	/**
	 * The assembly that owns this tree item. This could be null if this item is not managed by a CineAssembly
	 * For SubAssembly items, this is the assembly that manages this item in its SubAssemblies list.
	 * For Associated Asset items, this is the assembly that manages this item in its AssociatedAssets list.
	 */
	TWeakObjectPtr<UCineAssembly> OwningAssembly;

	/** The SubSection in the MovieScene of the Duplicate Assembly (or one of its SubAssemblies) */
	UMovieSceneSubSection* SubSection = nullptr;

	/** The original sequence referenced by this SubSection before any duplication occured */
	TWeakObjectPtr<UMovieSceneSequence> OriginalSequence;

	/** The duplication data associated with this subsequence */
	FSubsequenceDuplicationData DuplicationData;

	/** The ID of the associated asset in the owning assembly's AssociatedAssets array. Invalid for SubSequence item types. */
	FGuid AssociatedAssetID;

	/** Child subsequence tree items */
	TArray<TSharedPtr<FSequenceTreeItem>> Children;

	/** Parent sequence tree item */
	TWeakPtr<FSequenceTreeItem> Parent;

	/** Indicates whether this tree item represents a SubAssembly in the managed SubAssemblies list of an Assembly being duplicated */
	bool bIsManagedSubAssembly = false;

	/** If this tree item is a managed SubAssembly, returns the duplicate SubAssembly in the SubSection. Otherwise, returns nullptr */
	UCineAssembly* GetDuplicateSubAssembly() const;
};

/** Widget row for an item in the subsequence tree view */
class SSubsequenceDuplicationRow : public SMultiColumnTableRow<TSharedPtr<FSequenceTreeItem>>
{
public:

	SLATE_BEGIN_ARGS(SSubsequenceDuplicationRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedPtr<FSequenceTreeItem>& InTreeItem, UCineAssembly* DuplicateAssembly);

	/** Creates the widget for this row for the specified column */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	/** Make the widget for each column */
	TSharedRef<SWidget> MakeOriginalNameWidget();
	TSharedRef<SWidget> MakeDuplicationModeWidget();
	TSharedRef<SWidget> MakeDuplicateNameWidget();

	/** Returns the icon brush for the original name column */
	const FSlateBrush* GetOriginalNameIcon() const;

	/** Returns the display name for the original name column */
	FText GetOriginalDisplayName() const;

	/** Returns the display name for the duplicate name column (varies by duplication mode) */
	FText GetDuplicateDisplayName() const;

	/** Updates the duplication mode for the input tree item, and potentially its children */
	void SetDuplicationModeRecursive(TSharedPtr<FSequenceTreeItem> InItem, ECineAssemblyDuplicationMode Mode);

	/** Updates the subsequence template name associated with this tree row */
	void OnSubsequenceNameCommitted(const FText& InText, ETextCommit::Type InCommitType);

	/** Updates the subsequence resolved name after the token text was evaluated */
	void OnSubsequenceTokenTextEvaluated(const FText& InText);

	/** Validates the user input text for the subsequence name */
	bool ValidateSubsequenceName(const FText& InText, FText& OutErrorMessage) const;

	/** Get the display name of the input duplication mode */
	FText GetDuplicationModeDisplayName(ECineAssemblyDuplicationMode DuplicationMode) const;

private:
	/** The tree view item displayed by this row widget */
	TSharedPtr<FSequenceTreeItem> TreeItem;

	/** Naming Token Context object, with a pointer to the duplicate assembly */
	TStrongObjectPtr<UCineAssemblyNamingTokensContext> NamingTokenContext;

	/** Naming Token filter properties */
	FNamingTokenFilterArgs FilterArgs;

	/** Cached item class */
	UClass* ItemClass = nullptr;
};

/** A window to configure how a Cine Assembly asset should be duplicated */
class SDuplicateAssemblyWindow : public SWindow
{
public:
	SDuplicateAssemblyWindow() = default;

	SLATE_BEGIN_ARGS(SDuplicateAssemblyWindow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UCineAssembly* InAssembly);

public:
	static const FName OriginalNameColumn;
	static const FName DuplicationModeColumn;
	static const FName DuplicateNameColumn;

private:
	/** Creates the subsequences panel */
	TSharedRef<SWidget> MakeSubsequencesPanel();

	/** Creates the buttons on the bottom of the window */
	TSharedRef<SWidget> MakeButtonsPanel();

	/** Called when the Duplicate button is pressed, which will execute the duplication steps as defined by the duplication data for each subsequence */
	FReply OnDuplicateClicked();

	/** Called when the Cancel button is pressed, which will close the window without duplicating anything */
	FReply OnCancelClicked();

	/** Generates the row widget in the subsequence tree view for the input tree item */
	TSharedRef<ITableRow> OnGenerateTreeRow(TSharedPtr<FSequenceTreeItem> InTreeItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Adds the children of the input tree item to the OutNodes array */
	void OnGetChildren(TSharedPtr<FSequenceTreeItem> InTreeItem, TArray<TSharedPtr<FSequenceTreeItem>>& OutNodes);

	/** 
	 * Builds the list of tree items in the subsequence tree by recursively walking the input sequence and finding all subsequences that may need to be duplicated.
	 * SourceBasePath is the asset folder used to compute non-SubAssembly relative paths. 
	 */
	void PopulateSubsequenceTreeRecursive(UMovieSceneSequence* InSequence, TSharedPtr<FSequenceTreeItem> InTreeItem, const FString& SourceBasePath);

	/** Recursively duplicates any of the subsequences of the input sequence that are set to the DuplicateOriginal duplication mode */
	void DuplicateSubsequencesRecursive(UMovieSceneSequence* InSequence);

	/** Recursively restores the original subsequences of the input item that are not set to DuplicateOriginal duplication mode */
	void RestoreOriginalSubAssembliesRecursive(const TSharedPtr<FSequenceTreeItem>& InItem);

	/** Recursively removes any of the subsequences of the input sequence that are set to the Remove duplication mode */
	void RemoveSubsequencesRecursive(UMovieSceneSequence* InSequence);

	/** Recursively removes the set of tracks from the input folders (or child folders) */
	void RemoveTracksFromFoldersRecursive(TArrayView<UMovieSceneFolder* const> Folders, const TArray<UMovieSceneTrack*>& TracksToRemove);

	/** Recursively handles duplication, maintaining, or clearing of the associated assets based on the duplication mode */
	void ProcessAssociatedAssets(const TSharedPtr<FSequenceTreeItem>& InItem);

	/** Converts the subsequence tree into a map of subsequences to their associated duplication data */
	void BuildDuplicationMapRecursive(const TSharedPtr<FSequenceTreeItem> InTreeItem);

	/** Expands the input item in the subsequence tree view */
	void ExpandTreeItem(TSharedPtr<FSequenceTreeItem> InTreeItem) const;

	/** Save the duplication mode settings for each subassembly to a config file to re-use the next time an assembly with the same schema is duplicated */
	void SaveDuplicationPreferences() const;

	/** Recursively saves the duplication mode preference for the input tree item and its children */
	void SaveDuplicationPreferencesRecursive(const TSharedPtr<FSequenceTreeItem>& InItem, const FString& SectionName) const;

	/** Reads the saved config and returns the most recent duplication mode for the input subassembly */
	ECineAssemblyDuplicationMode GetDuplicationPreference(const FString& SubAssemblyName) const;

private:
	/** The original assembly being duplicated */
	UCineAssembly* OriginalAssembly;

	/** The duplicated assembly that is being configured */
	TStrongObjectPtr<UCineAssembly> DuplicateAssembly;

	/** List of subsequence tree items for the tree view */
	TArray<TSharedPtr<FSequenceTreeItem>> TreeItems;

	/** Tree view of subsequences, allowing the user to configure the duplication data */
	TSharedPtr<STreeView<TSharedPtr<FSequenceTreeItem>>> TreeView;

	/** Subset of the duplication data derived from the tree view for non-managed subsequences */
	TMap<UMovieSceneSequence*, FSubsequenceDuplicationData> SubsequenceDuplicationData;

	/** The path where the duplicate assembly asset should be created */
	FString DuplicationPath;

	/** The path of the original assembly asset */
	FString OriginalAssemblyPath;

	/** The root folder of the original assembly (i.e. the asset path with the default assembly path from the schema removed) */
	FString OriginalAssemblyRootFolder;

	/** Config section name for duplication preferences */
	static const FString DuplicationPreferenceSection;
};
