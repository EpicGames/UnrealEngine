// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActiveProductionCombo.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ProductionSettings.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SActiveProductionCombo"

void SActiveProductionCombo::Construct(const FArguments& InArgs)
{
	static auto GetActiveProductionName = []() -> FText
		{
			const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();

			TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
			const FString ActiveProductionName = ActiveProduction.IsSet() ? ActiveProduction.GetValue().ProductionName : FString();

			if (ActiveProductionName.IsEmpty())
			{
				return LOCTEXT("NoneProductionName", "None");
			}
			return FText::FromString(ActiveProductionName);
		};

	static auto OnSetActiveProduction = [](FGuid ProductionID)
		{
			UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
			ProductionSettings->SetActiveProduction(ProductionID);
		};

	auto BuildMenu = []() -> TSharedRef<SWidget>
		{
			FMenuBuilder MenuBuilder(true, nullptr);

			// None option, allowing the user to set no active production
			MenuBuilder.AddMenuEntry(
				FText::FromName(NAME_None),
				FText::FromName(NAME_None),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda(OnSetActiveProduction, FGuid())),
				NAME_None,
				EUserInterfaceActionType::Button
			);

			const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
			const TArray<FCinematicProduction> Productions = ProductionSettings->GetProductions();
			for (const FCinematicProduction& Production : Productions)
			{
				MenuBuilder.AddMenuEntry(
					FText::FromString(Production.ProductionName),
					FText::FromString(Production.ProductionName),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda(OnSetActiveProduction, Production.ProductionID)),
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}

			return MenuBuilder.MakeWidget();
		};

	ChildSlot
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
				.Padding(16.0f, 8.0f)
				[
					SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 8.0f, 0.0f)
						[
							SNew(STextBlock).Text(LOCTEXT("ActiveProductionText", "Active Production"))
						]

						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.VAlign(VAlign_Center)
						[
							SNew(SComboButton)
								.OnGetMenuContent_Lambda(BuildMenu)
								.ButtonContent()
								[
									SNew(STextBlock).Text_Lambda(GetActiveProductionName)
								]
						]
				]
		];
}

#undef LOCTEXT_NAMESPACE
