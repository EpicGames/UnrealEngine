// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Widgets/DeclarativeSyntaxSupport.h"

/**
 * Combo button widget for choosing an available production to be the Active production
 */
class SActiveProductionCombo : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SActiveProductionCombo) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
};
