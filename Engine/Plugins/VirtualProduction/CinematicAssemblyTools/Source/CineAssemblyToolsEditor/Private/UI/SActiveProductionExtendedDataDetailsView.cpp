// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActiveProductionExtendedDataDetailsView.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailCustomization.h"
#include "InstancedStructDetails.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "ProductionSettings.h"
#include "UI/SActiveProductionCombo.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SActiveProductionExtendedDataDetailsView"

namespace UE::ActiveProductionData::Private
{

constexpr int32 TabIndex_NoProduction = 0;
constexpr int32 TabIndex_Details = 1;

/** 
 * Private details customization used specifically for the instance in the SActiveProductionExtendedDataDetailsView.
 * 
 * This customization takes the target struct type into account and shows only the Extended Data in the active production
 * that matches.
 */
class FProductionSettingsSingleExtendedDataDetails : public IDetailCustomization
{
public:
	/** Creates an instance of this details customization */
	static TSharedRef<FProductionSettingsSingleExtendedDataDetails> MakeInstance(const UScriptStruct* ExtendedDataScriptStruct)
	{
		TSharedRef<FProductionSettingsSingleExtendedDataDetails> Instance = MakeShared<FProductionSettingsSingleExtendedDataDetails>();
		Instance->ExtendedDataScriptStruct = ExtendedDataScriptStruct;
		return Instance;
	}

	/** Set the target Extended Data struct type. */
	void SetExtendedDataScriptStruct(const UScriptStruct* DataStruct)
	{
		ExtendedDataScriptStruct = DataStruct;

		if (WeakBuilder.IsValid())
		{
			WeakBuilder.Pin()->ForceRefreshDetails();
		}
	}

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		TArray<TWeakObjectPtr<UObject>> ObjectsBeingModified;
		DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingModified);

		// This should only ever be a single object that is the default UProductionSettings.
		check(ObjectsBeingModified.Num() == 1);
		check(ObjectsBeingModified[0] == GetDefault<UProductionSettings>());

		// hide all categories
		TArray<FName> CategoryNames;
		DetailBuilder.GetCategoryNames(CategoryNames);

		for (const FName& Category : CategoryNames)
		{
			DetailBuilder.HideCategory(Category);
		}

		if (ExtendedDataScriptStruct == nullptr)
		{
			return;
		}

		// Find the property handle of the specific data we want to display
		const UProductionSettings* ProductionSettings = Cast<UProductionSettings>(ObjectsBeingModified[0]);

		TSharedPtr<IPropertyHandleArray> ProductionsHandleArray = DetailBuilder.GetProperty(
			UProductionSettings::GetProductionsPropertyMemberName(), UProductionSettings::StaticClass())->AsArray();
		check(ProductionsHandleArray);

		const FGuid ActiveProductionId = ProductionSettings->GetActiveProductionID();

		if (TSharedPtr<IPropertyHandle> ActiveProductionHandle = GetProductionPropertyHandle(ActiveProductionId, ProductionsHandleArray))
		{
			if (TSharedPtr<IPropertyHandle> TargetDataHandle = GetTargetExtendedDataPropertyHandle(ActiveProductionHandle))
			{
				// Add our custom property
				IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(ExtendedDataScriptStruct->GetFName());
				TSharedRef<FInstancedStructDataDetails> DataDetails = MakeShared<FInstancedStructDataDetails>(TargetDataHandle);
				CategoryBuilder.AddCustomBuilder(DataDetails);
			}
		}
	}

	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override
	{
		WeakBuilder = DetailBuilder;
		IDetailCustomization::CustomizeDetails(DetailBuilder);
	}
	//~ End IDetailCustomization interface

private:
	/** Gets the property handle for the production with the given ID. */
	TSharedPtr<IPropertyHandle> GetProductionPropertyHandle(FGuid ProductionID, TSharedPtr<IPropertyHandleArray> ProductionsHandleArray)
	{
		if (!ProductionID.IsValid())
		{
			return nullptr;
		}

		uint32 NumProductions;
		ProductionsHandleArray->GetNumElements(NumProductions);

		for (uint32 ProductionIndex = 0; ProductionIndex < NumProductions; ++ProductionIndex)
		{
			TArray<void*> RawData;
			TSharedRef<IPropertyHandle> ProductionHandle = ProductionsHandleArray->GetElement(ProductionIndex);
			ProductionHandle->AccessRawData(RawData);

			if (!RawData.IsEmpty())
			{
				if (FCinematicProduction* ProductionPtr = reinterpret_cast<FCinematicProduction*>(RawData[0]))
				{
					if (ProductionPtr->ProductionID == ProductionID)
					{
						return ProductionHandle;
					}
				}
			}
		}

		return nullptr;
	}

	/** Gets the property handle for the Extended Data struct of the target struct type. */
	TSharedPtr<IPropertyHandle> GetTargetExtendedDataPropertyHandle(TSharedPtr<IPropertyHandle> ProductionPropertyHandle)
	{
		if (TSharedPtr<IPropertyHandle> ExtendedDataHandle = ProductionPropertyHandle->GetChildHandle(FCinematicProduction::GetExtendedDataMemberName()))
		{
			TSharedPtr<IPropertyHandleMap> ExtendedDataHandleMap = ExtendedDataHandle->AsMap();

			uint32 NumElements;
			ExtendedDataHandleMap->GetNumElements(NumElements);

			for (uint32 DataIndex = 0; DataIndex < NumElements; ++DataIndex)
			{
				TArray<void*> RawData;
				TSharedRef<IPropertyHandle> DataHandle = ExtendedDataHandleMap->GetElement(DataIndex);
				DataHandle->AccessRawData(RawData);

				if (!RawData.IsEmpty())
				{
					if (FInstancedStruct* StructPtr = reinterpret_cast<FInstancedStruct*>(RawData[0]))
					{
						if (StructPtr->GetScriptStruct() == ExtendedDataScriptStruct)
						{
							return DataHandle;
						}
					}
				}
			}
		}

		return nullptr;
	}

	/** Extended Data target struct type. */
	const UScriptStruct* ExtendedDataScriptStruct = nullptr;

	/** Weak pointer to the layout builder to allow us to call refresh when the target script struct changes. */
	TWeakPtr<IDetailLayoutBuilder> WeakBuilder;
};

} // namespace UE::ActiveProductionData::Private

SActiveProductionExtendedDataDetailsView::~SActiveProductionExtendedDataDetailsView()
{
	if (UObjectInitialized())
	{
		if (UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>())
		{
			ProductionSettings->OnProductionListChanged().Remove(ProductionListChangedHandle);
			ProductionSettings->OnActiveProductionChanged().Remove(ActiveProductionChangedHandle);
		}
	}
}

void SActiveProductionExtendedDataDetailsView::Construct(const FArguments& InArgs)
{
	TargetScriptStruct = InArgs._TargetScriptStruct;
	bShowProductionSelection = InArgs._bShowProductionSelection;

	TSharedPtr<SWidget> Contents = nullptr;
	if (TargetScriptStruct == nullptr)
	{
		// Display a simple text block to inform the user that no target script struct has been set.
		Contents = SNew(STextBlock)
			.Justification(ETextJustify::Center)
			.ToolTipText(LOCTEXT("ActiveProductionDataPanel_NoTargetStructTooltip", "No details can be shown as the target data structure is null."))
			.Text(LOCTEXT("ActiveProductionDataPanel_NoTargetStructText", "No target data structure set."));
	}
	else
	{
		// Subscribe to be notified when the Production Settings active productions has changed
		UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
		ActiveProductionChangedHandle = ProductionSettings->OnActiveProductionChanged().AddSP(this, &SActiveProductionExtendedDataDetailsView::HandleActiveProductionChanged);
		ProductionListChangedHandle = ProductionSettings->OnProductionListChanged().AddSP(this, &SActiveProductionExtendedDataDetailsView::HandleProductionListChanged);
		Contents = MakeContents(InArgs._InnerWidget);
	}

	

	ChildSlot
		[
			Contents.ToSharedRef()
		];
}

void SActiveProductionExtendedDataDetailsView::HandleActiveProductionChanged()
{
	UpdateActiveData();
}

void SActiveProductionExtendedDataDetailsView::HandleProductionListChanged()
{
	UpdateActiveData();
}

void SActiveProductionExtendedDataDetailsView::UpdateActiveData()
{
	if (DetailsView == nullptr || TargetScriptStruct == nullptr)
	{
		return;
	}

	if (WeakDetailsCustomization.IsValid())
	{
		WeakDetailsCustomization.Pin()->SetExtendedDataScriptStruct(TargetScriptStruct);
	}

	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	if (ProductionSettings->GetActiveProductionID().IsValid())
	{
		DetailsWidgetSwitcher->SetActiveWidgetIndex(UE::ActiveProductionData::Private::TabIndex_Details);
	}
	else
	{
		DetailsWidgetSwitcher->SetActiveWidgetIndex(UE::ActiveProductionData::Private::TabIndex_NoProduction);
	}
}

TSharedRef<SWidget> SActiveProductionExtendedDataDetailsView::MakeContents(TSharedPtr<SWidget> InnerWidget)
{
	TSharedRef<SWidget> ContentsWidget = InnerWidget != nullptr
		? InnerWidget.ToSharedRef()
		: MakeDetailsWidget();

	if (bShowProductionSelection)
	{
		return SNew(SVerticalBox)

		// Active Production Selector
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SActiveProductionCombo)
			]

		// Separator
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSeparator)
					.Orientation(Orient_Horizontal)
					.Thickness(2.0f)
			]

		// Contents widget
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				ContentsWidget
			];
	}
	
	return ContentsWidget;
}

TSharedRef<SWidget> SActiveProductionExtendedDataDetailsView::MakeDetailsWidget()
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;

	FStructureDetailsViewArgs StructureViewArgs;
	StructureViewArgs.bShowObjects = true;
	StructureViewArgs.bShowAssets = true;
	StructureViewArgs.bShowClasses = true;
	StructureViewArgs.bShowInterfaces = true;

	FPropertyEditorModule& PropModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	DetailsView = PropModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateLambda([this]()
		{
			// Customize this instance of the details view to only display the single Extended Data. Using a view on the object
			// allows all change events and undos to propagate back correctly to the object.
			TSharedRef<UE::ActiveProductionData::Private::FProductionSettingsSingleExtendedDataDetails> DetailsCustomization = 
				UE::ActiveProductionData::Private::FProductionSettingsSingleExtendedDataDetails::MakeInstance(TargetScriptStruct);

			// Capture the customization object weakly.
			WeakDetailsCustomization = DetailsCustomization;
			return DetailsCustomization;
		}));
	DetailsView->SetObject(GetMutableDefault<UProductionSettings>());

	DetailsWidgetSwitcher = SNew(SWidgetSwitcher)
		+ SWidgetSwitcher::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			// If no production is active, display a simple message to the user to select a production.
			SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.ToolTipText(LOCTEXT("ActiveProductionDataPanel_NoProductionTooltip", "Select a Production to inspect its data."))
				.Text(LOCTEXT("ActiveProductionDataPanel_NoProductionText", "Select a Production"))
		]

		+ SWidgetSwitcher::Slot()
		[
			DetailsView.ToSharedRef()
		];

	UpdateActiveData();

	return DetailsWidgetSwitcher.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
