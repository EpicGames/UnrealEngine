// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class IDetailsView;
class SWidgetSwitcher;

namespace UE::ActiveProductionData::Private
{
class FProductionSettingsSingleExtendedDataDetails;
} // namespace UE::ActiveProductionData::Private

/** 
 * Widget that shows Extended Data in a details view for the active production. 
 * 
 * This widget displays a view of the Extended Data in the context of the UProductionSettings objects.
 * This means all change events propogate back to the UProductionSettings object.
 */
class SActiveProductionExtendedDataDetailsView : public SCompoundWidget
{
public:
	/** Default constructor. */
	SActiveProductionExtendedDataDetailsView() = default;

	/** Destructor unhooks from UProductionSettings events. */
	~SActiveProductionExtendedDataDetailsView();

	SLATE_BEGIN_ARGS(SActiveProductionExtendedDataDetailsView) :
		_TargetScriptStruct(nullptr),
		_InnerWidget(nullptr),
		_bShowProductionSelection(true)
		{}

		/** The target script structure of Extended Data to show. */
		SLATE_ARGUMENT(const UScriptStruct*, TargetScriptStruct)

		/** A custom widget to display instead of a details view. */
		SLATE_ARGUMENT(TSharedPtr<SWidget>, InnerWidget)

		/** Whether to display the Production Selection combo box. */
		SLATE_ARGUMENT(bool, bShowProductionSelection)
	SLATE_END_ARGS()

	//~ Begin SWidget Interface
	void Construct(const FArguments& InArgs);
	//~ End SWidget Interface

private:
	/** Handles when the active production changes in UProductionSettings. */
	void HandleActiveProductionChanged();

	/** Handles when the production list changes in UProductionSettings. */
	void HandleProductionListChanged();

	/** Updates the active and displayed data. */
	void UpdateActiveData();

	/** Make the contents of the widget taking a custom inner widget and parameters into account. */
	TSharedRef<SWidget> MakeContents(TSharedPtr<SWidget> InnerWidget);

	/** Makes the details widget and sets up customizations for the view. */
	TSharedRef<SWidget> MakeDetailsWidget();

	/** The details view in use. */
	TSharedPtr<IDetailsView> DetailsView;

	/** Switcher widget used to display a message when no active production is selected. */
	TSharedPtr<SWidgetSwitcher> DetailsWidgetSwitcher;

	/** Forward declared customization object used to customize this instance of the details view. */
	TWeakPtr<UE::ActiveProductionData::Private::FProductionSettingsSingleExtendedDataDetails> WeakDetailsCustomization;

	/** The Extended Data script structure to display. */
	const UScriptStruct* TargetScriptStruct = nullptr;

	/** Whether the Productions selection combo box should be shown. */
	bool bShowProductionSelection = true;

	/** Delegate bound to the Production Setting's OnActiveProductionChanged event */
	FDelegateHandle ActiveProductionChangedHandle;

	/** Delegate bound to the Production Setting's OnProductionListChanged event */
	FDelegateHandle ProductionListChangedHandle;
};
