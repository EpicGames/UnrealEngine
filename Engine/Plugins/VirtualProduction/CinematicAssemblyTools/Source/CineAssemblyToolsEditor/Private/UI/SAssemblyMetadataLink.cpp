// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssemblyMetadataLink.h"

#include "Algo/Contains.h"
#include "Algo/Find.h"
#include "CineAssembly.h"
#include "CineAssemblyNamingTokens.h"
#include "CineAssemblySchema.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MovieScene.h"
#include "MovieSceneSubAssemblySection.h"
#include "MovieSceneSubAssemblyTrack.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneSubSection.h"
#include "SNamingTokensEditableTextBox.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#define LOCTEXT_NAMESPACE "SAssemblyMetadataLink"

void SAssemblyMetadataLink::Construct(const FArguments& InArgs, UCineAssembly* InAssembly, FGuid InAssetID)
{
	// The widget requires a valid input assembly to be operational
	if (!ensure(InAssembly))
	{
		return;
	}

	// Asset Mode should not provide an unlinked value widget
	ensure(InArgs._UnlinkedValueWidget.Widget == SNullWidget::NullWidget);

	WeakAssembly = InAssembly;
	AssociatedAssetID = InAssetID;
	bIsAssetMode = true;

	ConstructInternal(InArgs);
}

void SAssemblyMetadataLink::Construct(const FArguments& InArgs, UCineAssembly* InAssembly, const TAttribute<FString>& InMetadataKey)
{
	// The widget requires a valid input assembly to be operational
	if (!ensure(InAssembly))
	{
		return;
	}

	WeakAssembly = InAssembly;
	MetadataKey = InMetadataKey;
	bIsAssetMode = false;

	ConstructInternal(InArgs);
}

void SAssemblyMetadataLink::ConstructInternal(const FArguments& InArgs)
{
	bEvaluateTokens = InArgs._EvaluateTokens;
	const bool bHasValueWidget = InArgs._UnlinkedValueWidget.Widget != SNullWidget::NullWidget;

	NamingTokensContext = TStrongObjectPtr<UCineAssemblyNamingTokensContext>(NewObject<UCineAssemblyNamingTokensContext>());
	UpdateNamingTokensContext();

	FNamingTokenFilterArgs TokenFilterArgs;
	TokenFilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

	TSharedRef<SHorizontalBox> Content = SNew(SHorizontalBox);

	constexpr int32 UnlinkedWidgetIndex = 0;
	constexpr int32 LinkedWidgetIndex = 1;

	if (bHasValueWidget && !bIsAssetMode)
	{
		// HAlign_Left + SBox(Min/Max) renders the value widget at its clamped natural width inside the FillWidth slot.
		// MinDesiredWidth + MaxDesiredWidth help ensure that the link button is not pushed offscreen.
		constexpr float MinValueWidgetWidth = 200.0f;
		constexpr float MaxValueWidgetWidth = 400.0f;

		Content->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SBox)
					.MinDesiredWidth(MinValueWidgetWidth)
					.MaxDesiredWidth(MaxValueWidgetWidth)
					[
						SNew(SWidgetSwitcher)
							.WidgetIndex_Lambda([this]() { return HasAnyLink() ? LinkedWidgetIndex : UnlinkedWidgetIndex; })

						+ SWidgetSwitcher::Slot()
							[
								InArgs._UnlinkedValueWidget.Widget
							]

						+ SWidgetSwitcher::Slot()
							.VAlign(VAlign_Center)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									.AutoWidth()
									.VAlign(VAlign_Fill)
									.Padding(0.0f, 0.0f, 4.0f, 0.0f)
									[
										SNew(SBox)
											.WidthOverride(16.0f)
											.VAlign(VAlign_Center)
											.HAlign(HAlign_Center)
											.Visibility(this, &SAssemblyMetadataLink::GetLinkedAssetIconVisibility)
											[
												SNew(SImage)
													.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
													.Image(this, &SAssemblyMetadataLink::GetLinkedAssetIcon)
											]
									]

								+ SHorizontalBox::Slot()
									.FillWidth(1.0f)
									.VAlign(VAlign_Center)
									[
										SNew(SNamingTokensEditableTextBox)
											.IsReadOnly(true)
											.AllowMultiLine(false)
											.DisplayTokenIcon(false)
											.DisplayBorderImage(false)
											.DisplayErrorMessage(false)
											.ShouldEvaluateTokens(bEvaluateTokens)
											.EvaluationFrequency(1.0f)
											.Contexts({ NamingTokensContext.Get() })
											.FilterArgs(TokenFilterArgs)
											.OnPreEvaluateNamingTokens_Lambda([this]() { UpdateNamingTokensContext(); })
											.Text(this, &SAssemblyMetadataLink::GetLinkedAssetTemplateText)
									]
							]
					]
			];
	}

	Content->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SComboButton)
				.HasDownArrow(false)
				.ContentPadding(FMargin(2.0f))
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText(bIsAssetMode
					? LOCTEXT("AssetModeLinkTooltip", "Link this asset to a metadata field so it is auto-populated when the assembly is created")
					: LOCTEXT("FieldModeLinkTooltip", "Link to an associated asset so this field is auto-populated when the assembly is created"))
				.Visibility(this, &SAssemblyMetadataLink::GetLinkButtonVisibility)
				.OnGetMenuContent(this, &SAssemblyMetadataLink::BuildMenuContent)
				.ButtonContent()
				[
					SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Link"))
						.ColorAndOpacity(this, &SAssemblyMetadataLink::GetLinkButtonColor)
				]
		];

	ChildSlot
	[
		Content
	];
}

TSharedRef<SWidget> SAssemblyMetadataLink::BuildMenuContent()
{
	constexpr bool bCloseAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterMenuSelection, nullptr);

	if (bIsAssetMode)
	{
		BuildAssetModeMenu(MenuBuilder);
	}
	else
	{
		BuildMetadataModeMenu(MenuBuilder);
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ActionsSection", "Actions"));

	const FText UnlinkAllTooltip = bIsAssetMode
		? LOCTEXT("UnlinkAllAssetTooltip", "Clear every metadata field that links to this asset")
		: LOCTEXT("UnlinkAllFieldTooltip", "Clear this field's link");

	MenuBuilder.AddMenuEntry(
		LOCTEXT("UnlinkAllLabel", "Unlink All"),
		UnlinkAllTooltip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SAssemblyMetadataLink::UnlinkAll),
			FCanExecuteAction::CreateSP(this, &SAssemblyMetadataLink::HasAnyLink)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SAssemblyMetadataLink::BuildAssetModeMenu(FMenuBuilder& MenuBuilder)
{
	const UCineAssemblySchema* Schema = GetSchema();
	if (!Schema)
	{
		return;
	}

	// "Properties" section: schema's built-in linkable assembly properties (e.g. DefaultLevel).
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("AssetModeSectionProperties", "Properties"));
	for (const FAssemblyMetadataDesc& MetadataDesc : Schema->GetAssemblyPropertyMetadata())
	{
		if (!IsMetadataDescCompatible(MetadataDesc))
		{
			continue;
		}
		const FText Tooltip = IsAssetLinked(MetadataDesc.Key, AssociatedAssetID)
			? FText::Format(LOCTEXT("UnlinkMetaTooltip", "Unlink from the '{0}' metadata field"), FText::FromString(MetadataDesc.Key))
			: FText::Format(LOCTEXT("LinkMetaTooltip", "Link to the '{0}' metadata field"), FText::FromString(MetadataDesc.Key));

		MenuBuilder.AddMenuEntry(
			FText::FromString(MetadataDesc.Key),
			Tooltip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SAssemblyMetadataLink::ToggleMetadataLink, MetadataDesc.Key, AssociatedAssetID),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SAssemblyMetadataLink::IsAssetLinked, MetadataDesc.Key, AssociatedAssetID)
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
	}
	MenuBuilder.EndSection();

	// "Metadata" section: user-defined fields on the schema.
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("AssetModeSectionMetadata", "Metadata"));
	for (const FAssemblyMetadataDesc& MetadataDesc : Schema->AssemblyMetadata)
	{
		if (!IsMetadataDescCompatible(MetadataDesc))
		{
			continue;
		}
		const FText Tooltip = IsAssetLinked(MetadataDesc.Key, AssociatedAssetID)
			? FText::Format(LOCTEXT("UnlinkMetaTooltip", "Unlink from the '{0}' metadata field"), FText::FromString(MetadataDesc.Key))
			: FText::Format(LOCTEXT("LinkMetaTooltip", "Link to the '{0}' metadata field"), FText::FromString(MetadataDesc.Key));

		MenuBuilder.AddMenuEntry(
			FText::FromString(MetadataDesc.Key),
			Tooltip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SAssemblyMetadataLink::ToggleMetadataLink, MetadataDesc.Key, AssociatedAssetID),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SAssemblyMetadataLink::IsAssetLinked, MetadataDesc.Key, AssociatedAssetID)
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
	}
	MenuBuilder.EndSection();
}

void SAssemblyMetadataLink::BuildMetadataModeMenu(FMenuBuilder& MenuBuilder)
{
	UCineAssembly* Assembly = WeakAssembly.Get();
	const UCineAssemblySchema* Schema = GetSchema();
	if (!Assembly || !Schema)
	{
		return;
	}

	const FAssemblyMetadataDesc* MetadataDesc = Schema->FindMetadataDesc(MetadataKey.Get());
	if (!MetadataDesc)
	{
		return;
	}

	// Metadata Mode: list compatible associated assets as radio buttons (a metadata field can only be linked to one asset)
	// Associated assets are only compatible with String and AssetPath metadata types, not CineAssembly
	if (MetadataDesc->Type == ECineAssemblyMetadataType::String || MetadataDesc->Type == ECineAssemblyMetadataType::AssetPath)
	{
		const UClass* AllowedClass = MetadataDesc->AssetClass.ResolveClass();

		// Collect the distinct classes among compatible associated assets to make distinct menu sections
		TArray<UClass*> AssetClasses;
		for (const FAssemblyAssociatedAssetDesc& AssetDesc : Assembly->AssociatedAssets)
		{
			UClass* AssetClass = AssetDesc.AssetClass.Get();
			if (IsAssetClassCompatible(AssetClass, AllowedClass))
			{
				AssetClasses.AddUnique(AssetClass);
			}
		}

		// Add associated assets to the appropriate menu section
		for (UClass* SectionClass : AssetClasses)
		{
			const FText SectionLabel = SectionClass
				? FText::Format(LOCTEXT("AssociatedAssetSectionFmt", "{0}s"), SectionClass->GetDisplayNameText())
				: LOCTEXT("AssociatedAssetSectionGeneric", "Associated Assets");

			MenuBuilder.BeginSection(NAME_None, SectionLabel);
			for (const FAssemblyAssociatedAssetDesc& AssetDesc : Assembly->AssociatedAssets)
			{
				if (AssetDesc.AssetClass.Get() == SectionClass)
				{
					AddMenuEntryForAsset(MenuBuilder, AssetDesc.AssetID, FText::FromString(AssetDesc.AssetName.Template));
				}
			}
			MenuBuilder.EndSection();
		}
	}

	// Also list compatible SubAssemblies in their own section
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("SubAssemblySection", "Sub-Assemblies"));
	ForEachSubAssembly([this, &MenuBuilder, MetadataDesc](FGuid SubAssemblyID, const FText& DisplayName, const UCineAssemblySchema*)
	{
		if (IsSubAssemblyCompatible(SubAssemblyID, *MetadataDesc))
		{
			AddMenuEntryForAsset(MenuBuilder, SubAssemblyID, DisplayName);
		}
		return true;
	});
	MenuBuilder.EndSection();
}

void SAssemblyMetadataLink::AddMenuEntryForAsset(FMenuBuilder& MenuBuilder, FGuid AssetID, const FText& DisplayName)
{
	// Resolve the icon based on whether the entry is an associated asset or a SubAssembly
	FSlateIcon AssetIcon = FSlateIconFinder::FindIconForClass(UCineAssembly::StaticClass());

	if (UCineAssembly* Assembly = WeakAssembly.Get())
	{
		if (const FAssemblyAssociatedAssetDesc* AssetDesc = Algo::FindBy(Assembly->AssociatedAssets, AssetID, &FAssemblyAssociatedAssetDesc::AssetID))
		{
			AssetIcon = FSlateIconFinder::FindIconForClass(AssetDesc->AssetClass.Get());
		}
	}

	const FString CurrentKey = MetadataKey.Get();
	const FText Tooltip = IsAssetLinked(CurrentKey, AssetID)
		? FText::Format(LOCTEXT("UnlinkTooltip", "Unlink from '{0}'"), DisplayName)
		: FText::Format(LOCTEXT("LinkTooltip", "Link to '{0}'"), DisplayName);

	MenuBuilder.AddMenuEntry(
		DisplayName,
		Tooltip,
		AssetIcon,
		FUIAction(
			FExecuteAction::CreateSP(this, &SAssemblyMetadataLink::ToggleMetadataLink, CurrentKey, AssetID),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SAssemblyMetadataLink::IsAssetLinked, CurrentKey, AssetID)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
}

void SAssemblyMetadataLink::ToggleMetadataLink(FString InMetadataKey, FGuid InAssetID)
{
	UCineAssembly* Assembly = WeakAssembly.Get();
	if (!Assembly)
	{
		return;
	}

	Assembly->Modify();
	if (IsAssetLinked(InMetadataKey, InAssetID))
	{
		Assembly->MetadataLinks.Remove(InMetadataKey);
	}
	else
	{
		Assembly->MetadataLinks.Add(InMetadataKey, InAssetID);

		if (InMetadataKey == UCineAssemblySchema::DefaultLevelMetadataKey)
		{
			if (UCineAssemblySchema* Schema = Assembly->GetTypedOuter<UCineAssemblySchema>())
			{
				if (!Schema->bOverrideDefaultLevel)
				{
					Schema->Modify();
					Schema->bOverrideDefaultLevel = true;
				}
			}
		}
	}
}

void SAssemblyMetadataLink::UnlinkAll()
{
	UCineAssembly* Assembly = WeakAssembly.Get();
	if (!Assembly)
	{
		return;
	}

	const FScopedTransaction Transaction(bIsAssetMode ? LOCTEXT("UnlinkAllAssetTransaction", "Unlink all metadata fields from asset") : LOCTEXT("UnlinkAllFieldTransaction", "Unlink metadata field"));
	Assembly->Modify();

	if (bIsAssetMode)
	{
		TArray<FString> KeysToRemove;
		for (const TPair<FString, FGuid>& MetadataLink : Assembly->MetadataLinks)
		{
			if (MetadataLink.Value == AssociatedAssetID)
			{
				KeysToRemove.Add(MetadataLink.Key);
			}
		}

		for (const FString& Key : KeysToRemove)
		{
			Assembly->MetadataLinks.Remove(Key);
		}
	}
	else
	{
		Assembly->MetadataLinks.Remove(MetadataKey.Get());
	}
}

bool SAssemblyMetadataLink::IsAssetLinked(FString InMetadataKey, FGuid InAssetID) const
{
	if (UCineAssembly* Assembly = WeakAssembly.Get())
	{
		const FGuid* LinkedID = Assembly->MetadataLinks.Find(InMetadataKey);
		return LinkedID && *LinkedID == InAssetID;
	}
	return false;
}

bool SAssemblyMetadataLink::HasAnyLink() const
{
	UCineAssembly* Assembly = WeakAssembly.Get();
	if (!Assembly)
	{
		return false;
	}

	if (bIsAssetMode)
	{
		// Check if any metadata key in the map points to this asset
		for (const TPair<FString, FGuid>& MetadataLink : Assembly->MetadataLinks)
		{
			if (MetadataLink.Value == AssociatedAssetID)
			{
				return true;
			}
		}
		return false;
	}
	else if (const UCineAssemblySchema* Schema = GetSchema())
	{
		const FString CurrentKey = MetadataKey.Get();
		const bool bKeyIsValid = Schema->FindMetadataDesc(CurrentKey) != nullptr;
		return bKeyIsValid && Assembly->MetadataLinks.Contains(CurrentKey);
	}

	return false;
}

EVisibility SAssemblyMetadataLink::GetLinkButtonVisibility() const
{
	UCineAssembly* Assembly = WeakAssembly.Get();
	const UCineAssemblySchema* Schema = GetSchema();
	if (!Assembly || !Schema)
	{
		return EVisibility::Collapsed;
	}

	if (bIsAssetMode)
	{
		// Visible if there are any compatible metadata fields
		bool bHasCompatibleField = false;
		Schema->ForEachMetadataDesc([this, &bHasCompatibleField](const FAssemblyMetadataDesc& MetadataDesc)
		{
			bHasCompatibleField = IsMetadataDescCompatible(MetadataDesc);
			return !bHasCompatibleField;
		});
		return bHasCompatibleField ? EVisibility::Visible : EVisibility::Collapsed;
	}
	else if (const FAssemblyMetadataDesc* MetadataDesc = Schema->FindMetadataDesc(MetadataKey.Get()))
	{
		// Check if any associated assets are compatible (only for String and AssetPath types)
		if (MetadataDesc->Type == ECineAssemblyMetadataType::String || MetadataDesc->Type == ECineAssemblyMetadataType::AssetPath)
		{
			const UClass* AllowedClass = MetadataDesc->AssetClass.ResolveClass();
			const bool bHasCompatibleAsset = Assembly->AssociatedAssets.ContainsByPredicate([this, AllowedClass](const FAssemblyAssociatedAssetDesc& AssetDesc) { return IsAssetClassCompatible(AssetDesc.AssetClass.Get(), AllowedClass); });
			if (bHasCompatibleAsset)
			{
				return EVisibility::Visible;
			}
		}

		// Check if any SubAssemblies are compatible
		bool bHasCompatibleSubAssembly = false;
		ForEachSubAssembly([this, MetadataDesc, &bHasCompatibleSubAssembly](FGuid SubAssemblyID, const FText&, const UCineAssemblySchema*)
		{
			bHasCompatibleSubAssembly = IsSubAssemblyCompatible(SubAssemblyID, *MetadataDesc);
			return !bHasCompatibleSubAssembly; // Stop on first match
		});

		if (bHasCompatibleSubAssembly)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

FSlateColor SAssemblyMetadataLink::GetLinkButtonColor() const
{
	return HasAnyLink() ? FStyleColors::White : FSlateColor::UseSubduedForeground();
}

FText SAssemblyMetadataLink::GetLinkedAssetTemplateText() const
{
	if (bIsAssetMode)
	{
		// In Asset Mode, there is no extra unlinked/linked slot to display any text for
		return FText::GetEmpty();
	}

	UCineAssembly* Assembly = WeakAssembly.Get();
	if (!Assembly)
	{
		return FText::GetEmpty();
	}

	const FGuid* LinkedID = Assembly->MetadataLinks.Find(MetadataKey.Get());
	if (!LinkedID)
	{
		return FText::GetEmpty();
	}

	if (const FAssemblyAssociatedAssetDesc* AssetDesc = Algo::FindBy(Assembly->AssociatedAssets, *LinkedID, &FAssemblyAssociatedAssetDesc::AssetID))
	{
		return FText::FromString(AssetDesc->AssetName.Template);
	}

	FText SubAssemblyName;
	ForEachSubAssembly([&LinkedID, &SubAssemblyName](FGuid ID, const FText& DisplayName, const UCineAssemblySchema*)
	{
		if (ID == *LinkedID)
		{
			SubAssemblyName = DisplayName;
			return false;
		}
		return true;
	});

	if (!SubAssemblyName.IsEmpty())
	{
		return SubAssemblyName;
	}

	return LOCTEXT("LinkedAssetNotFound", "(Linked target not found)");
}

void SAssemblyMetadataLink::UpdateNamingTokensContext()
{
	if (!NamingTokensContext.IsValid())
	{
		return;
	}

	UCineAssembly* Assembly = WeakAssembly.Get();
	if (!Assembly)
	{
		NamingTokensContext->Assembly = nullptr;
		return;
	}

	// For SubAssembly links, use the SubAssembly for the naming tokens context
	if (!bIsAssetMode)
	{
		if (const FGuid* LinkedID = Assembly->MetadataLinks.Find(MetadataKey.Get()))
		{
			if (UCineAssembly* LinkedSubAssembly = Assembly->FindSubAssembly(*LinkedID))
			{
				NamingTokensContext->Assembly = LinkedSubAssembly;
				return;
			}
		}
	}

	// For Associated Asset links (or asset mode), use the parent assembly.
	NamingTokensContext->Assembly = Assembly;
}

const FSlateBrush* SAssemblyMetadataLink::GetLinkedAssetIcon() const
{
	if (bIsAssetMode)
	{
		return nullptr;
	}

	UCineAssembly* Assembly = WeakAssembly.Get();
	if (!Assembly)
	{
		return nullptr;
	}

	const FGuid* LinkedID = Assembly->MetadataLinks.Find(MetadataKey.Get());
	if (!LinkedID)
	{
		return nullptr;
	}

	// If the linked asset is an associated asset, get the icon for its class.
	// Otherwise the link targets a SubAssembly (or a stale GUID)
	const UClass* IconClass = UCineAssembly::StaticClass();
	if (const FAssemblyAssociatedAssetDesc* AssetDesc = Algo::FindBy(Assembly->AssociatedAssets, *LinkedID, &FAssemblyAssociatedAssetDesc::AssetID))
	{
		IconClass = AssetDesc->AssetClass.Get();
	}
	return FSlateIconFinder::FindIconForClass(IconClass ? IconClass : UObject::StaticClass()).GetIcon();
}

EVisibility SAssemblyMetadataLink::GetLinkedAssetIconVisibility() const
{
	return GetLinkedAssetIcon() ? EVisibility::Visible : EVisibility::Collapsed;
}

const UCineAssemblySchema* SAssemblyMetadataLink::GetSchema() const
{
	UCineAssembly* Assembly = WeakAssembly.Get();
	if (!Assembly)
	{
		return nullptr;
	}

	// Try the outer chain first (for template sequences owned by schemas), then fall back to GetSchema()
	if (const UCineAssemblySchema* Schema = Assembly->GetTypedOuter<UCineAssemblySchema>())
	{
		return Schema;
	}
	return Assembly->GetSchema();
}

bool SAssemblyMetadataLink::IsAssetClassCompatible(const UClass* InAssetClass, const UClass* InFilterClass) const
{
	if (!InFilterClass)
	{
		return true;
	}
	return InAssetClass && InAssetClass->IsChildOf(InFilterClass);
}

bool SAssemblyMetadataLink::IsSchemaCompatible(FGuid InSubAssemblyID, const FSoftObjectPath& InSchemaType) const
{
	if (!InSchemaType.IsValid())
	{
		return true;
	}

	bool bIsCompatible = false;
	ForEachSubAssembly([&](FGuid ID, const FText&, const UCineAssemblySchema* Schema)
	{
		if (ID == InSubAssemblyID)
		{
			bIsCompatible = Schema && InSchemaType == FSoftObjectPath(Schema);
			return false;
		}
		return true;
	});

	return bIsCompatible;
}

bool SAssemblyMetadataLink::IsMetadataDescCompatible(const FAssemblyMetadataDesc& InMetadataDesc) const
{
	switch(InMetadataDesc.Type)
	{
	case ECineAssemblyMetadataType::String:
		return true;

	case ECineAssemblyMetadataType::AssetPath:
		{
			const UClass* AssetClass = nullptr;
			if (UCineAssembly* Assembly = WeakAssembly.Get())
			{
				if (const FAssemblyAssociatedAssetDesc* AssetDesc = Algo::FindBy(Assembly->AssociatedAssets, AssociatedAssetID, &FAssemblyAssociatedAssetDesc::AssetID))
				{
					AssetClass = AssetDesc->AssetClass.Get();
				}
			}
			return IsAssetClassCompatible(AssetClass, InMetadataDesc.AssetClass.ResolveClass());
		}

	case ECineAssemblyMetadataType::CineAssembly:
		{
			// CineAssembly metadata can only link to SubAssemblies, not associated assets
			if (UCineAssembly* Assembly = WeakAssembly.Get())
			{
				if (Algo::FindBy(Assembly->AssociatedAssets, AssociatedAssetID, &FAssemblyAssociatedAssetDesc::AssetID))
				{
					return false;
				}
			}
			return IsSchemaCompatible(AssociatedAssetID, InMetadataDesc.SchemaType);
		}

	case ECineAssemblyMetadataType::Bool:
	case ECineAssemblyMetadataType::Integer:
	case ECineAssemblyMetadataType::Float:
		return false;

	default:
		checkNoEntry();
	}

	return false;
}

bool SAssemblyMetadataLink::IsSubAssemblyCompatible(FGuid InSubAssemblyID, const FAssemblyMetadataDesc& InMetadataDesc) const
{
	switch (InMetadataDesc.Type)
	{
	case ECineAssemblyMetadataType::String:
		return true;

	case ECineAssemblyMetadataType::AssetPath:
		return IsAssetClassCompatible(UCineAssembly::StaticClass(), InMetadataDesc.AssetClass.ResolveClass());

	case ECineAssemblyMetadataType::CineAssembly:
		return IsSchemaCompatible(InSubAssemblyID, InMetadataDesc.SchemaType);

	case ECineAssemblyMetadataType::Bool:
	case ECineAssemblyMetadataType::Integer:
	case ECineAssemblyMetadataType::Float:
		return false;

	default:
		checkNoEntry();
	}

	return false;
}

void SAssemblyMetadataLink::ForEachSubAssembly(TFunctionRef<bool(FGuid ID, const FText& DisplayName, const UCineAssemblySchema* Schema)> Visitor) const
{
	UCineAssembly* Assembly = WeakAssembly.Get();
	if (!Assembly)
	{
		return;
	}

	// If the Assembly is the TemplateSequence of a Schema, then we traverse the SubAssembly Sections. 
	// Otherwise, the Assembly is a real configured object, so we traverse the managed SubAssemblies list. 
	const bool bIsTemplateAssembly = Assembly->GetTypedOuter<UCineAssemblySchema>() != nullptr;
	if (bIsTemplateAssembly)
	{
		if (UMovieScene* MovieScene = Assembly->GetMovieScene())
		{
			for (UMovieSceneSection* Section : MovieScene->GetAllSections())
			{
				if (UMovieSceneSubAssemblySection* SubAssemblySection = Cast<UMovieSceneSubAssemblySection>(Section))
				{
					if (SubAssemblySection->IsTemplateSection())
					{
						if (!Visitor(SubAssemblySection->GetSectionID(), SubAssemblySection->GetSequenceName(), SubAssemblySection->GetTemplateSchema()))
						{
							return;
						}
					}
				}
			}
		}
	}
	else
	{
		for (UMovieSceneSubSection* SubSection : Assembly->SubAssemblies)
		{
			if (UCineAssembly* SubAssembly = Cast<UCineAssembly>(SubSection ? SubSection->GetSequence() : nullptr))
			{
				if (!Visitor(SubAssembly->GetAssemblyGuid(), FText::FromString(SubAssembly->AssemblyName.Template), SubAssembly->GetSchema()))
				{
					return;
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
