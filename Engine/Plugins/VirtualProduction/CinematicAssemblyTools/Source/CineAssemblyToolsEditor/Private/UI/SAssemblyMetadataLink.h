// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CineAssemblyNamingTokens.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/SCompoundWidget.h"

class FMenuBuilder;
class UCineAssembly;
class UCineAssemblySchema;
struct FAssemblyAssociatedAssetDesc;
struct FAssemblyMetadataDesc;
struct FSlateBrush;

/**
 * A composite widget that handles the linking between a CineAssembly's associated assets and metadata fields.
 *
 * Operates in one of two modes:
 *     Asset Mode:    Handles the logic to link an associated asset to one or more metadata fields
 *     Metadata Mode: Handles the logic to link a metadata field to an associated asset
 *     
 * In Metadata Mode, this widget wraps a value widget to display when the field is not linked, 
 * and handles switching to a read-only textbox displaying the name of the linked asset when it is linked.
 */
class SAssemblyMetadataLink : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAssemblyMetadataLink)
		: _EvaluateTokens(false)
		{}

		/** Optional value widget to show when unlinked. Only applicable to Metadata Mode */
		SLATE_DEFAULT_SLOT(FArguments, UnlinkedValueWidget)

		/** Whether to evaluate naming tokens when displaying the linked asset name */
		SLATE_ARGUMENT(bool, EvaluateTokens)

	SLATE_END_ARGS()

	/** Asset Mode */
	void Construct(const FArguments& InArgs, UCineAssembly* InAssembly, FGuid InAssetID);

	/** 
	 * Metadata Mode. 
	 * InMetadataKey is a TAttribute to prevent the widget from caching a FString Key that could be renamed. 
	 */
	void Construct(const FArguments& InArgs, UCineAssembly* InAssembly, const TAttribute<FString>& InMetadataKey);

private:
	/** Shared widget construction for Asset and Metadata modes */
	void ConstructInternal(const FArguments& InArgs);

	/** Builds the dropdown menu content showing the link options */
	TSharedRef<SWidget> BuildMenuContent();

	/** Builds the asset mode dropdown menu options with metadata fields this asset can link to */
	void BuildAssetModeMenu(FMenuBuilder& MenuBuilder);

	/** Builds the metadata mode dropdown menu options with associated assets and SubAssemblies this field can link to */
	void BuildMetadataModeMenu(FMenuBuilder& MenuBuilder);

	/** Adds a menu entry to the dropdown menu content for the input Asset */
	void AddMenuEntryForAsset(FMenuBuilder& MenuBuilder, FGuid AssetID, const FText& DisplayName);

	/** Toggles a link between a metadata key and an asset GUID */
	void ToggleMetadataLink(FString InMetadataKey, FGuid InAssetID);

	/** Whether the input metadata key and associated asset are currently linked */
	bool IsAssetLinked(FString InMetadataKey, FGuid InAssetID) const;

	/**
	 * AssetMode: Whether the tracked AssociatedAssetID is linked to any metadata field
	 * MetadataMode: Whether the tracked MetadataKey is linked to any associated asset
	 */
	bool HasAnyLink() const;

	/**
	 * Clears every metadata link relevant to this widget's mode in a single transaction.
	 * AssetMode: removes every MetadataLinks entry that points at the tracked AssociatedAssetID.
	 * MetadataMode: removes the MetadataLinks entry for the tracked MetadataKey.
	 */
	void UnlinkAll();

	/** Determines the visibility of the link button */
	EVisibility GetLinkButtonVisibility() const;

	/** Returns the link button icon color based on link status */
	FSlateColor GetLinkButtonColor() const;

	/**
	 * Returns the linked target's (unresolved) template text. Token evaluation is handled by the SNamingTokensEditableTextBox
	 * when EvaluateTokens is true, using NamingTokensContext as its evaluation context.
	 */
	FText GetLinkedAssetTemplateText() const;

	/** Updates NamingTokensContext->Assembly to match the current link target so token evaluation resolves correctly. */
	void UpdateNamingTokensContext();

	/** Returns the class icon for the linked target, or nullptr if the target can't be resolved. */
	const FSlateBrush* GetLinkedAssetIcon() const;

	/** Collapses the linked asset icon slot when no icon can be resolved. */
	EVisibility GetLinkedAssetIconVisibility() const;

	/** Returns the schema that defines the assembly's metadata fields */
	const UCineAssemblySchema* GetSchema() const;

	/** Whether the AssetClass is compatible with the FilterClass. Always returns true if FilterClass is nullptr. */
	bool IsAssetClassCompatible(const UClass* InAssetClass, const UClass* InFilterClass) const;

	/** Whether the input SubAssembly has a schema that is compatible with the input SchemaType filter. Always return true if SchemaType is not specified. */
	bool IsSchemaCompatible(FGuid InSubAssemblyID, const FSoftObjectPath& InSchemaType) const;

	/** Whether the input metadata desc is compatible for linking with this asset */
	bool IsMetadataDescCompatible(const FAssemblyMetadataDesc& InMetadataDesc) const;

	/** Whether the input SubAssembly is compatible for linking with the input metadata desc */
	bool IsSubAssemblyCompatible(FGuid InSubAssemblyID, const FAssemblyMetadataDesc& InMetadataDesc) const;

	/** Iterates all SubAssemblies of the tracked Assembly and calls the visitor for each */
	void ForEachSubAssembly(TFunctionRef<bool(FGuid ID, const FText& DisplayName, const UCineAssemblySchema* Schema)> Visitor) const;

private:
	/** The assembly whose associated assets and schema metadata are being linked */
	TWeakObjectPtr<UCineAssembly> WeakAssembly;

	/** Asset Mode: the associated asset being linked */
	FGuid AssociatedAssetID;

	/** Metadata Mode: the metadata field key. Stored as a TAttribute so renames made while the widget is alive are reflected on the next read. */
	TAttribute<FString> MetadataKey;

	/** Whether this widget is in Asset Mode (true) or Metadata Mode (false) */
	bool bIsAssetMode = false;

	/** Whether to evaluate naming tokens when displaying the linked asset name */
	bool bEvaluateTokens = false;

	/** Context for naming token evaluation */
	TStrongObjectPtr<UCineAssemblyNamingTokensContext> NamingTokensContext;
};
