// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssetNamingPanel.h"

#include "AssetDefinitionRegistry.h"
#include "AssetToolsModule.h"
#include "CineAssemblyNamingTokens.h"
#include "CineAssemblyToolsStyle.h"
#include "ClassViewerFilter.h"
#include "Factories/Factory.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ProductionSettings.h"
#include "PropertyCustomizationHelpers.h"
#include "Styling/StyleColors.h"
#include "UI/SActiveProductionCombo.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SAssetNamingPanel"

namespace UE::CineAssemblyTools::AssetNaming::Private
{
	// Filter for all classes that have a registered asset definition
	// This roughly approximates anything that the user can create in the content browser, and that they might want to assign a default name
	class FAssetNamingClassFilter : public IClassViewerFilter
	{
	public:
		FAssetNamingClassFilter()
		{
			IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
			TArray<UFactory*> Factories = AssetTools.GetNewAssetFactories();
			for (UFactory* Factory : Factories)
			{
				AllowedAssetClasses.Add(Factory->GetSupportedClass());
			}
		}

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
		{
			return AllowedAssetClasses.Contains(InClass);
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const class IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<class FClassViewerFilterFuncs> InFilterFuncs) override
		{
			return false;
		}

	private:
		TArray<UClass*> AllowedAssetClasses;
	};

	FString GetFactoryDefaultNameForClass(const UClass* Class)
	{
		const IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		FString FactoryDefaultName;
		for (UFactory* Factory : AssetTools.GetNewAssetFactories())
		{
			UClass* SupportedClass = Factory->GetSupportedClass();

			if (SupportedClass != nullptr && SupportedClass == Class)
			{
				FactoryDefaultName = Factory->GetDefaultNewAssetName();
				break;
			}
		}

		if (FactoryDefaultName.IsEmpty())
		{
			FactoryDefaultName = FString::Printf(TEXT("New%s"), *Class->GetName());
		}

		return FactoryDefaultName;
	}

	static FName ClassColumnName = TEXT("Class");
	static FName NamingColumnName = TEXT("Naming");
	static FName DeleteColumnName = TEXT("Delete");
}

void SAssetNamingPanel::Construct(const FArguments& InArgs)
{
	using namespace UE::CineAssemblyTools::AssetNaming::Private;

	// Subscribe to be notified when the Production Settings active productions has changed
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ActiveProductionChangedHandle = ProductionSettings->OnActiveProductionChanged().AddSP(this, &SAssetNamingPanel::UpdateAssetNamingList);

	// Initialize the asset naming list items for the current active production
	UpdateAssetNamingList();

	// Lambdas associate with the "Create New Naming" button
	auto IsActiveProductionValid = []() -> bool
		{
			const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
			TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
			return ActiveProduction.IsSet();
		};

	auto OnCreateNewNaming = [this]() -> FReply
		{
			AssetNamingListItems.Add(MakeShared<FAssetNamingRowData>());

			if (AssetNamingListView)
			{
				AssetNamingListView->RequestListRefresh();
			}

			return FReply::Handled();
		};

	ChildSlot
		[
			SNew(SVerticalBox)

			// Active Production Selector
			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SActiveProductionCombo)
				]

			// Separator
			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator)
						.Orientation(Orient_Horizontal)
						.Thickness(2.0f)
				]

			// Asset Naming Panel
			+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
						.Padding(16.0f)
						[
							SNew(SVerticalBox)

							// Title
							+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 4.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("AssetNamingTitle", "Production Settings"))
										.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.TitleFont"))
								]

							// Heading
							+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 4.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("AssetNamingHeading", "Asset Naming"))
										.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.HeadingFont"))
								]

							// Info Text
							+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 16.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("AssetNamingInfoText", "For each asset class, you can set a name to be used automatically for newly created assets. This helps keep asset names consistent across your project."))
										.AutoWrapText(true)
								]

							// Create New Naming Button
							+ SVerticalBox::Slot()
								.AutoHeight()
								.HAlign(HAlign_Left)
								.Padding(0.0f, 0.0f, 0.0f, 8.0f)
								[
									SNew(SButton)
										.ContentPadding(FMargin(2.0f))
										.OnClicked_Lambda(OnCreateNewNaming)
										.IsEnabled_Lambda(IsActiveProductionValid)
										[
											SNew(SHorizontalBox)

											+ SHorizontalBox::Slot()
												.AutoWidth()
												.Padding(0.0f, 0.0f, 4.0f, 0.0f)
												[
													SNew(SImage)
														.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
														.ColorAndOpacity(FStyleColors::AccentGreen)
												]

											+ SHorizontalBox::Slot()
												.AutoWidth()
												[
													SNew(STextBlock).Text(LOCTEXT("CreateNewNamingButton", "Create New Naming"))
												]
										]
								]

							// Asset Naming List View
							+ SVerticalBox::Slot()
								.FillHeight(1.0f)
								[
									SAssignNew(AssetNamingListView, SListView<TSharedPtr<FAssetNamingRowData>>)
										.ListItemsSource(&AssetNamingListItems)
										.OnGenerateRow(this, &SAssetNamingPanel::OnGenerateAssetNamingRow)
										.OnMouseButtonDoubleClick(this, &SAssetNamingPanel::OnListViewDoubleClick)
										.SelectionMode(ESelectionMode::None)
										.HeaderRow
										(
											SNew(SHeaderRow)

											+ SHeaderRow::Column(ClassColumnName)
												.DefaultLabel(LOCTEXT("ClassLabel", "Class"))
												.FillWidth(0.25f)

											+ SHeaderRow::Column(NamingColumnName)
												.DefaultLabel(LOCTEXT("NamingLabel", "Default Naming"))
												.FillWidth(0.7f)

											+ SHeaderRow::Column(DeleteColumnName)
												.DefaultLabel(FText::GetEmpty())
												.FillWidth(0.05f)
										)
								]
						]
				]
		];
}

SAssetNamingPanel::~SAssetNamingPanel()
{
	if (UObjectInitialized())
	{
		if (UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>())
		{
			ProductionSettings->OnActiveProductionChanged().Remove(ActiveProductionChangedHandle);
		}
	}
}

void SAssetNamingPanel::UpdateAssetNamingList()
{
	AssetNamingListItems.Empty();

	const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
	TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
	if (ActiveProduction.IsSet())
	{
		for (const TPair<TObjectPtr<const UClass>, FString>& AssetNaming : ActiveProduction.GetValue().DefaultAssetNames)
		{
			AssetNamingListItems.Add(MakeShared<FAssetNamingRowData>(AssetNaming.Key, AssetNaming.Value));
		}
	}

	if (AssetNamingListView)
	{
		AssetNamingListView->RequestListRefresh();
	}
}

TSharedRef<ITableRow> SAssetNamingPanel::OnGenerateAssetNamingRow(TSharedPtr<FAssetNamingRowData> InAssetNaming, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SAssetNamingRow, OwnerTable, InAssetNaming)
		.OnDeleteRow(this, &SAssetNamingPanel::UpdateAssetNamingList);
}

void SAssetNamingPanel::OnListViewDoubleClick(TSharedPtr<FAssetNamingRowData> InAssetNaming)
{
	if (TSharedPtr<SAssetNamingRow> Widget = StaticCastSharedPtr<SAssetNamingRow>(AssetNamingListView->WidgetFromItem(InAssetNaming)))
	{
		Widget->EditableTextBlock->EnterEditingMode();
	}
}

void SAssetNamingRow::Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedPtr<FAssetNamingRowData>& InAssetNaming)
{
	AssetNaming = InAssetNaming;

	OnDeleteRow = Args._OnDeleteRow;

	SMultiColumnTableRow<TSharedPtr<FAssetNamingRowData>>::Construct(FSuperRowType::FArguments(), OwnerTableView);
}

TSharedRef<SWidget> SAssetNamingRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	using namespace UE::CineAssemblyTools::AssetNaming::Private;

	if (ColumnName == ClassColumnName)
	{
		return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 0.0f, 1.0f)
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
					.Padding(FMargin(4.0, 1.0f))
					[
						SNew(SClassPropertyEntryBox)
							.MetaClass(UObject::StaticClass())
							.SelectedClass_Lambda([this]() { return AssetNaming->Class; })
							.ClassViewerFilters({ MakeShared<FAssetNamingClassFilter>() })
							.ShowDisplayNames(true)
							.OnSetClass(this, &SAssetNamingRow::SetAssetClass)
					]
			];
	}
	else if (ColumnName == NamingColumnName)
	{
		FNamingTokenFilterArgs FilterArgs;
		FilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

		EditableTextBlock = SNew(SNamingTokensEditableTextBox)
			.SupportInlineEdit(true)
			.ShouldEvaluateTokens(false)
			.FilterArgs(FilterArgs)
			.DisplayTokenIcon(false)
			.DisplayBorderImage(false)
			.IsReadOnly_Lambda([this]() { return (AssetNaming->Class == nullptr); })
			.DisplayErrorMessage_Lambda([this]() { return (AssetNaming->Class != nullptr); })
			.Text_Lambda([this]() { return FText::FromString(AssetNaming->DefaultName); })
			.OnTextCommitted(this, &SAssetNamingRow::SetDefaultAssetName)
			.OnValidateTokenizedText(this, &SAssetNamingRow::ValidateDefaultAssetName);

		return SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.Padding(1.0f, 0.0f, 0.0f, 1.0f)
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
					.Padding(FMargin(4.0, 1.0f))
					.OnMouseButtonDown(this, &SAssetNamingRow::SummonEditMenu)
					[
						EditableTextBlock.ToSharedRef()
					]
			];
	}
	else if (ColumnName == DeleteColumnName)
	{
		return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
			.Padding(1.0f, 0.0f, 0.0f, 1.0f)
			[
				SNew(SButton)
					.ButtonStyle(FCineAssemblyToolsStyle::Get(), "ProductionWizard.PanelButton")
					.ContentPadding(FMargin(0.0f, 0.0f))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.OnClicked(this, &SAssetNamingRow::DeleteRow)
					.Content()
					[
						SNew(SImage).Image(FAppStyle::Get().GetBrush("Icons.Delete"))
					]
			];
	}

	return SNullWidget::NullWidget;
}

void SAssetNamingRow::SetAssetClass(const UClass* SelectedClass)
{
	// Early-out if the same class was just selected
	if (AssetNaming->Class == SelectedClass)
	{
		return;
	}

	// Before updating the asset class for this row, remove the old class from the active production settings list
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ProductionSettings->RemoveAssetNaming(ProductionSettings->GetActiveProductionID(), AssetNaming->Class);

	AssetNaming->Class = SelectedClass;

	if (SelectedClass)
	{	
		// Set the default name for this row to the factory default name for the selected asset type
		AssetNaming->DefaultName = UE::CineAssemblyTools::AssetNaming::Private::GetFactoryDefaultNameForClass(SelectedClass);

		// Let the user immediately start editing the asset name
		FSlateApplication::Get().SetKeyboardFocus(EditableTextBlock.ToSharedRef());
		EditableTextBlock->EnterEditingMode();
	}
	else
	{
		AssetNaming->DefaultName.Empty();
	}
}

void SAssetNamingRow::SetDefaultAssetName(const FText& InText, ETextCommit::Type InCommitType)
{
	AssetNaming->DefaultName = InText.ToString();

	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ProductionSettings->AddAssetNaming(ProductionSettings->GetActiveProductionID(), AssetNaming->Class, AssetNaming->DefaultName);
}

bool SAssetNamingRow::ValidateDefaultAssetName(const FText& InText, FText& OutErrorMessage)
{
	// An empty name is invalid
	if (InText.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("EmptyNameErrorMessage", "Please provide a default asset name");
		return false;
	}

	// Ensure that the name does not contain any characters that would be invalid for an asset name
	// This matches the validation that would happen if the user was renaming an asset in the content browser
	FString InvalidCharacters = INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS;

	// These characters are actually valid, because we want to support naming tokens
	InvalidCharacters = InvalidCharacters.Replace(TEXT("{}"), TEXT(""));
	InvalidCharacters = InvalidCharacters.Replace(TEXT(":"), TEXT(""));

	return FName::IsValidXName(InText.ToString(), InvalidCharacters, &OutErrorMessage);
}

FReply SAssetNamingRow::DeleteRow()
{
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ProductionSettings->RemoveAssetNaming(ProductionSettings->GetActiveProductionID(), AssetNaming->Class);

	OnDeleteRow.ExecuteIfBound();

	return FReply::Handled();
}

FReply SAssetNamingRow::SummonEditMenu(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Create the context menu to be launched on right mouse click
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("RenameAction", "Rename"),
			LOCTEXT("RenameActionToolTip", "Change the default name for this asset type"),
			FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.AssetNaming"),
			FUIAction(FExecuteAction::CreateLambda([this]()
				{
					EditableTextBlock->EnterEditingMode();
				}
			)),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DeleteAction", "Delete"),
			LOCTEXT("DeleteActionToolTip", "Delete this default naming entry"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
			FUIAction(FExecuteAction::CreateLambda([this]() { DeleteRow(); } )),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

		FSlateApplication::Get().PushMenu(
			AsShared(),
			WidgetPath,
			MenuBuilder.MakeWidget(),
			MouseEvent.GetScreenSpacePosition(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
