// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "SNamingTokensEditableTextBox.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"

/** Data to display in a row in the asset naming list view */
struct FAssetNamingRowData
{
	/** The asset class being given a new default asset name (chosen using a class picker widget) */
	const UClass* Class;

	/** The new default asset name to register for the asset type */
	FString DefaultName;
};

DECLARE_DELEGATE(FOnDeleteRow);

/** Row widget for the asset naming list view */
class SAssetNamingRow : public SMultiColumnTableRow<TSharedPtr<FAssetNamingRowData>>
{
public:

	SLATE_BEGIN_ARGS(SAssetNamingRow) 
		: _OnDeleteRow()
		{}

		/** Called when a row is deleted from the list */
		SLATE_EVENT(FOnDeleteRow, OnDeleteRow)

	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedPtr<FAssetNamingRowData>& InRowData);

	/** Creates the widget for this row for the specified column */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

public:
	/** The text box where the user can type the default asset name */
	TSharedPtr<SNamingTokensEditableTextBox> EditableTextBlock;

private:
	/** Set the asset class for this row widget */
	void SetAssetClass(const UClass* SelectedClass);

	/** Set the default name for this row widget */
	void SetDefaultAssetName(const FText& InText, ETextCommit::Type InCommitType);

	/** Validate the text entered by the user to ensure it will be a valid asset name */
	bool ValidateDefaultAssetName(const FText& InText, FText& OutErrorMessage);

	/** Removes this row's default naming from the active production settings, and removes it from the list view */
	FReply DeleteRow();

	/** Summons a right-click context menu for the current row */
	FReply SummonEditMenu(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

private:
	/** The underlying asset naming data, used to properly display the class picker and default naming for that class */
	TSharedPtr<FAssetNamingRowData> AssetNaming;

	/** Delegate to execute when a row is deleted from the list */
	FOnDeleteRow OnDeleteRow;
};

/**
 * UI for the Asset Naming panel in the Production Wizard
 */
class SAssetNamingPanel : public SCompoundWidget
{
public:
	SAssetNamingPanel() = default;
	~SAssetNamingPanel();

	SLATE_BEGIN_ARGS(SAssetNamingPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	/** Generates a row displaying asset naming data */
	TSharedRef<ITableRow> OnGenerateAssetNamingRow(TSharedPtr<FAssetNamingRowData> InAssetNaming, const TSharedRef<STableViewBase>& OwnerTable);

	/** Handle double clicks on rows in the list view */
	void OnListViewDoubleClick(TSharedPtr<FAssetNamingRowData> InAssetNaming);

	/** Update the list view with the asset naming properties of the current active production */
	void UpdateAssetNamingList();

private:
	/** Source of default asset names for the active production */
	TArray<TSharedPtr<FAssetNamingRowData>> AssetNamingListItems;

	/** List view displaying the default asset names for the active production */
	TSharedPtr<SListView<TSharedPtr<FAssetNamingRowData>>> AssetNamingListView;

	/** Delegate bound to the Production Setting's OnActiveProductionChanged event */
	FDelegateHandle ActiveProductionChangedHandle;
};
