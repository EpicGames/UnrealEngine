// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFolderHierarchyPanel.h"

#include "Algo/Contains.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetViewUtils.h"
#include "CineAssemblyNamingTokens.h"
#include "CineAssemblyToolsAnalytics.h"
#include "CineAssemblyToolsStyle.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserItem.h"
#include "DirectoryPlaceholder.h"
#include "DirectoryPlaceholderUtils.h"
#include "Engine/Engine.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/FileManager.h"
#include "IContentBrowserDataModule.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/MessageDialog.h"
#include "NamingTokensEngineSubsystem.h"
#include "ObjectTools.h"
#include "ProductionSettings.h"
#include "Styling/StyleColors.h"
#include "UI/SActiveProductionCombo.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SFolderHierarchyPanel"

DEFINE_LOG_CATEGORY_STATIC(LogFolderHierarchyPanel, Log, All);

void SFolderHierarchyPanel::Construct(const FArguments& InArgs)
{
	// Subscribe to be notified when the Production Settings active productions has changed
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ActiveProductionChangedHandle = ProductionSettings->OnActiveProductionChanged().AddSP(this, &SFolderHierarchyPanel::BuildTreeFromProductionTemplate);

	// Create the tree root
	ContentRootItem = MakeShared<FTemplateFolderTreeItem>();
	ContentRootItem->Path.SetPathFromString(TEXT("/Game"), EContentBrowserPathType::Internal);
	ContentRootItem->FriendlyName = TEXT("Content");
	ContentRootItem->Status = ETemplateFolderStatus::Exists;
	ContentRootItem->bIsReadOnly = true;

	PluginRootItem = MakeShared<FTemplateFolderTreeItem>();
	PluginRootItem->Path.SetPathFromString(TEXT("/"), EContentBrowserPathType::Internal);
	PluginRootItem->FriendlyName = TEXT("Plugins");
	PluginRootItem->Status = ETemplateFolderStatus::Exists;
	PluginRootItem->bIsReadOnly = true;

	// Initialize the tree view using the active production's folder template
	BuildTreeFromProductionTemplate();

	TreeView = SNew(STreeView<TSharedPtr<FTemplateFolderTreeItem>>)
		.TreeItemsSource(&FolderItemsSource)
		.SelectionMode(ESelectionMode::Single)
		.OnGenerateRow(this, &SFolderHierarchyPanel::OnGenerateRow)
		.OnGetChildren(this, &SFolderHierarchyPanel::OnGetChildren)
		.OnItemsRebuilt(this, &SFolderHierarchyPanel::OnItemsRebuilt)
		.OnKeyDownHandler(this, &SFolderHierarchyPanel::OnKeyDown)
		.OnMouseButtonDoubleClick(this, &SFolderHierarchyPanel::OnTreeViewDoubleClick)
		.OnSetExpansionRecursive(this, &SFolderHierarchyPanel::SetExpansionRecursive)
		.OnContextMenuOpening(this, &SFolderHierarchyPanel::OnContextMenuOpening);

	// Start with the entire template tree expanded
	SetExpansionRecursive(ContentRootItem, true);
	SetExpansionRecursive(PluginRootItem, true);

	ChildSlot
		[
			SNew(SVerticalBox)

			// Active Production Selector
			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SActiveProductionCombo)
				]

			// Separator
			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator)
						.Orientation(Orient_Horizontal)
						.Thickness(2.0f)
				]

			// Folder Hierarchy Panel
			+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
						.Padding(16.0f)
						[
							SNew(SVerticalBox)

							// Title
							+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 4.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("FolderHierarchyTitle", "Production Settings"))
										.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.TitleFont"))
								]

							// Heading
							+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 4.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("FolderHierarchyHeading", "Folder Hierarchy"))
										.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.HeadingFont"))
								]

							// Info Text 1
							+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 8.0f)
								[
									SNew(STextBlock).Text(LOCTEXT("FolderHierarchyInfoText1", "Set up the folders you want to use for your production's assets.\n"
											"(Folders that already contain assets can only be deleted in the Content Browser.)"))
								]

							// Info Text 2
							+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 16.0f)
								[
									SNew(STextBlock).Text(LOCTEXT("FolderHierarchyInfoText2", "Click Create Template Folders when you are done."))
								]

							// Main Panel Content
							+ SVerticalBox::Slot()
								[
									// This entire box will be disabled when there is no active production
									SNew(SVerticalBox)
										.IsEnabled_Lambda([]() -> bool
											{
												const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
												TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
												return ActiveProduction.IsSet();
											}
										)

									// Add / Reset Buttons
									+ SVerticalBox::Slot()
										.AutoHeight()
										.Padding(0.0f, 0.0f, 0.0f, 8.0f)
										[
											SNew(SHorizontalBox)

											// Add Folder To Template Button
											+ SHorizontalBox::Slot()
												.AutoWidth()
												[
													SNew(SButton)
														.ContentPadding(FMargin(2.0f))
														.OnClicked(this, &SFolderHierarchyPanel::OnAddFolderToTemplate)
														[
															SNew(SHorizontalBox)

															+ SHorizontalBox::Slot()
																.AutoWidth()
																.Padding(0.0f, 0.0f, 4.0f, 0.0f)
																[
																	SNew(SImage)
																		.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
																		.ColorAndOpacity(FStyleColors::AccentGreen)
																]

															+ SHorizontalBox::Slot()
																.AutoWidth()
																[
																	SNew(STextBlock).Text(LOCTEXT("AddFolderToTemplateButton", "Add Folder To Template"))
																]
														]
												]

											// Reset Template Changes Button
											+ SHorizontalBox::Slot()
												.AutoWidth()
												[
													SNew(SButton)
														.ContentPadding(FMargin(2.0f))
														.OnClicked(this, &SFolderHierarchyPanel::OnResetTemplateChanges)
														[
															SNew(SHorizontalBox)

															+ SHorizontalBox::Slot()
																.AutoWidth()
																.Padding(0.0f, 0.0f, 4.0f, 0.0f)
																[
																	SNew(SImage).Image(FAppStyle::Get().GetBrush("PropertyWindow.DiffersFromDefault"))
																]

															+ SHorizontalBox::Slot()
																.AutoWidth()
																[
																	SNew(STextBlock).Text(LOCTEXT("OnResetTemplateChangesButton", "Reset Template Changes"))
																]
														]
												]
										]

									// Tree View
									+ SVerticalBox::Slot()
										.FillHeight(1.0f)
										.Padding(0.0f, 0.0f, 0.0f, 8.0f)
										[
											TreeView.ToSharedRef()
										]

									// Apply Changes Button
									+ SVerticalBox::Slot()
										.AutoHeight()
										.HAlign(HAlign_Right)
										.Padding(0.0f, 0.0f, 0.0f, 8.0f)
										[
											SNew(SButton)
												.ContentPadding(FMargin(2.0f))
												.OnClicked(this, &SFolderHierarchyPanel::OnCreateTemplateFolders)
												[
													SNew(SHorizontalBox)

													+ SHorizontalBox::Slot()
														.AutoWidth()
														.Padding(0.0f, 0.0f, 4.0f, 0.0f)
														[
															SNew(SImage).Image(FCineAssemblyToolsStyle::Get().GetBrush("Icons.Initialize"))
														]

													+ SHorizontalBox::Slot()
														.AutoWidth()
														[
															SNew(STextBlock).Text(LOCTEXT("CreateTemplateFoldersButton", "Create Template Folders"))
														]
												]
										]
								]
						]
				]
		];
}

SFolderHierarchyPanel::~SFolderHierarchyPanel()
{
	if (UObjectInitialized())
	{
		if (UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>())
		{
			ProductionSettings->OnActiveProductionChanged().Remove(ActiveProductionChangedHandle);
		}
	}
}

TSharedRef<ITableRow> SFolderHierarchyPanel::OnGenerateRow(TSharedPtr<FTemplateFolderTreeItem> TreeItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SLayeredImage> FolderIcon = SNew(SLayeredImage)
		.ColorAndOpacity(FAppStyle::Get().GetSlateColor("ContentBrowser.DefaultFolderColor"))
		.Image(FCineAssemblyToolsStyle::Get().GetBrush("Icons.Folder"));

	FolderIcon->AddLayer(TAttribute<const FSlateBrush*>::CreateSP(this, &SFolderHierarchyPanel::GetFolderIconBadge, TreeItem));

	FNamingTokenFilterArgs FilterArgs;
	FilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

	return SNew(STableRow<TSharedPtr<FTemplateFolderTreeItem>>, OwnerTable)
		.ShowSelection(true)
		.Padding(FMargin(8.0f, 0.0f, 8.0f, 0.0f))
 		.OnCanAcceptDrop_Lambda([this](const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<FTemplateFolderTreeItem> InItem)
 			{
 				return (InItem != PluginRootItem) ? EItemDropZone::OntoItem : TOptional<EItemDropZone>();
			})
		.OnAcceptDrop(this, &SFolderHierarchyPanel::OnTreeRowAcceptDrop)
		.OnDragDetected(this, &SFolderHierarchyPanel::OnTreeRowDragDetected)
		.Content()
		[
			SNew(SHorizontalBox)

			// Folder Icon
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 2.0f, 0.0f)
				[
					FolderIcon
				]

			// Folder Name TextBlock
			+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SAssignNew(TreeItem->NameWidget, SNamingTokensEditableTextBox)
						.SupportInlineEdit(true)
						.ShouldEvaluateTokens(false)
						.FilterArgs(FilterArgs)
						.TextBoxPadding(FMargin(4, 2, 0, 2))
						.DisplayTokenIcon(false)
						.DisplayBorderImage(false)
						.IsReadOnly(TreeItem->bIsReadOnly)
						.DisplayErrorMessage(!TreeItem->bIsReadOnly)
						.Text_Lambda([TreeItem]() -> FText
							{
								const FString ItemName = !TreeItem->FriendlyName.IsEmpty() ? TreeItem->FriendlyName : FPaths::GetPathLeaf(TreeItem->Path.GetInternalPathString());
								return FText::FromString(ItemName);
							})
						.OnValidateTokenizedText(this, &SFolderHierarchyPanel::IsValidFolderName, TreeItem)
						.OnTextCommitted(this, &SFolderHierarchyPanel::SetTemplateFolderName, TreeItem)
				]

			// Folder Status Text
			+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(this, &SFolderHierarchyPanel::GetFolderStatusText, TreeItem)
						.ColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.4f))
				]
		];
}

void SFolderHierarchyPanel::OnGetChildren(TSharedPtr<FTemplateFolderTreeItem> TreeItem, TArray<TSharedPtr<FTemplateFolderTreeItem>>& OutNodes)
{
	OutNodes = TreeItem->Children;
}

void SFolderHierarchyPanel::OnItemsRebuilt()
{
	// Upon regenerating the tree view, allow the user to immediately interact with the name widget of the newly added template folder in order to rename it
	if (MostRecentlyAddedItem && MostRecentlyAddedItem->NameWidget)
	{
		MostRecentlyAddedItem->NameWidget->EnterEditingMode();
		MostRecentlyAddedItem.Reset();
	}
}

void SFolderHierarchyPanel::SetExpansionRecursive(TSharedPtr<FTemplateFolderTreeItem> TreeItem, bool bInExpand) const
{
	TreeView->SetItemExpansion(TreeItem, bInExpand);

	for (const TSharedPtr<FTemplateFolderTreeItem>& ChildItem : TreeItem->Children)
	{
		SetExpansionRecursive(ChildItem, bInExpand);
	}
}

TSharedPtr<SWidget> SFolderHierarchyPanel::OnContextMenuOpening()
{
	// The TreeView uses single selection mode, so at most one item can ever be selected by the user
	TArray<TSharedPtr<FTemplateFolderTreeItem>> SelectedNodes = TreeView->GetSelectedItems();
	if (SelectedNodes.Num() == 1)
	{
		constexpr bool bCloseAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bCloseAfterMenuSelection, nullptr);

		TSharedPtr<FTemplateFolderTreeItem>& SelectedTreeItem = SelectedNodes[0];

		// If the selected template folder does not currently exist in the Content Browser, give the user the option to toggle its status between "Create" and "Do Not Create"
		if (SelectedTreeItem->Status == ETemplateFolderStatus::MissingCreate)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("MarkAsDoNotCreate", "Mark as Do Not Create"),
				LOCTEXT("MarkAsDoNotCreateToolTip", "Mark this folder so that it will not be created when clicking Create Template Folders"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.StatusIcon.Off"),
				FUIAction(FExecuteAction::CreateSP(this, &SFolderHierarchyPanel::SetTemplateFolderStatusRecursive, SelectedTreeItem, ETemplateFolderStatus::MissingDoNotCreate)),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		else if (SelectedTreeItem->Status == ETemplateFolderStatus::MissingDoNotCreate)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("MarkAsCreate", "Mark as Create"),
				LOCTEXT("MarkAsCreateToolTip", "Mark this folder so that it will be created when clicking Create Template Folders"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.AddCircle"),
				FUIAction(FExecuteAction::CreateSP(this, &SFolderHierarchyPanel::SetTemplateFolderStatusRecursive, SelectedTreeItem, ETemplateFolderStatus::MissingCreate)),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}

		// Option to add a new child folder
		if (SelectedTreeItem != PluginRootItem)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("AddChildFolderAction", "Add Child Folder"),
				LOCTEXT("AddChildFolderTooltip", "Add child folder"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"),
				FUIAction(FExecuteAction::CreateLambda([this]() { OnAddFolderToTemplate(); })),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}

		if (!SelectedTreeItem->bIsReadOnly)
		{
			// Option to rename the template folder (and update the path of the children below it)
			MenuBuilder.AddMenuEntry(
				LOCTEXT("RenameAction", "Rename"),
				LOCTEXT("RenameActionToolTip", "Rename template folder"),
				FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.AssetNaming"),
				FUIAction(
					FExecuteAction::CreateLambda([SelectedTreeItem]()
						{
							SelectedTreeItem->NameWidget->EnterEditingMode();
						})),
				NAME_None,
				EUserInterfaceActionType::Button
			);

			// Option to delete the template folder and all of its children (recursively)
			MenuBuilder.AddMenuEntry(
				LOCTEXT("DeleteAction", "Delete"),
				LOCTEXT("DeleteActionToolTip", "Delete this folder"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
				FUIAction(FExecuteAction::CreateSP(this, &SFolderHierarchyPanel::DeleteTemplateFolder, SelectedTreeItem)),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}

		return MenuBuilder.MakeWidget();
	}

	return SNullWidget::NullWidget;
}

FReply SFolderHierarchyPanel::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// The TreeView uses single selection mode, so at most one item can ever be selected by the user
	TArray<TSharedPtr<FTemplateFolderTreeItem>> SelectedNodes = TreeView->GetSelectedItems();
	if (SelectedNodes.Num() == 1)
	{
		TSharedPtr<FTemplateFolderTreeItem>& SelectedTreeItem = SelectedNodes[0];

		if ((InKeyEvent.GetKey() == EKeys::Delete) && (!SelectedTreeItem->bIsReadOnly))
		{
			DeleteTemplateFolder(SelectedTreeItem);
		}
	}

	return FReply::Handled();
}

void SFolderHierarchyPanel::OnTreeViewDoubleClick(TSharedPtr<FTemplateFolderTreeItem> TreeItem)
{
	// Put the textblock for the input tree item into edit mode so the user can rename the item
	if (TreeItem && TreeItem->NameWidget)
	{
		TreeItem->NameWidget->EnterEditingMode();
	}
}

FReply SFolderHierarchyPanel::OnTreeRowDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		TArray<TSharedPtr<FTemplateFolderTreeItem>> SelectedTreeItems = TreeView->GetSelectedItems();
		if (SelectedTreeItems.Num() == 1)
		{
			TSharedPtr<FTemplateFolderTreeItem> SelectedTreeItem = SelectedTreeItems[0];

			// Only editable tree rows can be dragged
			if (SelectedTreeItem->bIsReadOnly)
			{
				return FReply::Handled();
			}

			const FString SelectedFolderName = FPaths::GetPathLeaf(SelectedTreeItem->Path.GetInternalPathString());
			TSharedRef<FTemplateFolderDragDrop> Operation = FTemplateFolderDragDrop::New(SelectedFolderName);
			Operation->SourceTreeItem = SelectedTreeItem;

 			Operation->OnDropNotHandled.BindLambda([this, SelectedTreeItem]()
 				{
					// Return the dragged folder to its original location in the tree
					SelectedTreeItem->Parent->Children.Add(SelectedTreeItem);
					SetExpansionRecursive(SelectedTreeItem, true);
					TreeView->RequestTreeRefresh();
 				});

			// Remove this item from the tree while it is being dragged
			SelectedTreeItem->Parent->Children.Remove(SelectedTreeItem);

			TreeView->RequestTreeRefresh();
			return FReply::Handled().BeginDragDrop(Operation);
		}
	}

	return FReply::Unhandled();
}

FReply SFolderHierarchyPanel::OnTreeRowAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<FTemplateFolderTreeItem> InItem)
{
	TSharedPtr<FTemplateFolderDragDrop> Operation = InDragDropEvent.GetOperationAs<FTemplateFolderDragDrop>();
	if (Operation.IsValid())
	{
		// If an item with the same name already exists in the drop location, do not handle the drop (the dragged item will be reset to its original location)
		if (Algo::ContainsBy(InItem->Children, FContentBrowserItemPath(InItem->Path.GetInternalPathString() / Operation->ItemName, EContentBrowserPathType::Internal), &FTemplateFolderTreeItem::Path))
		{
			return FReply::Unhandled();
		}

		// Add the dropped item as a child to the folder it was dropped on
		InItem->Children.Add(Operation->SourceTreeItem);
		Operation->SourceTreeItem->Parent = InItem;

		// Update the paths of everything in the subtree to be relative to the new parent folder
		SetChildrenPathRecursive(InItem, InItem->Path.GetInternalPathString());

		InItem->Children.Sort([](const TSharedPtr<FTemplateFolderTreeItem>& A, const TSharedPtr<FTemplateFolderTreeItem>& B) { return A->Path.GetInternalPathString() < B->Path.GetInternalPathString(); });

		SetExpansionRecursive(InItem, true);
		TreeView->RequestTreeRefresh();
	}

	return FReply::Handled();
}

void SFolderHierarchyPanel::BuildTreeFromProductionTemplate()
{
	FolderItemsSource.Empty();
	ContentRootItem->Children.Empty();
	PluginRootItem->Children.Empty();

	FolderItemsSource.Add(ContentRootItem);

	TArray<TSharedRef<IPlugin>> EnabledPlugins = IPluginManager::Get().GetEnabledPluginsWithContent();

	for (const TSharedRef<IPlugin>& Plugin : EnabledPlugins)
	{
		if (Plugin.Get().GetType() != EPluginType::Engine)
		{
			FString PluginContentPath = Plugin.Get().GetMountedAssetPath();
			PluginContentPath.RemoveFromEnd(TEXT("/"));

			TSharedPtr<FTemplateFolderTreeItem> PluginItem = MakeShared<FTemplateFolderTreeItem>();
			PluginItem->Path.SetPathFromString(PluginContentPath, EContentBrowserPathType::Internal);
			PluginItem->FriendlyName = Plugin.Get().GetFriendlyName();
			PluginItem->Status = ETemplateFolderStatus::Exists;
			PluginItem->bIsReadOnly = true;

			PluginRootItem->Children.Add(PluginItem);
		}
	}

	// Don't add the plugin root folder to the tree if there are no actual plugins enabled
	if (!PluginRootItem->Children.IsEmpty())
	{
		FolderItemsSource.Add(PluginRootItem);
	}

	PluginRootItem->Children.Sort([](const TSharedPtr<FTemplateFolderTreeItem>& A, const TSharedPtr<FTemplateFolderTreeItem>& B) { return A->FriendlyName < B->FriendlyName; });

	const UProductionSettings* const ProductionSettings = GetMutableDefault<UProductionSettings>();
	TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
	if (ActiveProduction.IsSet())
	{
		// Sort the folder paths from the active production's template alphabetically to put them in the order they should appear in the tree
		TArray<FFolderTemplate> ProductionTemplate = ActiveProduction.GetValue().TemplateFolders;
		CachedInitialState = ProductionTemplate;

		ProductionTemplate.Sort([](const FFolderTemplate& A, const FFolderTemplate& B) { return A.InternalPath < B.InternalPath; });

		for (const FFolderTemplate& FolderTemplate : ProductionTemplate)
		{
			const FString ParentPath = FPaths::GetPath(FolderTemplate.InternalPath);

			// Walk the tree until we find an item whose path matches the parent path. The new tree item will be created as one of its children
			// If no parent is found, we do not add the template folder to the tree view because it is considered ill-formed
			if (TSharedPtr<FTemplateFolderTreeItem> ParentItem = FindItemAtPath(ParentPath))
			{
				TSharedPtr<FTemplateFolderTreeItem> NewItem = MakeShared<FTemplateFolderTreeItem>();
				NewItem->Path.SetPathFromString(FolderTemplate.InternalPath, EContentBrowserPathType::Internal);

				if (DoesPathExist(NewItem->Path.GetInternalPathString()))
				{
					NewItem->Status = ETemplateFolderStatus::Exists;
				}
				else
				{
					NewItem->Status = FolderTemplate.bCreateIfMissing ? ETemplateFolderStatus::MissingCreate : ETemplateFolderStatus::MissingDoNotCreate;
				}

				ParentItem->Children.Add(NewItem);
				NewItem->Parent = ParentItem;
			}
		}
	}

	if (TreeView)
	{
		TreeView->RequestTreeRefresh();
		SetExpansionRecursive(ContentRootItem, true);
		SetExpansionRecursive(PluginRootItem, true);
	}
}

FReply SFolderHierarchyPanel::OnAddFolderToTemplate()
{
	// Get the parent item for the new template folder being added (this can be the root folder if no parent is currently selected)
	// The TreeView uses single selection mode, so at most one item can ever be selected by the user
	TSharedPtr<FTemplateFolderTreeItem> SelectedTreeItem = ContentRootItem;

	TArray<TSharedPtr<FTemplateFolderTreeItem>> SelectedNodes = TreeView->GetSelectedItems();
	if (SelectedNodes.Num() == 1)
	{
		SelectedTreeItem = SelectedNodes[0];
	}

	TSharedPtr<FTemplateFolderTreeItem> NewFolder = MakeShared<FTemplateFolderTreeItem>();

	const FString ParentPath = SelectedTreeItem->Path.GetInternalPathString();
	const FString NewFolderName = CreateUniqueFolderName(SelectedTreeItem->Path);
	NewFolder->Path.SetPathFromString(ParentPath / NewFolderName, EContentBrowserPathType::Internal);

	if (DoesPathExist(NewFolder->Path.GetInternalPathString()))
	{
		NewFolder->Status = ETemplateFolderStatus::Exists;
	}
	else if (SelectedTreeItem->Status == ETemplateFolderStatus::MissingDoNotCreate)
	{
		NewFolder->Status = ETemplateFolderStatus::MissingDoNotCreate;
	}
	else
	{
		NewFolder->Status = ETemplateFolderStatus::MissingCreate;
	}

	SelectedTreeItem->Children.Add(NewFolder);
	NewFolder->Parent = SelectedTreeItem;

	MostRecentlyAddedItem = NewFolder;

	// Sort the children alphabetically to maintain a good ordering with the new folder
	SelectedTreeItem->Children.Sort([](const TSharedPtr<FTemplateFolderTreeItem>& A, const TSharedPtr<FTemplateFolderTreeItem>& B) { return A->Path.GetInternalPathString() < B->Path.GetInternalPathString(); });

	// Add the new folder to the active production's template
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();

	const bool bCreateIfMissing = (NewFolder->Status == ETemplateFolderStatus::MissingDoNotCreate) ? false : true;
	ProductionSettings->AddTemplateFolder(ProductionSettings->GetActiveProductionID(), NewFolder->Path.GetInternalPathString(), bCreateIfMissing);

	TreeView->SetItemExpansion(SelectedTreeItem, true);
	TreeView->RequestTreeRefresh();

	return FReply::Handled();
}

FReply SFolderHierarchyPanel::OnCreateTemplateFolders()
{
	// Walk the tree and create all template folders in the Content Browser that are marked as MissingCreate
	CreateFolderFromTemplateRecursive(ContentRootItem);

	for (const TSharedPtr<FTemplateFolderTreeItem>& PluginItem : PluginRootItem->Children)
	{
		CreateFolderFromTemplateRecursive(PluginItem);
	}

	const UProductionSettings* const ProductionSettings = GetMutableDefault<UProductionSettings>();
	TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();

	if (ActiveProduction.IsSet())
	{
		CachedInitialState = ActiveProduction.GetValue().TemplateFolders;
	}

	TreeView->RequestTreeRefresh();
	SetExpansionRecursive(ContentRootItem, true);
	SetExpansionRecursive(PluginRootItem, true);

	UE::CineAssemblyToolsAnalytics::RecordEvent_ProductionCreateTemplateFolders();

	return FReply::Handled();
}

FReply SFolderHierarchyPanel::OnResetTemplateChanges()
{
	// Reset the active production's template to the latest cached state, then rebuild the tree view
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ProductionSettings->SetTemplateFolderHierarchy(ProductionSettings->GetActiveProductionID(), CachedInitialState);

	BuildTreeFromProductionTemplate();

	return FReply::Handled();
}

bool SFolderHierarchyPanel::IsValidFolderName(const FText& InText, FText& OutErrorMessage, TSharedPtr<FTemplateFolderTreeItem> TreeItem)
{
	// Check for empty text string
	if (InText.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("EmptyNameErrorMessage", "Please provide a name for this folder");
		return false;
	}

	// Check for invalid characters
	FString InvalidCharacters = INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS;

	// These characters are actually valid, because we want to support naming tokens
	InvalidCharacters = InvalidCharacters.Replace(TEXT("{}"), TEXT(""));
	InvalidCharacters = InvalidCharacters.Replace(TEXT(":"), TEXT(""));

	if (!FName::IsValidXName(InText.ToString(), InvalidCharacters, &OutErrorMessage))
	{
		return false;
	}

	// Try to heuristically determine if this will result in a valid path.
	// If the path contains any tokens, then we don't actually know what its final length will be, but we try our best to warn the user against egregious mistakes.
	const FString ParentPath = FPaths::GetPath(TreeItem->Path.GetInternalPathString());
	const FString ProposedPath = ParentPath / InText.ToString();

	FString ProposedPathForValidation = ProposedPath;
	ProposedPathForValidation.ReplaceInline(TEXT("{"), TEXT(""));
	ProposedPathForValidation.ReplaceInline(TEXT("}"), TEXT(""));
	ProposedPathForValidation.ReplaceInline(TEXT(":"), TEXT(""));

	if (!AssetViewUtils::IsValidPackageForCooking(ProposedPathForValidation, OutErrorMessage))
	{
		return false;
	}

	// Check for duplicate folder names
	TSharedPtr<FTemplateFolderTreeItem> FoundItem = FindItemAtPath(ProposedPath);

	if (FoundItem && FoundItem != TreeItem)
	{
		OutErrorMessage = LOCTEXT("DuplicateNameErrorMessage", "A folder already exists at this location with this name");
		return false;
	}

	return true;
}

void SFolderHierarchyPanel::SetTemplateFolderName(const FText& InText, ETextCommit::Type InCommitType, TSharedPtr<FTemplateFolderTreeItem> TreeItem)
{
	// Guards against re-entrant SetTemplateFolderName when FMessageDialog::Open's nested message pump dispatches a second OnTextCommitted.
	if (bIsSettingTemplateFolderName)
	{
		return;
	}

	TGuardValue<bool> ReentryGuard(bIsSettingTemplateFolderName, true);

	// Early-out if the folder is read-only, or if the name has not actually changed
	if (TreeItem->bIsReadOnly)
	{
		return;
	}

	const FString OldPath = TreeItem->Path.GetInternalPathString();
	const FString OldFolderName = FPaths::GetPathLeaf(OldPath);
	if (OldFolderName.Equals(InText.ToString()))
	{
		return;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");

	const FString NewPath = FPaths::GetPath(OldPath) / InText.ToString();

	// If the folder being renamed exists in the content browser, and it is empty, prompt the user with a pop-up dialog asking if they want to rename the Content Browser folder
	if (TreeItem->Status == ETemplateFolderStatus::Exists && IsFolderEmpty(TreeItem))
	{
		const FText DialogMessage = LOCTEXT("RenameDialogMessage", "An empty folder matching this template path already exists in the Content Browser. Do you want to rename that empty folder?");
		EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNoCancel, DialogMessage);

		// Rename the existing Content Browser folder
		if (Response == EAppReturnType::Yes)
		{
			const FString NewRelativeFilePath = FPackageName::LongPackageNameToFilename(NewPath);
			const FString NewAbsoluteFilePath = FPaths::ConvertRelativePathToFull(NewRelativeFilePath);

			// Create the directory on disk, if it doesn't already exist
			if (!IFileManager::Get().DirectoryExists(*NewAbsoluteFilePath))
			{
				constexpr bool bCreateParentFoldersIfMissing = false;
				IFileManager::Get().MakeDirectory(*NewAbsoluteFilePath, bCreateParentFoldersIfMissing);
			}

			AssetRegistryModule.Get().AddPath(NewPath);

			// If successful, this will move the contents (which at most will consist of directory placeholders) into the new folder and then delete the old folder
			AssetViewUtils::RenameFolder(NewPath, OldPath);

			TreeItem->Status = ETemplateFolderStatus::Exists;
		}
		else if (Response == EAppReturnType::No)
		{
			TreeItem->Status = ETemplateFolderStatus::MissingCreate;
		}
		else if (Response == EAppReturnType::Cancel)
		{
			return;
		}
	}

	// Remove the old path from the active production's template, and add the new path instead
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();

	ProductionSettings->RemoveTemplateFolder(ProductionSettings->GetActiveProductionID(), OldPath);

	const bool bCreateIfMissing = (TreeItem->Status == ETemplateFolderStatus::MissingDoNotCreate) ? false : true;
	ProductionSettings->AddTemplateFolder(ProductionSettings->GetActiveProductionID(), NewPath, bCreateIfMissing);

	// Update the name input tree item and the path of all of its children (recursively)
	TreeItem->Path.SetPathFromString(NewPath, EContentBrowserPathType::Internal);
	SetChildrenPathRecursive(TreeItem, NewPath);

	// Now that the path has changed, check again if this item exists, and update the status accordingly
	if (DoesPathExist(NewPath))
	{
		TreeItem->Status = ETemplateFolderStatus::Exists;
	}

	// Sort the items at the same level in the tree as the input item to maintain a good ordering with the new folder name
	TreeItem->Parent->Children.Sort([](const TSharedPtr<FTemplateFolderTreeItem>& A, const TSharedPtr<FTemplateFolderTreeItem>& B) { return A->Path.GetInternalPathString() < B->Path.GetInternalPathString(); });

	TreeView->RequestTreeRefresh();
}

void SFolderHierarchyPanel::DeleteTemplateFolder(TSharedPtr<FTemplateFolderTreeItem> TreeItem)
{
	// Before removing any folders from the template, the user is prompted with a pop-up dialog confirming the action.

	// If the folder being deleted exists in the content browser, and it is empty, the user is also prompted asking if they want to delete the Content Browser folder
	if (TreeItem->Status == ETemplateFolderStatus::Exists && IsFolderEmpty(TreeItem))
	{
		const FText DialogMessage = LOCTEXT("DeleteExistingFolderDialogMessage", "An empty folder matching this template path exists in the Content Browser. Do you want to delete that empty folder?");
		EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNoCancel, DialogMessage);

		if (Response == EAppReturnType::Yes)
		{
			// Resolve tokens before deleting
			const FString ResolvedPath = ResolveTemplatePath(TreeItem->Path.GetInternalPathString());
			UDirectoryPlaceholderLibrary::DeletePlaceholdersInPath(ResolvedPath);
			AssetViewUtils::DeleteFolders({ ResolvedPath });
		}
		else if (Response == EAppReturnType::Cancel)
		{
			return;
		}
	}
	else
	{
		const FText DialogMessage = LOCTEXT("DeleteTemplateDialogMessage", "Are you sure you want to delete this folder from the template?");
		EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNo, DialogMessage);

		if ((Response == EAppReturnType::No) || (Response == EAppReturnType::Cancel))
		{
			return;
		}
	}

	// Recursively remove this item and all of its children from the active production's template and from the tree view
	RemoveFolderFromTemplateRecursive(TreeItem);

	// Now remove this item from its parent's list of children to finish cleaning up
	TreeItem->Parent->Children.Remove(TreeItem);

	TreeView->RequestTreeRefresh();
}

TSharedPtr<FTemplateFolderTreeItem> SFolderHierarchyPanel::FindItemAtPathRecursive(TSharedPtr<FTemplateFolderTreeItem> TreeItem, const FString& Path)
{
	if (TreeItem->Path.GetInternalPathString().Equals(Path))
	{
		return TreeItem;
	}

	for (const TSharedPtr<FTemplateFolderTreeItem>& Child : TreeItem->Children)
	{
		if (const TSharedPtr<FTemplateFolderTreeItem>& ItemAtPath = FindItemAtPathRecursive(Child, Path))
		{
			return ItemAtPath;
		}
	}

	return nullptr;
}

TSharedPtr<FTemplateFolderTreeItem> SFolderHierarchyPanel::FindItemAtPath(const FString& Path)
{
	// Check for the item within the Content tree
	if (TSharedPtr<FTemplateFolderTreeItem> FoundItem = FindItemAtPathRecursive(ContentRootItem, Path))
	{
		return FoundItem;
	}

	// Check for the item with the Plugins tree
	for (const TSharedPtr<FTemplateFolderTreeItem>& PluginItem : PluginRootItem->Children)
	{
		if (TSharedPtr<FTemplateFolderTreeItem> FoundItem = FindItemAtPathRecursive(PluginItem, Path))
		{
			return FoundItem;
		}
	}

	return nullptr;
}

void SFolderHierarchyPanel::CreateFolderFromTemplateRecursive(TSharedPtr<FTemplateFolderTreeItem> TreeItem)
{
	if (TreeItem->Status == ETemplateFolderStatus::MissingCreate)
	{
		// Sanitize the name, in case any tokens were not able to be resolved
		const FString ResolvedPath = ResolveTemplatePath(TreeItem->Path.GetInternalPathString());
		const FString SanitizedPath = ObjectTools::SanitizeInvalidChars(ResolvedPath, INVALID_LONGPACKAGE_CHARACTERS);

		// Validate the path before attempting to create on disk.
		FText InvalidPackageError;
		if (!AssetViewUtils::IsValidPackageForCooking(SanitizedPath, InvalidPackageError))
		{
			UE_LOGF(LogFolderHierarchyPanel, Warning, "Cannot create template folder '%ls': %ls", *SanitizedPath, *InvalidPackageError.ToString());
		}
		else
		{
			const FString RelativeFilePath = FPackageName::LongPackageNameToFilename(SanitizedPath);
			const FString AbsoluteFilePath = FPaths::ConvertRelativePathToFull(RelativeFilePath);

			if (IFileManager::Get().DirectoryExists(*AbsoluteFilePath))
			{
				TreeItem->Status = ETemplateFolderStatus::Exists;
			}
			else
			{
				constexpr bool bCreateParentFoldersIfMissing = false;
				if (IFileManager::Get().MakeDirectory(*AbsoluteFilePath, bCreateParentFoldersIfMissing))
				{
					FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
					AssetRegistryModule.Get().AddPath(SanitizedPath);
					TreeItem->Status = ETemplateFolderStatus::Exists;
				}
				else
				{
					UE_LOGF(LogFolderHierarchyPanel, Warning, "Failed to create template folder '%ls'", *AbsoluteFilePath);
				}
			}
		}
	}

	// If we are not going to create this folder, we do not need to check any of its children
	if (TreeItem->Status != ETemplateFolderStatus::MissingDoNotCreate)
	{
		for (TSharedPtr<FTemplateFolderTreeItem>& Child : TreeItem->Children)
		{
			CreateFolderFromTemplateRecursive(Child);
		}
	}
}

void SFolderHierarchyPanel::RemoveFolderFromTemplateRecursive(TSharedPtr<FTemplateFolderTreeItem> TreeItem)
{
	for (TSharedPtr<FTemplateFolderTreeItem>& Child : TreeItem->Children)
	{
		RemoveFolderFromTemplateRecursive(Child);
	}

	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ProductionSettings->RemoveTemplateFolder(ProductionSettings->GetActiveProductionID(), TreeItem->Path.GetInternalPathString());

	TreeItem->Children.Empty();
}

void SFolderHierarchyPanel::SetChildrenPathRecursive(TSharedPtr<FTemplateFolderTreeItem> TreeItem, const FString& NewPath)
{
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();

	for (TSharedPtr<FTemplateFolderTreeItem>& Child : TreeItem->Children)
	{
		const FString OldChildPath = Child->Path.GetInternalPathString();
		const FString OldChildFolderName = FPaths::GetPathLeaf(OldChildPath);

		// Update the child path with the new path of the parent
		const FString NewChildPath = NewPath / OldChildFolderName;
		Child->Path.SetPathFromString(NewChildPath, EContentBrowserPathType::Internal);

		// Now that the path has changed, check again if this item exists, and update the status accordingly
		if (DoesPathExist(NewChildPath))
		{
			Child->Status = ETemplateFolderStatus::Exists;
		}
		else if (TreeItem->Status == ETemplateFolderStatus::MissingDoNotCreate)
		{
			Child->Status = ETemplateFolderStatus::MissingDoNotCreate;
		}
		else
		{
			Child->Status = ETemplateFolderStatus::MissingCreate;
		}

		// Remove the old path from the active production's template, and add the new path instead
		ProductionSettings->RemoveTemplateFolder(ProductionSettings->GetActiveProductionID(), OldChildPath);

		const bool bCreateIfMissing = (Child->Status == ETemplateFolderStatus::MissingDoNotCreate) ? false : true;
		ProductionSettings->AddTemplateFolder(ProductionSettings->GetActiveProductionID(), NewChildPath, bCreateIfMissing);

		SetChildrenPathRecursive(Child, NewChildPath);
	}
}

void SFolderHierarchyPanel::SetTemplateFolderStatusRecursive(TSharedPtr<FTemplateFolderTreeItem> TreeItem, ETemplateFolderStatus NewStatus)
{
	TreeItem->Status = NewStatus;

	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();

	const FString Path = TreeItem->Path.GetInternalPathString();
	ProductionSettings->RemoveTemplateFolder(ProductionSettings->GetActiveProductionID(), Path);

	const bool bCreateIfMissing = (TreeItem->Status == ETemplateFolderStatus::MissingDoNotCreate) ? false : true;
	ProductionSettings->AddTemplateFolder(ProductionSettings->GetActiveProductionID(), Path, bCreateIfMissing);

	for (TSharedPtr<FTemplateFolderTreeItem>& Child : TreeItem->Children)
	{
		SetTemplateFolderStatusRecursive(Child, NewStatus);
	}
}

FString SFolderHierarchyPanel::CreateUniqueFolderName(FContentBrowserItemPath InPath)
{
	// This method for creating a unique folder name is adapted from the implementation used by the Content Browser to achieve a similar convention
	const FText DefaultFolderBaseName = LOCTEXT("DefaultFolderName", "NewFolder");
	UContentBrowserDataSubsystem* ContentBrowserDataSubsystem = IContentBrowserDataModule::Get().GetSubsystem();

	const FString DefaultFolderName = DefaultFolderBaseName.ToString();
	int32 NewFolderPostfix = 0;
	FName CombinedPathName;
	for (;;)
	{
		FString CombinedPathNameStr = InPath.GetVirtualPathString() / DefaultFolderName;
		if (NewFolderPostfix > 0)
		{
			CombinedPathNameStr.AppendInt(NewFolderPostfix);
		}
		++NewFolderPostfix;

		CombinedPathName = *CombinedPathNameStr;

		// Check if the path matches a folder that already exists in the content browser
		const FContentBrowserItem ExistingFolder = ContentBrowserDataSubsystem->GetItemAtPath(CombinedPathName, EContentBrowserItemTypeFilter::IncludeFolders);

		// The path used by the Content Browser subsystem function was a virtual path, but our function operates on internal paths, so we have to strip the beginning off the front
		CombinedPathNameStr.RemoveFromStart(TEXT("/All/Plugins"));
		CombinedPathNameStr.RemoveFromStart(TEXT("/All"));

		// Check if the path matches a folder in our template
		TSharedPtr<FTemplateFolderTreeItem> Item = FindItemAtPath(CombinedPathNameStr);

		if (!ExistingFolder.IsValid() && !Item.IsValid())
		{
			break;
		}
	}

	return FPaths::GetPathLeaf(CombinedPathName.ToString());
}

bool SFolderHierarchyPanel::IsFolderEmpty(TSharedPtr<FTemplateFolderTreeItem> TreeItem) const
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Resolve tokens in the template path
	const FString ResolvedPath = ResolveTemplatePath(TreeItem->Path.GetInternalPathString());

	FARFilter Filter;
	Filter.bRecursivePaths = true;
	Filter.PackagePaths.Add(FName(*ResolvedPath));

	// Find all of the assets (recursively) in the input folder
	TArray<FAssetData> AssetDataList;
	AssetRegistryModule.Get().GetAssets(Filter, AssetDataList);

	// Check if all of the assets found are placeholders
	bool bIsEmpty = true;
	for (const FAssetData& AssetData : AssetDataList)
	{
		if (AssetData.GetClass() != UDirectoryPlaceholder::StaticClass())
		{
			bIsEmpty = false;
		}
	}

	return bIsEmpty;
}

bool SFolderHierarchyPanel::DoesPathExist(const FString& Path) const
{
	// Evaluate any tokens found in the template path before checking if the folder already exists
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
	return AssetRegistryModule.Get().PathExists(ResolveTemplatePath(Path));
}

FString SFolderHierarchyPanel::ResolveTemplatePath(const FString& TemplatePath) const
{
	UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();

	FNamingTokenFilterArgs FilterArgs;
	FilterArgs.AdditionalNamespacesToInclude.Add(UCineAssemblyNamingTokens::TokenNamespace);

	const FNamingTokenResultData Result = NamingTokensSubsystem->EvaluateTokenString(TemplatePath, FilterArgs);
	return Result.EvaluatedText.ToString();
}

const FSlateBrush* SFolderHierarchyPanel::GetFolderIconBadge(TSharedPtr<FTemplateFolderTreeItem> TreeItem) const
{
	if (TreeItem->Status == ETemplateFolderStatus::MissingCreate)
	{
		const FSlateBrush* WarningBrush = FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Badges.FolderNew").GetIcon();
		return WarningBrush;
	}
	else if (TreeItem->Status == ETemplateFolderStatus::MissingDoNotCreate)
	{
		const FSlateBrush* WarningBrush = FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Badges.FolderRename").GetIcon();
		return WarningBrush;
	}
	return nullptr;
}

FText SFolderHierarchyPanel::GetFolderStatusText(TSharedPtr<FTemplateFolderTreeItem> TreeItem) const
{
	switch (TreeItem->Status)
	{
	case ETemplateFolderStatus::MissingCreate:
		return LOCTEXT("Status_MissingCreate", "Create");
	case ETemplateFolderStatus::MissingDoNotCreate:
		return LOCTEXT("Status_MissingDoNotCreate", "Do Not Create");
	default:
		return FText::GetEmpty();
	}
}

TSharedRef<FTemplateFolderDragDrop> FTemplateFolderDragDrop::New(const FString& Name)
{
	TSharedRef<FTemplateFolderDragDrop> DragDropOp = MakeShared<FTemplateFolderDragDrop>();
	DragDropOp->MouseCursor = EMouseCursor::GrabHandClosed;
	DragDropOp->ItemName = Name;
	DragDropOp->Construct();

	return DragDropOp;
}

TSharedPtr<SWidget> FTemplateFolderDragDrop::GetDefaultDecorator() const
{
	return SNew(SBorder)
		.Padding(8.0f)
		.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.PanelBackground"))
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 2.0f, 0.0f)
				[
					SNew(SImage)
						.ColorAndOpacity(FAppStyle::Get().GetSlateColor("ContentBrowser.DefaultFolderColor"))
						.Image(FCineAssemblyToolsStyle::Get().GetBrush("Icons.Folder"))
				]

			+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock).Text(FText::FromString(ItemName))
				]
		];
}

void FTemplateFolderDragDrop::OnDragged(const class FDragDropEvent& DragDropEvent)
{
	if (CursorDecoratorWindow.IsValid())
	{
		CursorDecoratorWindow->MoveWindowTo(DragDropEvent.GetScreenSpacePosition() - (CursorDecoratorWindow->GetSizeInScreen() * FVector2f(0.0f, 0.5f)));
	}
}

void FTemplateFolderDragDrop::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	if (!bDropWasHandled)
	{
		OnDropNotHandled.ExecuteIfBound();
	}
}

#undef LOCTEXT_NAMESPACE
