// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "ContentBrowserItemPath.h"
#include "ProductionSettings.h"
#include "SNamingTokensEditableTextBox.h"
#include "Widgets/Views/STreeView.h"

/** States in which a template folder can exist */
enum class ETemplateFolderStatus : uint8
{
	/** The status has not yet been determined */
	None,

	/** Template folder path exists in the Project Content folder */
	Exists,

	/** Template folder path does not exist in the Project Content folder, but should be created OnApply */
	MissingCreate,

	/** Template folder path does not exist in the Project Content folder, but should not be created OnApply */
	MissingDoNotCreate,
};

/** An entry in the template folder tree view with knowledge of its path, children, and parent */
struct FTemplateFolderTreeItem
{
	/** The path of this template folder */
	FContentBrowserItemPath Path;

	/** Friendly name that should appear in the tree view. If empty, the path leaf will be displayed */
	FString FriendlyName;

	/** The parent of this item in the tree */
	TSharedPtr<FTemplateFolderTreeItem> Parent;

	/** The children of this item in the tree */
	TArray<TSharedPtr<FTemplateFolderTreeItem>> Children;

	/** The status of this template folder, indicating whether it needs to be created OnApply */
	ETemplateFolderStatus Status = ETemplateFolderStatus::None;

	/** The text widget that display this item's name and supports renaming */
	TSharedPtr<SNamingTokensEditableTextBox> NameWidget;

	/** If true, this item cannot be renamed or deleted */
	bool bIsReadOnly = false;
};

/** Drag and Drop operation to handle dragging template folders  */
class FTemplateFolderDragDrop : public FDragDropOperation
{
public:

	DRAG_DROP_OPERATOR_TYPE(FTemplateFolderDragDrop, FDragDropOperation)

	DECLARE_DELEGATE(FOnDropNotHandled);

	FTemplateFolderDragDrop() = default;

	static TSharedRef<FTemplateFolderDragDrop> New(const FString& Name);

	// Begin FDragDropOperation overrides
	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;
	virtual void OnDragged(const class FDragDropEvent& DragDropEvent) override;
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	// End FDragDropOperation overrides

	/** The source item being dragged. This could be null if the thing being dragged is from the Asset List (not the tree) */
	TSharedPtr<FTemplateFolderTreeItem> SourceTreeItem;

	/** The name of the item being displayed, used by the decorator and to name the new tree item that will get made when dropped */
	FString ItemName;

	/** Callback to handle the case where an item is dropped somewhere not on the tree view */
	FOnDropNotHandled OnDropNotHandled;
};

/**
 * UI for the Folder Hierarchy panel in the Production Wizard
 */
class SFolderHierarchyPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFolderHierarchyPanel) {}
	SLATE_END_ARGS()

	SFolderHierarchyPanel() = default;
	~SFolderHierarchyPanel();

	void Construct(const FArguments& InArgs);

private:
	/** Generates the row widget for an entry in the tree view */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FTemplateFolderTreeItem> TreeItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Gets the children of the input tree view item to build additional tree rows */
	void OnGetChildren(TSharedPtr<FTemplateFolderTreeItem> TreeItem, TArray<TSharedPtr<FTemplateFolderTreeItem>>& OutNodes);

	/** Callback when the tree view rebuilds itself, used here to allow the user to immediately rename the item after the tree view updates to show it */
	void OnItemsRebuilt();

	/** Set the expansion state of the input treem item and all of its children (recursively) */
	void SetExpansionRecursive(TSharedPtr<FTemplateFolderTreeItem> TreeItem, bool bInExpand) const;

	/** Spawns a context menu when the user right-clicks on an entry in the tree view */
	TSharedPtr<SWidget> OnContextMenuOpening();

	/** Handles key presses on the tree view */
	FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/** Handles double clicks on the tree view to rename the item */
	void OnTreeViewDoubleClick(TSharedPtr<FTemplateFolderTreeItem> TreeItem);

	/** Begins a drag and drop event to drag an item out of the content tree view */
	FReply OnTreeRowDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Callback when an item is dropped onto a folder in the content tree view to add it to the children of that row */
	FReply OnTreeRowAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<FTemplateFolderTreeItem> InItem);

	/** Builds the template folder list items (displayed in the tree view) using the current active production's folder template */
	void BuildTreeFromProductionTemplate();

	/** Adds a new folder to the template under the currently selected tree item */
	FReply OnAddFolderToTemplate();

	/** Creates any template folders whose status is MissingCreate in the Content Browser */
	FReply OnCreateTemplateFolders();

	/** Resets the active production's template to the previously cached state and updates the tree view */
	FReply OnResetTemplateChanges();

	/** Returns true if the input text is a valid folder name, otherwise false and fills the output error message accordingly */
	bool IsValidFolderName(const FText& InText, FText& OutErrorMessage, TSharedPtr<FTemplateFolderTreeItem> TreeItem);

	/** 
	 * Sets the name of the input template folder.
	 * If the folder exists in the Content Browser, and is empty, the user is prompted with the option to also rename the Content Browser folder
	 */
	void SetTemplateFolderName(const FText& InText, ETextCommit::Type InCommitType, TSharedPtr<FTemplateFolderTreeItem> TreeItem);

	/** 
	 * Deletes the input template folder from the template.
	 * If the folder exists in the Content Browser, and is empty, the user is prompted with the option to also delete the Content Browser folder
	 */
	void DeleteTemplateFolder(TSharedPtr<FTemplateFolderTreeItem> SelectedTreeItem);

	/** Returns the template folder tree item whose path matches the input path */
	TSharedPtr<FTemplateFolderTreeItem> FindItemAtPath(const FString& Path);
	TSharedPtr<FTemplateFolderTreeItem> FindItemAtPathRecursive(TSharedPtr<FTemplateFolderTreeItem> TreeItem, const FString& Path);

	/** Walks the template folder tree and creates all folders marked as MissingCreate */
	void CreateFolderFromTemplateRecursive(TSharedPtr<FTemplateFolderTreeItem> TreeItem);

	/** Removes the input tree item and all of its children (recursively) from the template */
	void RemoveFolderFromTemplateRecursive(TSharedPtr<FTemplateFolderTreeItem> TreeItem);

	/** Modifies the path of the input tree item and all of its children (recursively) with the input new path */
	void SetChildrenPathRecursive(TSharedPtr<FTemplateFolderTreeItem> TreeItem, const FString& NewPath);

	/** Sets the folder status of the input tree item and all of its children (recursively) */
	void SetTemplateFolderStatusRecursive(TSharedPtr<FTemplateFolderTreeItem> TreeItem, ETemplateFolderStatus NewStatus);

	/** Returns a valid folder name that is unique at the input path in both the Content Browser and the Folder Template */
	FString CreateUniqueFolderName(FContentBrowserItemPath InPath);

	/** Returns true if the input tree item exists in the Content Browser and contains any assets (other than placeholders) */
	bool IsFolderEmpty(TSharedPtr<FTemplateFolderTreeItem> TreeItem) const;

	/** Returns the badge icon to overlay on top of the folder icon for the input tree item */
	const FSlateBrush* GetFolderIconBadge(TSharedPtr<FTemplateFolderTreeItem> TreeItem) const;

	/** Returns the status text to display next to the input tree item in its row in the tree view */
	FText GetFolderStatusText(TSharedPtr<FTemplateFolderTreeItem> TreeItem) const;

	/** Returns true if the path already exists in the asset registry */
	bool DoesPathExist(const FString& Path) const;

	/** Resolves any naming tokens in the input template path against the active production context. Returns the resolved path string. */
	FString ResolveTemplatePath(const FString& TemplatePath) const;

private:
	/** The source template folder items for the tree view */
	TArray<TSharedPtr<FTemplateFolderTreeItem>> FolderItemsSource;

	/** The tree view of template folders for the current active production */
	TSharedPtr<STreeView<TSharedPtr<FTemplateFolderTreeItem>>> TreeView;

	/** Convenient alias for the Project Content root item in the tree view */
	TSharedPtr<FTemplateFolderTreeItem> ContentRootItem;

	/** Convenient alias for the Project Plugin root item in the tree view */
	TSharedPtr<FTemplateFolderTreeItem> PluginRootItem;

	/** The cached state of the template, used to reset user changes */
	TArray<FFolderTemplate> CachedInitialState;

	/** The mostly recently added tree item, used to allow the user to immediately rename the item after the tree view updates to show it */
	TSharedPtr<FTemplateFolderTreeItem> MostRecentlyAddedItem;

	/** Guards against re-entrant SetTemplateFolderName when FMessageDialog::Open's nested message pump dispatches a second OnTextCommitted. */
	bool bIsSettingTemplateFolderName = false;

	FDelegateHandle ActiveProductionChangedHandle;
};
