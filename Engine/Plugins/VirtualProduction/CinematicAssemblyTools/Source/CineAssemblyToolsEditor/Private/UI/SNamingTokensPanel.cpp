// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNamingTokensPanel.h"

#include "Algo/Reverse.h"
#include "CineAssemblyToolsStyle.h"
#include "Engine/Engine.h"
#include "GlobalNamingTokens.h"
#include "NamingTokenData.h"
#include "NamingTokensEngineSubsystem.h"
#include "ProductionSettings.h"
#include "UI/SActiveProductionCombo.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "SNamingTokensPanel"

void SNamingTokensPanel::Construct(const FArguments& InArgs)
{
	const UNamingTokensEngineSubsystem* const NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();

	// Build the list of the global tokens
	const UNamingTokens* const GlobalTokens = NamingTokensSubsystem->GetNamingTokens(UGlobalNamingTokens::GetGlobalNamespace());
	const TArray<FNamingTokenData>& GlobalTokenData = GlobalTokens->GetDefaultTokens();
	for (const FNamingTokenData& TokenData : GlobalTokenData)
	{
		GlobalTokenListItems.Add(MakeShared<FNamingTokenData>(TokenData));
	}
	GlobalTokenListItems.Sort([](const TSharedPtr<FNamingTokenData>& A, const TSharedPtr<FNamingTokenData>& B) { return A->TokenKey < B->TokenKey; });

	// Build the list of all discovered naming token namespaces
	TArray<FString> TokenNamespaces = NamingTokensSubsystem->GetAllNamespaces();

	// The global namespace is displayed separately, and also cannot be added to the active production settings DenyList
	TokenNamespaces.Remove(UGlobalNamingTokens::GetGlobalNamespace());

	for (const FString& GlobalNamespace : TokenNamespaces)
	{
		NamingTokenNamespaceListItems.Add(MakeShared<FString>(GlobalNamespace));
	}
	NamingTokenNamespaceListItems.Sort([](const TSharedPtr<FString>& A, const TSharedPtr<FString>& B) { return *A < *B; });

	// The custom token list starts empty and is populated when one of the namespace list items is selected
	CustomTokenListItems.Empty();

	ChildSlot
		[
			SNew(SVerticalBox)

			// Active Production Selector
			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SActiveProductionCombo)
				]

			// Separator
			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator)
						.Orientation(Orient_Horizontal)
						.Thickness(2.0f)
				]

			// Naming Tokens Panel
			+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
						.Padding(16.0f)
						[
							SNew(SVerticalBox)

							// Title
							+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 4.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("NamingTokensTitle", "Production Settings"))
										.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.TitleFont"))
								]

							// Heading
							+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 4.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("NamingTokensHeading", "Naming Tokens"))
										.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.HeadingFont"))
								]

							// Info Text
							+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 8.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("NamingTokensInfoText", "Naming tokens insert relevant information automatically into your asset names."))
								]

							// Global Tokens Info Text
							+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 16.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("GlobalTokensInfoText", "The following global tokens are built-in tokens you can use in all projects and tools. They are always replaced by the values shown here."))
										.AutoWrapText(true)
								]

							// Global Tokens List View
							+ SVerticalBox::Slot()
								.AutoHeight()
								.MaxHeight(26 + 24 * 5)
								.Padding(0.0f, 0.0f, 0.0f, 16.0f)
								[
									SAssignNew(GlobalTokenListView, SListView<TSharedPtr<FNamingTokenData>>)
										.ListItemsSource(&GlobalTokenListItems)
										.OnGenerateRow(this, &SNamingTokensPanel::OnGenerateNamingTokenRow)
										.SelectionMode(ESelectionMode::None)
										.HeaderRow
										(
											SNew(SHeaderRow)

											+ SHeaderRow::Column("NamingTokens")
												.DefaultLabel(LOCTEXT("GlobalTokensLabel", "Global Tokens"))
												.FillWidth(0.3f)
												.OnSort(this, &SNamingTokensPanel::HandleGlobalTokenListSort)
												.SortMode_Lambda([this]() { return GlobalTokenListSortMode; })

											+ SHeaderRow::Column("Description")
												.DefaultLabel(LOCTEXT("DescriptionLabel", "Description"))
												.FillWidth(0.7f)
										)
								]

							// Separator
							+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 16.0f)
								[
									SNew(SSeparator)
										.Orientation(Orient_Horizontal)
										.Thickness(2.0f)
								]

							// Custom Tokens Info Text
							+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 16.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("CustomTokensInfoText", "Your project also defines the following namespaces and custom naming tokens."))
										.AutoWrapText(true)
								]

							// Custom Naming Tokens
							+ SVerticalBox::Slot()
								.FillHeight(1.0f)
								.Padding(0.0f, 0.0f, 0.0f, 0.0f)
								[
									SNew(SBorder)
										.BorderImage(FAppStyle::Get().GetBrush("Brushes.Background"))
										.Padding(4.0f)
										[
											SNew(SHorizontalBox)

											// Namespaces List View
											+ SHorizontalBox::Slot()
												.FillWidth(0.5f)
												.Padding(0.0f, 0.0f, 4.0f, 0.0f)
												[
													SNew(SOverlay)

													+ SOverlay::Slot()
														[
															SAssignNew(NamingTokenNamespaceListView, SListView<TSharedPtr<FString>>)
																.ListItemsSource(&NamingTokenNamespaceListItems)
																.OnGenerateRow(this, &SNamingTokensPanel::OnGenerateNamingTokenNamespaceRow)
																.SelectionMode(ESelectionMode::Single)
																.OnSelectionChanged(this, &SNamingTokensPanel::OnNamespaceSelectionChanged)
																.HeaderRow
																(
																	SNew(SHeaderRow)

																	+ SHeaderRow::Column("Namespaces")
																		.DefaultLabel(LOCTEXT("NamespacesLabel", "Namespaces"))
																		.OnSort(this, &SNamingTokensPanel::HandleNamespaceListSort)
																		.SortMode_Lambda([this]() { return NamespaceListSortMode; })
																)
														]

													+ SOverlay::Slot()
														.Padding(0.0f, 42.0f, 0.0f, 0.0f)
														.HAlign(HAlign_Center)
														[
															SNew(STextBlock)
																.Text(LOCTEXT("NoNamespacesFoundHintText", "No namespaces found in your project."))
																.TextStyle(FAppStyle::Get(), "HintText")
																.Visibility_Lambda([this]() -> EVisibility
																	{
																		return NamingTokenNamespaceListItems.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed;
																	})
														]
												]

											// Custom Naming Token List View
											+ SHorizontalBox::Slot()
												.FillWidth(0.5f)
												[
													SNew(SOverlay)

													+ SOverlay::Slot()
														[
															SAssignNew(CustomTokenListView, SListView<TSharedPtr<FNamingTokenData>>)
																.ListItemsSource(&CustomTokenListItems)
																.OnGenerateRow(this, &SNamingTokensPanel::OnGenerateNamingTokenRow)
																.SelectionMode(ESelectionMode::None)
																.HeaderRow
																(
																	SNew(SHeaderRow)

																	+ SHeaderRow::Column("NamingTokens")
																		.DefaultLabel(LOCTEXT("CustomTokensLabel", "Naming Tokens"))
																		.FillWidth(0.5f)
																		.OnSort(this, &SNamingTokensPanel::HandleCustomTokenListSort)
																		.SortMode_Lambda([this]() { return CustomTokenListSortMode; })

																	+ SHeaderRow::Column("Description")
																		.DefaultLabel(LOCTEXT("DescriptionLabel", "Description"))
																		.FillWidth(0.5f)
																)
														]

													+ SOverlay::Slot()
														.Padding(0.0f, 42.0f, 0.0f, 0.0f)
														.HAlign(HAlign_Center)
														[
															SNew(STextBlock)
																.Text(LOCTEXT("NoNamespaceSelectedHintText", "Select a namespace to see its naming tokens."))
																.TextStyle(FAppStyle::Get(), "HintText")
																.Visibility_Lambda([this]() -> EVisibility
																	{
																		return (NamingTokenNamespaceListView->GetNumItemsSelected() == 0) ? EVisibility::Visible : EVisibility::Collapsed;
																	})
														]
												]
										]
								]
						]
				]
		];
}

void SNamingTokensPanel::OnNamespaceSelectionChanged(TSharedPtr<FString> SelectedNamespace, ESelectInfo::Type SelectInfo)
{
	CustomTokenListItems.Empty();

	if (SelectedNamespace)
	{
		UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();
		if (UNamingTokens* CustomTokens = NamingTokensSubsystem->GetNamingTokens(*SelectedNamespace))
		{
			const TArray<FNamingTokenData>& CustomTokenData = CustomTokens->GetAllTokens();

			for (const FNamingTokenData& TokenData : CustomTokenData)
			{
				CustomTokenListItems.Add(MakeShared<FNamingTokenData>(TokenData));
			}
		}

		if (CustomTokenListSortMode == EColumnSortMode::Ascending)
		{
			CustomTokenListItems.Sort([](const TSharedPtr<FNamingTokenData>& A, const TSharedPtr<FNamingTokenData>& B) { return A->TokenKey < B->TokenKey; });
		}
		else
		{
			CustomTokenListItems.Sort([](const TSharedPtr<FNamingTokenData>& A, const TSharedPtr<FNamingTokenData>& B) { return A->TokenKey > B->TokenKey; });
		}
	}

	CustomTokenListView->RebuildList();
}

TSharedRef<ITableRow> SNamingTokensPanel::OnGenerateNamingTokenRow(TSharedPtr<FNamingTokenData> InTokenData, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SNamingTokenRow, OwnerTable, InTokenData);
}

TSharedRef<ITableRow> SNamingTokensPanel::OnGenerateNamingTokenNamespaceRow(TSharedPtr<FString> InNamespace, const TSharedRef<STableViewBase>& OwnerTable)
{
	auto IsActiveProductionValid = []() -> bool
		{
			const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
			TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
			return ActiveProduction.IsSet();
		};

	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		.ShowSelection(true)
		.Padding(FMargin(8.0f, 4.0f, 8.0f, 4.0f))
		.Content()
		[
			SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
						.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
						.OnCheckStateChanged(this, &SNamingTokensPanel::OnNamespaceChecked, InNamespace)
						.IsChecked(this, &SNamingTokensPanel::GetNamespaceCheckBoxState, InNamespace)
						.IsEnabled_Lambda(IsActiveProductionValid)
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(FText::FromString(*InNamespace))
				]
		];
}

void SNamingTokensPanel::OnNamespaceChecked(ECheckBoxState CheckBoxState, TSharedPtr<FString> Namespace)
{
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	if (CheckBoxState == ECheckBoxState::Checked)
	{
		ProductionSettings->RemoveNamespaceFromDenyList(ProductionSettings->GetActiveProductionID(), *Namespace);
	}
	else if (CheckBoxState == ECheckBoxState::Unchecked)
	{
		ProductionSettings->AddNamespaceToDenyList(ProductionSettings->GetActiveProductionID(), *Namespace);
	}
}

ECheckBoxState SNamingTokensPanel::GetNamespaceCheckBoxState(TSharedPtr<FString> Namespace) const
{
	const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
	TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
	if (ActiveProduction.IsSet())
	{
		if (ActiveProduction.GetValue().NamingTokenNamespaceDenyList.Contains(*Namespace))
		{
			return ECheckBoxState::Unchecked;
		}
	}
	return ECheckBoxState::Checked;
}

void SNamingTokensPanel::HandleGlobalTokenListSort(EColumnSortPriority::Type, const FName& ColumnId, EColumnSortMode::Type SortMode)
{
	GlobalTokenListSortMode = SortMode;
	Algo::Reverse(GlobalTokenListItems);
	GlobalTokenListView->RebuildList();
}

void SNamingTokensPanel::HandleNamespaceListSort(EColumnSortPriority::Type, const FName& ColumnId, EColumnSortMode::Type SortMode)
{
	NamespaceListSortMode = SortMode;
	Algo::Reverse(NamingTokenNamespaceListItems);
	NamingTokenNamespaceListView->RebuildList();
}

void SNamingTokensPanel::HandleCustomTokenListSort(EColumnSortPriority::Type, const FName& ColumnId, EColumnSortMode::Type SortMode)
{
	CustomTokenListSortMode = SortMode;
	Algo::Reverse(CustomTokenListItems);
	CustomTokenListView->RebuildList();
}

void SNamingTokenRow::Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView, TSharedPtr<FNamingTokenData>& InRowData)
{
	TokenData = InRowData;

	FSuperRowType::FArguments StyleArguments = FSuperRowType::FArguments()
		.Padding(FMargin(8.0f, 4.0f, 8.0f, 4.0f));

	SMultiColumnTableRow<TSharedPtr<FNamingTokenData>>::Construct(StyleArguments, OwnerTableView);
}

TSharedRef<SWidget> SNamingTokenRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == TEXT("NamingTokens"))
	{
		const FString FormattedTokenKey = TEXT("{") + TokenData->TokenKey + TEXT("}");

		return SNew(STextBlock)
			.Text(FText::FromString(FormattedTokenKey));
	}
	else if (ColumnName == TEXT("Description"))
	{
		const FText ToolTipText = TokenData->Description.IsEmpty() ? TokenData->DisplayName : TokenData->Description;

		return SNew(STextBlock)
			.Text(TokenData->DisplayName)
			.ToolTipText(ToolTipText);
	}

	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE
