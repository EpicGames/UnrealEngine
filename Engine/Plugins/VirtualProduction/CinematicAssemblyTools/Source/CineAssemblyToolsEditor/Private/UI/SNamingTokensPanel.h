// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"

struct FNamingTokenData;

/** Row widget for the global tokens list view */
class SNamingTokenRow : public SMultiColumnTableRow<TSharedPtr<FNamingTokenData>>
{
public:
	SLATE_BEGIN_ARGS(SNamingTokenRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView, TSharedPtr<FNamingTokenData>& InRowData);

	/** Creates the widget for this row for the specified column */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	/** The underlying token data, used to propertly display its token key and display name */
	TSharedPtr<FNamingTokenData> TokenData;
};

/**
 * UI for the Naming Tokens panel in the Production Wizard
 */
class SNamingTokensPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNamingTokensPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	/** Generates a row displaying token data */
	TSharedRef<ITableRow> OnGenerateNamingTokenRow(TSharedPtr<FNamingTokenData> InTokenData, const TSharedRef<STableViewBase>& OwnerTable);

	/** Generates a row displaying a namespace, and a checkbox to add/remove it from the active production settings DenyList */
	TSharedRef<ITableRow> OnGenerateNamingTokenNamespaceRow(TSharedPtr<FString> InNamespace, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback when the selection in the namespace list view changes, which updates the custom token list view to display the tokens in the selected namespace */
	void OnNamespaceSelectionChanged(TSharedPtr<FString> SelectedNamespace, ESelectInfo::Type SelectInfo);

	/** Adds / Removes the associated namespace from the active production settings DenyList */
	void OnNamespaceChecked(ECheckBoxState CheckBoxState, TSharedPtr<FString> Namespace);

	/** Indicates whether the associated namespace is in the active production settings DenyList */
	ECheckBoxState GetNamespaceCheckBoxState(TSharedPtr<FString> Namespace) const;

	/** Handles sorting of the items in each list view  in this panel */
	void HandleGlobalTokenListSort(EColumnSortPriority::Type, const FName& ColumnId, EColumnSortMode::Type SortMode);
	void HandleNamespaceListSort(EColumnSortPriority::Type, const FName& ColumnId, EColumnSortMode::Type SortMode);
	void HandleCustomTokenListSort(EColumnSortPriority::Type, const FName& ColumnId, EColumnSortMode::Type SortMode);

private:
	/** Sources for the list views in this panel */
	TArray<TSharedPtr<FNamingTokenData>> GlobalTokenListItems;
	TArray<TSharedPtr<FString>> NamingTokenNamespaceListItems;
	TArray<TSharedPtr<FNamingTokenData>> CustomTokenListItems;

	/** List view widgets in this panel */
	TSharedPtr<SListView<TSharedPtr<FNamingTokenData>>> GlobalTokenListView;
	TSharedPtr<SListView<TSharedPtr<FString>>> NamingTokenNamespaceListView;
	TSharedPtr<SListView<TSharedPtr<FNamingTokenData>>> CustomTokenListView;

	/** Sort mode for each of the sortable lists in this panel */
	EColumnSortMode::Type GlobalTokenListSortMode = EColumnSortMode::Ascending;
	EColumnSortMode::Type NamespaceListSortMode = EColumnSortMode::Ascending;
	EColumnSortMode::Type CustomTokenListSortMode = EColumnSortMode::Ascending;
};
