// Copyright Epic Games, Inc. All Rights Reserved.

#include "SProductionListPanel.h"

#include "Algo/Contains.h"
#include "CineAssemblyToolsStyle.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/FileManager.h"
#include "JsonObjectConverter.h"
#include "Layout/WidgetPath.h"
#include "Misc/MessageDialog.h"
#include "ProductionSettings.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"

#include "SProductionListPanel.h"

#define LOCTEXT_NAMESPACE "SProductionListPanel"

void SProductionListPanel::Construct(const FArguments& InArgs)
{
	// Subscribe to be notified when the Production Settings list of productions has changed (for example, if a production was added/removed)
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ProductionListChangedHandle = ProductionSettings->OnProductionListChanged().AddSP(this, &SProductionListPanel::UpdateProductionList);

	// Build the list of productions for the list view before building the widget for the first time
	UpdateProductionList();

	auto OnCreateNewProductionClicked = [this]() -> FReply
		{
			UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
			ProductionSettings->AddProduction();

			const TArray<FCinematicProduction> Productions = ProductionSettings->GetProductions();
			MostRecentProductionID = Productions.Last().ProductionID;
			return FReply::Handled();
		};

	ChildSlot
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
				.Padding(16.0f)
				[
					SNew(SVerticalBox)

						// Title
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 0.0f, 0.0f, 4.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("ProductionsTitle", "User Setup"))
								.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.TitleFont"))
						]

						// Heading
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 0.0f, 0.0f, 4.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("ProductionsHeading", "Productions"))
								.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.HeadingFont"))
						]

						// Info Text
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 0.0f, 0.0f, 16.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("ProductionsInfoText", "You can use this Unreal project for multiple shows, and assign different settings for each show.\n"
									"Create a production here for each show, then choose one to be the active production."))
						]

						// Create / Import Buttons
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 0.0f, 0.0f, 4.0f)
						[
							SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SButton)
										.ContentPadding(FMargin(2.0f))
										.OnClicked_Lambda(OnCreateNewProductionClicked)
										[
											SNew(SHorizontalBox)

												+ SHorizontalBox::Slot()
												.AutoWidth()
												.Padding(0.0f, 0.0f, 4.0f, 0.0f)
												[
													SNew(SImage)
														.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
														.ColorAndOpacity(FStyleColors::AccentGreen)
												]


												+ SHorizontalBox::Slot()
												.AutoWidth()
												[
													SNew(STextBlock).Text(LOCTEXT("CreateNewProductionButton", "Create a New Production"))
												]
										]
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SButton)
										.ContentPadding(FMargin(2.0f))
										.OnClicked(this, &SProductionListPanel::ImportProduction)
										[
											SNew(SHorizontalBox)

												+ SHorizontalBox::Slot()
												.AutoWidth()
												.Padding(0.0f, 0.0f, 4.0f, 0.0f)
												[
													SNew(SImage)
														.Image(FAppStyle::Get().GetBrush("Icons.Import"))
												]

												+ SHorizontalBox::Slot()
												.AutoWidth()
												[
													SNew(STextBlock)
														.Text(LOCTEXT("ImportProductionButton", "Import Production"))
												]
										]
								]
						]

						// Production List View
						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							SNew(SBorder)
								.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
								.Padding(8.0f)
								[
									SAssignNew(ProductionListView, SListView<TSharedPtr<FCinematicProduction>>)
										.ListItemsSource(&ProductionList)
										.SelectionMode(ESelectionMode::None)
										.OnGenerateRow(this, &SProductionListPanel::OnGenerateProductionRow)
										.OnItemsRebuilt(this, &SProductionListPanel::OnProductionListItemsRebuilt)
								]
						]
				]
		];
}

SProductionListPanel::~SProductionListPanel()
{
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	ProductionSettings->OnProductionListChanged().Remove(ProductionListChangedHandle);
}

void SProductionListPanel::UpdateProductionList()
{
	ProductionList.Empty();

	const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
	const TArray<FCinematicProduction> Productions = ProductionSettings->GetProductions();
	for (const FCinematicProduction& Production : Productions)
	{
		ProductionList.Add(MakeShared<FCinematicProduction>(Production));
	}

	if (ProductionListView)
	{
		ProductionListView->RequestListRefresh();
	}
}

void SProductionListPanel::OnProductionListItemsRebuilt()
{
	if (ProductionListButtonToRename)
	{
		ProductionListButtonToRename->EnterEditMode();
		ProductionListButtonToRename.Reset();
		MostRecentProductionID.Invalidate();
	}
}

TSharedRef<ITableRow> SProductionListPanel::OnGenerateProductionRow(TSharedPtr<FCinematicProduction> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	auto OnDeleteProduction = [](FGuid ProductionID) -> FReply
		{
			UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
			ProductionSettings->DeleteProduction(ProductionID);
			return FReply::Handled();
		};

	TSharedRef<SProductionListButton> ListButton = SNew(SProductionListButton)
		.ProductionID(InItem->ProductionID);

	// If the production shown in this row was just created, it will have a valid default name, but the user should have an opportunity to immediately rename it.
	// However, the production's list button cannot be put into edit mode and focused until after the widget is created.
	// Therefore, the widget is saved, and will be put into edit mode after the items for this ListView are finished being rebuilt. 
	if (InItem->ProductionID.IsValid() && InItem->ProductionID == MostRecentProductionID)
	{
		ProductionListButtonToRename = ListButton;
	}

	return SNew(STableRow<TSharedPtr<FName>>, OwnerTable)
		.ShowSelection(true)
		.Content()
		[
			SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					ListButton
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
						.OnClicked(this, &SProductionListPanel::ExportProduction, InItem)
						.ButtonStyle(FCineAssemblyToolsStyle::Get(), "ProductionWizard.RecessedButton")
						.ToolTipText(LOCTEXT("ExportProductionButtonToolTip", "Export Production"))
						[
							SNew(SImage).Image(FCineAssemblyToolsStyle::Get().GetBrush("Icons.Export"))
						]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
						.ButtonStyle(FCineAssemblyToolsStyle::Get(), "ProductionWizard.RecessedButton")
						.OnClicked_Lambda(OnDeleteProduction, InItem->ProductionID)
						.ToolTipText(LOCTEXT("DeleteProductionButtonToolTip", "Delete Production"))
						[
							SNew(SImage).Image(FAppStyle::Get().GetBrush("Icons.Delete"))
						]
				]
		];
}

FReply SProductionListPanel::ImportProduction()
{
	// Prompt the user to choose a production setting .json file to open
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
	const FString Title = TEXT("Import Production Settings");
	const FString DefaultPath = FPaths::ProjectSavedDir();
	const FString DefaultFile = TEXT("");
	const FString FileTypes = TEXT("json|*.json");
	const uint32 FileFlags = 0;

	TArray<FString> OpenFileNames;
	const bool bFileSelected = DesktopPlatform->OpenFileDialog(ParentWindowHandle, Title, DefaultPath, DefaultFile, FileTypes, FileFlags, OpenFileNames);

	if (!bFileSelected || OpenFileNames.Num() != 1)
	{
		return FReply::Handled();
	}

	const FString JsonFileName = OpenFileNames[0];

	if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*JsonFileName)))
	{
		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(FileReader.Get());

		TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		if (FJsonSerializer::Deserialize(JsonReader, JsonObject))
		{
			TSharedPtr<FCinematicProduction> ImportedProduction = MakeShared<FCinematicProduction>();

			// We enforce strict mode to ensure that every persisted field in the UStruct is present in the imported json.
			// Skip Deprecated/Transient to match UStructToJsonObject's implicit defaults on export.
			constexpr int64 CheckFlags = 0;
			constexpr int64 SkipFlags = CPF_Deprecated | CPF_Transient;
			constexpr bool bStrictMode = true;
			FText ErrorReason;
			const bool bSuccessfulImport = FJsonObjectConverter::JsonObjectToUStruct<FCinematicProduction>(JsonObject.ToSharedRef(), ImportedProduction.Get(), CheckFlags, SkipFlags, bStrictMode, &ErrorReason);

			if (bSuccessfulImport)
			{
				UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
				ImportedProduction->ProductionID = FGuid::NewGuid();

				// If the name of the production being imported already exists in the list of productions, prompt the user to confirm before actually adding it
				const TArray<FCinematicProduction> Productions = ProductionSettings->GetProductions();			
				const bool bProductionNameAlreadyExists = Algo::ContainsBy(Productions, ImportedProduction->ProductionName, &FCinematicProduction::ProductionName);

				bool bShouldAddProduction = true;
				if (bProductionNameAlreadyExists)
				{
					const FText DialogMessage = FText::Format(LOCTEXT("ProductionNameAlreadyExistsMessage", "A Production named {0} already exists. Do you still want to import this production?"), FText::FromString(ImportedProduction->ProductionName));
					const EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNo, DialogMessage);

					if (Response != EAppReturnType::Yes)
					{
						bShouldAddProduction = false;
					}
				}

				if (bShouldAddProduction)
				{
					ProductionSettings->AddProduction(*ImportedProduction);
				}
			}
			else
			{
				const FText ErrorMessage = FText::Format(LOCTEXT("SpecificImportErrorMessage", "The selected file failed to import.\n\n{0}"), ErrorReason);
				FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
			}
		}
		else
		{
			const FText ErrorMessage = LOCTEXT("GenericImportErrorMessage", "The selected file failed to import.\n\nThe JSON file may be incorrectly formatted.");
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
		}
	}

	return FReply::Handled();
}

FReply SProductionListPanel::ExportProduction(TSharedPtr<FCinematicProduction> InItem)
{
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	TOptional<const FCinematicProduction> ProductionToExport = ProductionSettings->GetProduction(InItem->ProductionID);
	if (!ProductionToExport.IsSet())
	{
		return FReply::Handled();
	}

	// Convert the production settings to a json object that can be written to .json file on disk
	if (const TSharedPtr<FJsonObject>& JsonObject = FJsonObjectConverter::UStructToJsonObject<FCinematicProduction>(ProductionToExport.GetValue()))
	{
		// Prompt the user to choose a location to save the production setting .json file
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
		const FString Title = TEXT("Export Production Settings");
		const FString DefaultPath = FPaths::ProjectSavedDir();
		const FString DefaultFile = ProductionToExport.GetValue().ProductionName + ".json";
		const FString FileTypes = TEXT("json|*.json");
		const uint32 FileFlags = 0;

		TArray<FString> SaveFileNames;
		const bool bFileSelected = DesktopPlatform->SaveFileDialog(ParentWindowHandle, Title, DefaultPath, DefaultFile, FileTypes, FileFlags, SaveFileNames);

		if (!bFileSelected || SaveFileNames.Num() != 1)
		{
			return FReply::Handled();
		}

		const FString JsonFileName = SaveFileNames[0];

		// Write the contents of the json object to disk
		if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*JsonFileName)))
		{
			TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(FileWriter.Get());

			FJsonSerializer::Serialize(JsonObject.ToSharedRef(), JsonWriter);
			FileWriter->Close();
		}
	}

	return FReply::Handled();
}

SProductionListButton::SProductionListButton()
	: ProductionID(*this)
{
}

void SProductionListButton::Construct(const FArguments& InArgs)
{
	ProductionID.Assign(*this, InArgs._ProductionID);

	SButton::Construct(SButton::FArguments());
	SetButtonStyle(&FCineAssemblyToolsStyle::Get().GetWidgetStyle<FButtonStyle>("ProductionWizard.RecessedButton"));

	auto OnProductionButtonClicked = [](FGuid InProductionID) -> FReply
		{
			UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
			if (ProductionSettings->IsActiveProduction(InProductionID))
			{
				ProductionSettings->SetActiveProduction(FGuid());
			}
			else
			{
				ProductionSettings->SetActiveProduction(InProductionID);
			}

			return FReply::Handled();
		};

	SetOnClicked(FOnClicked::CreateLambda(OnProductionButtonClicked, ProductionID.Get()));

	// Returns the color of the checkmark icon based on the active production and button hover state
	auto GetCheckMarkColor = [this]() -> FSlateColor
		{
			const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
			if (ProductionSettings->IsActiveProduction(ProductionID.Get()))
			{
				return FStyleColors::AccentGreen;
			}
			else if (bIsHovered)
			{
				return FStyleColors::Foreground;
			}
			return FStyleColors::Transparent;
		};

	// Returns the production name based on the ProductionID assigned to this button
	auto GetProductionName = [this]() -> FText
		{
			const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
			TOptional<const FCinematicProduction> Production = ProductionSettings->GetProduction(ProductionID.Get());
			if (Production.IsSet())
			{
				return FText::FromString(Production.GetValue().ProductionName);
			}
			return FText::GetEmpty();
		};

	// Renames this button's production
	auto OnRenameFinished = [this](const FText& InText, ETextCommit::Type InCommitType)
		{
			UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
			ProductionSettings->RenameProduction(ProductionID.Get(), InText.ToString());
		};

	auto OnVerifyRename = [this](const FText& InText, FText& OutErrorMessage) -> bool
		{
			const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();

			if (InText.IsEmpty())
			{
				OutErrorMessage = LOCTEXT("EmptyNameErrorMessage", "Please provide a name for the production");
				return false;
			}

			bool bIsNameValid = true;
			const TArray<FCinematicProduction> Productions = ProductionSettings->GetProductions();
			for (const FCinematicProduction& Production : Productions)
			{
				if (Production.ProductionID == ProductionID.Get())
				{
					continue;
				}

				if (InText.ToString() == Production.ProductionName)
				{
					bIsNameValid = false;
					OutErrorMessage = LOCTEXT("ExistingNameErrorMessage", "A production already exists with this name");
				}
			}
			return bIsNameValid;
		};

	ChildSlot
		[
			SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Check"))
						.ColorAndOpacity_Lambda(GetCheckMarkColor)
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SAssignNew(EditableTextBlock, SInlineEditableTextBlock)
						.Text_Lambda(GetProductionName)
						.OnVerifyTextChanged_Lambda(OnVerifyRename)
						.OnTextCommitted_Lambda(OnRenameFinished)
						.IsSelected_Lambda([]() { return false; }) // Disable double-select to rename
						.MaximumLength(UProductionSettings::ProductionNameMaxLength)
				]
		];
}

void SProductionListButton::EnterEditMode()
{
	EditableTextBlock->EnterEditingMode();
}

FReply SProductionListButton::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Create the context menu to be launched on right mouse click
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		FMenuBuilder MenuBuilder(true, nullptr);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("RenameProduction", "Rename"),
			LOCTEXT("RenameProductionToolTip", "Rename production"),
			FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.AssetNaming"),
			FUIAction(FExecuteAction::CreateSP(this, &SProductionListButton::EnterEditMode)),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DuplicateProduction", "Duplicate"),
			LOCTEXT("DuplicateProductionToolTip", "Duplicate production"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate"),
			FUIAction(FExecuteAction::CreateLambda([this]()
				{
					UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
					ProductionSettings->DuplicateProduction(ProductionID.Get());
				}
			)),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

		FSlateApplication::Get().PushMenu(
			AsShared(),
			WidgetPath,
			MenuBuilder.MakeWidget(),
			MouseEvent.GetScreenSpacePosition(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);

		return FReply::Handled();
	}

	return SButton::OnMouseButtonDown(MyGeometry, MouseEvent);
}

void SProductionListButton::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SButton::OnMouseEnter(MyGeometry, MouseEvent);
	bIsHovered = true;
}

void SProductionListButton::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SButton::OnMouseLeave(MouseEvent);
	bIsHovered = false;
}

#undef LOCTEXT_NAMESPACE
