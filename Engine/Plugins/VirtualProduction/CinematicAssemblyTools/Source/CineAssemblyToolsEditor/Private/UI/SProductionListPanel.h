// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SListView.h"

struct FCinematicProduction;

/**
 * Button used in the Productions panel, containing functionality for a single FCinematicProduction.
 * This button supports right-click (for renaming) and will show an editable text box for the user to rename a button.
 */
class SProductionListButton : public SButton
{
public:
	SProductionListButton();

	SLATE_BEGIN_ARGS(SProductionListButton)
		: _ProductionID()
		{ }

		/** ID of the production that this button controls */
		SLATE_ATTRIBUTE(FGuid, ProductionID)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** OnMouseButtonDown is overridden because the base class SButton ignores right-clicks */
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	/** Put the editable text block into edit mode */
	void EnterEditMode();

private:
	/** ID of the production that this button controls */
	TSlateAttribute<FGuid, EInvalidateWidgetReason::Paint> ProductionID;

	/** User-editable text widget for renaming the production */
	TSharedPtr<SInlineEditableTextBlock> EditableTextBlock;

	/** Hover state, used to determine color/opacity of the button icon */
	bool bIsHovered = false;
};

/**
 * UI for the Productions panel of the Production Wizard
 */
class SProductionListPanel : public SCompoundWidget
{
public:
	SProductionListPanel() = default;
	~SProductionListPanel();

	SLATE_BEGIN_ARGS(SProductionListPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	/** Creates the list of productions used by the production list view widget*/
	void UpdateProductionList();

	/** Generates a row in the production list view */
	TSharedRef<ITableRow> OnGenerateProductionRow(TSharedPtr<FCinematicProduction> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback when the production list view is finished regenerating its rows */
	void OnProductionListItemsRebuilt();

	/** Exports the input production to a .json file on disk containing all of its production settings */
	FReply ExportProduction(TSharedPtr<FCinematicProduction> InItem);

	/** Imports a production setting .json file from disk and adds it to the Production Project Settings list of productions */
	FReply ImportProduction();

private:
	/** List items sources for the production list view */
	TArray<TSharedPtr<FCinematicProduction>> ProductionList;

	/** Production list views, which allows the user to interact with one of the available productions in the Productions menu */
	TSharedPtr<SListView<TSharedPtr<FCinematicProduction>>> ProductionListView;

	/** Button belonging to one of the production list view rows that needs to be put into edit mode for the user to rename its production */
	TSharedPtr<SProductionListButton> ProductionListButtonToRename;

	/** Valid immediately after a new production is added, used to determine which production list item should be put into edit mode */
	FGuid MostRecentProductionID;

	/** Handle to the delegate which responds to changes in UProductionSettings list of productions */
	FDelegateHandle ProductionListChangedHandle;
};
