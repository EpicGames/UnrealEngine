// Copyright Epic Games, Inc. All Rights Reserved.

#include "SProductionWizard.h"

#include "CineAssemblyToolsStyle.h"
#include "CineAssemblyToolsEditorModule.h"
#include "ProductionSettings.h"
#include "UI/SNamingTokensPanel.h"
#include "UI/SActiveProductionExtendedDataDetailsView.h"
#include "UI/SAssetNamingPanel.h"
#include "UI/SFolderHierarchyPanel.h"
#include "UI/SProductionListPanel.h"
#include "UI/SRevisionControlPanel.h"
#include "UI/SSequencerSettingsPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SProductionWizard"

void SProductionWizard::Construct(const FArguments& InArgs)
{
	ChildSlot
		[
			SNew(SBorder)
				.Padding(0.f)
				.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.PanelBackground"))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
						[
							SNew(SSplitter)
								.Orientation(Orient_Horizontal)
								.PhysicalSplitterHandleSize(2.0f)

							+ SSplitter::Slot()
								.Value(0.25f)
								.MinSize(192.0f)
								.Expose(MenuPanelSlot)

							+ SSplitter::Slot()
								.Value(0.75f)
								.Expose(ContentPanelSlot)
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SSeparator)
								.Orientation(Orient_Horizontal)
								.Thickness(2.0f)
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							MakeButtonsPanel()
						]
				]
		];

	MenuPanelSlot->AttachWidget(MakeMenuPanel());
}

TSharedRef<SWidget> SProductionWizard::MakeButtonsPanel()
{
	auto IsBackButtonEnabled = [this]()
		{
			TArray<TSharedPtr<FProductionWizardMenuEntry>> SelectedUserMenuEntries = UserMenuListView->GetSelectedItems();
			if (SelectedUserMenuEntries.Num() > 0)
			{
				if (UserMenuEntries.IndexOfByKey(SelectedUserMenuEntries[0]) == 0)
				{
					return false;
				}
			}
			return true;
		};


	auto IsNextButtonEnabled = [this]()
		{
			TArray<TSharedPtr<FProductionWizardMenuEntry>> SelectedProductionMenuEntries = ProductionMenuListView->GetSelectedItems();
			if (SelectedProductionMenuEntries.Num() > 0)
			{
				if (ProductionMenuEntries.IndexOfByKey(SelectedProductionMenuEntries[0]) == ProductionMenuEntries.Num() - 1)
				{
					return false;
				}
			}
			return true;
		};

	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(16.0f)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.MinWidth(118.0f)
				.MaxWidth(118.0f)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SButton)
						.Text(LOCTEXT("BackButton", "Back"))
						.HAlign(HAlign_Center)
						.OnClicked(this, &SProductionWizard::OnBackClicked)
						.IsEnabled_Lambda(IsBackButtonEnabled)
				]

			+ SHorizontalBox::Slot()
				.MinWidth(118.0f)
				.MaxWidth(118.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
						.Text(LOCTEXT("NextButton", "Next"))
						.HAlign(HAlign_Center)
						.OnClicked(this, &SProductionWizard::OnNextClicked)
						.IsEnabled_Lambda(IsNextButtonEnabled)
				]
		];
}

FReply SProductionWizard::OnBackClicked()
{
	// Both list views are single-select only, so they should only ever have 0 or 1 item selected
	TArray<TSharedPtr<FProductionWizardMenuEntry>> SelectedUserMenuEntries = UserMenuListView->GetSelectedItems();
	TArray<TSharedPtr<FProductionWizardMenuEntry>> SelectedProductionMenuEntries = ProductionMenuListView->GetSelectedItems();

	if (SelectedUserMenuEntries.Num() > 0)
	{
		const int32 PreviousEntryIndex = UserMenuEntries.IndexOfByKey(SelectedUserMenuEntries[0]) - 1;
		if (PreviousEntryIndex >= 0)
		{
			UserMenuListView->SetSelection(UserMenuEntries[PreviousEntryIndex]);
		}
	}
	else if (SelectedProductionMenuEntries.Num() > 0)
	{
		const int32 PreviousEntryIndex = ProductionMenuEntries.IndexOfByKey(SelectedProductionMenuEntries[0]) - 1;
		if (PreviousEntryIndex >= 0)
		{
			ProductionMenuListView->SetSelection(ProductionMenuEntries[PreviousEntryIndex]);
		}
		else
		{
			UserMenuListView->SetSelection(UserMenuEntries.Last());
		}
	}

	return FReply::Handled();
}

FReply SProductionWizard::OnNextClicked()
{
	// Both list views are single-select only, so they should only ever have 0 or 1 item selected
	TArray<TSharedPtr<FProductionWizardMenuEntry>> SelectedUserMenuEntries = UserMenuListView->GetSelectedItems();
	TArray<TSharedPtr<FProductionWizardMenuEntry>> SelectedProductionMenuEntries = ProductionMenuListView->GetSelectedItems();

	if (SelectedUserMenuEntries.Num() > 0)
	{
		const int32 NextIndex = UserMenuEntries.IndexOfByKey(SelectedUserMenuEntries[0]) + 1;
		if (NextIndex < UserMenuEntries.Num())
		{
			UserMenuListView->SetSelection(UserMenuEntries[NextIndex]);
		}
		else
		{
			ProductionMenuListView->SetSelection(ProductionMenuEntries[0]);
		}
	}
	else if (SelectedProductionMenuEntries.Num() > 0)
	{
		const int32 NextIndex = ProductionMenuEntries.IndexOfByKey(SelectedProductionMenuEntries[0]) + 1;
		if (NextIndex < ProductionMenuEntries.Num())
		{
			ProductionMenuListView->SetSelection(ProductionMenuEntries[NextIndex]);
		}
	}

	return FReply::Handled();
}

void SProductionWizard::MakeMenuExtensions()
{
	// Grab all User Settings extensions to add
	FCineAssemblyToolsEditorModule& CatEdModule = FModuleManager::Get().GetModuleChecked<FCineAssemblyToolsEditorModule>("CineAssemblyToolsEditor");
	const FProductionExtensions& Extensions = CatEdModule.GetProductionExtensions();

	for (const TPair<FName, FProductionWizardUserSettingsExtensionParameters>& Entry : Extensions.GetProductionWizardUserSettingsExtensions())
	{
		TSharedPtr<SWidget> CustomWidget = nullptr;
		
		if (Entry.Value.MakeCustomWidget.IsBound())
		{
			CustomWidget = Entry.Value.MakeCustomWidget.Execute();
		}

		if (CustomWidget == nullptr)
		{
			continue;
		}

		TSharedPtr<FProductionWizardMenuEntry> UserSettingsExtensionEntry = MakeShared<FProductionWizardMenuEntry>();
		UserSettingsExtensionEntry->Label = Entry.Value.Label;

		if (Entry.Value.Icon.IsSet())
		{
			UserSettingsExtensionEntry->Icon = Entry.Value.Icon.Get();
		}

		UserSettingsExtensionEntry->Panel = CustomWidget;

		UserMenuEntries.Add(MoveTemp(UserSettingsExtensionEntry));
	}

	// Grab all production settings extensions and customizations to add
	const TMap<const UScriptStruct*, FProductionWizardCustomizationParameters>& WizardCustomizations = Extensions.GetProductionWizardCustomizations();

	for (const UScriptStruct* ProductionExtensionType : Extensions.GetProductionExtensions())
	{
		TSharedPtr<FProductionWizardMenuEntry> ExtensionMenuEntry = MakeShared<FProductionWizardMenuEntry>();
		ExtensionMenuEntry->Label = FText::FromName(ProductionExtensionType->GetFName());

		bool bShowProductionSelection = true;
		TSharedPtr<SWidget> InnerWidget = nullptr;

		if (const FProductionWizardCustomizationParameters* Customization = WizardCustomizations.Find(ProductionExtensionType))
		{
			// Only add the entry if the customization is not hiding it from the wizard.
			if (Customization->bHideInWizard)
			{
				continue;
			}

			bShowProductionSelection = Customization->bShowProductionSelection;

			if (Customization->MakeCustomWidget.IsBound())
			{
				InnerWidget = Customization->MakeCustomWidget.Execute();
			}

			if (Customization->Label.IsSet())
			{
				ExtensionMenuEntry->Label = Customization->Label.Get();
			}

			if (Customization->Icon.IsSet())
			{
				ExtensionMenuEntry->Icon = Customization->Icon.Get();
			}
		}

		ExtensionMenuEntry->Panel = SNew(SActiveProductionExtendedDataDetailsView)
			.bShowProductionSelection(bShowProductionSelection)
			.InnerWidget(InnerWidget)
			.TargetScriptStruct(ProductionExtensionType);

		ProductionMenuEntries.Add(MoveTemp(ExtensionMenuEntry));
	}
}

TSharedRef<SWidget> SProductionWizard::MakeMenuPanel()
{
	// Build the Revision Control menu entry, including its content panel
	TSharedPtr<FProductionWizardMenuEntry> RevisionControlMenuEntry = MakeShared<FProductionWizardMenuEntry>();
	RevisionControlMenuEntry->Label = LOCTEXT("RevisionControlMenuLabel", "Revision Control");
	RevisionControlMenuEntry->Icon = FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.RevisionControl");
	RevisionControlMenuEntry->Panel = SNew(SRevisionControlPanel);

	// Build the Productions menu entry, including its content panel
	TSharedPtr<FProductionWizardMenuEntry> ProductionMenuEntry = MakeShared<FProductionWizardMenuEntry>();
	ProductionMenuEntry->Label = LOCTEXT("ProductionsMenuLabel", "Productions");
	ProductionMenuEntry->Icon = FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.Productions");
	ProductionMenuEntry->Panel = SNew(SProductionListPanel);

	UserMenuEntries.Add(MoveTemp(RevisionControlMenuEntry));
	UserMenuEntries.Add(MoveTemp(ProductionMenuEntry));

	// Build the Sequencer Settings menu entry, including its content panel
	TSharedPtr<FProductionWizardMenuEntry> SequencerSettingsMenuEntry = MakeShared<FProductionWizardMenuEntry>();
	SequencerSettingsMenuEntry->Label = LOCTEXT("SequencerSettingsMenuLabel", "Sequencer Settings");
	SequencerSettingsMenuEntry->Icon = FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.Sequencer");
	SequencerSettingsMenuEntry->Panel = SNew(SSequencerSettingsPanel);

	// Build the Naming Tokens menu entry, including its content panel
	TSharedPtr<FProductionWizardMenuEntry> NamingTokensMenuEntry = MakeShared<FProductionWizardMenuEntry>();
	NamingTokensMenuEntry->Label = LOCTEXT("NamingTokensMenuLabel", "Naming Tokens");
	NamingTokensMenuEntry->Icon = FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.NamingTokens");
	NamingTokensMenuEntry->Panel = SNew(SNamingTokensPanel);

	// Build the Asset Naming menu entry, including its content panel
	TSharedPtr<FProductionWizardMenuEntry> AssetNamingMenuEntry = MakeShared<FProductionWizardMenuEntry>();
	AssetNamingMenuEntry->Label = LOCTEXT("AssetNamingMenuLabel", "Asset Naming");
	AssetNamingMenuEntry->Icon = FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.AssetNaming");
	AssetNamingMenuEntry->Panel = SNew(SAssetNamingPanel);

	// Build the Folder Hierarchy menu entry, including its content panel
	TSharedPtr<FProductionWizardMenuEntry> FolderHierarchyMenuEntry = MakeShared<FProductionWizardMenuEntry>();
	FolderHierarchyMenuEntry->Label = LOCTEXT("FolderHierarchyMenuLabel", "Folder Hierarchy");
	FolderHierarchyMenuEntry->Icon = FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Icons.Folder");
	FolderHierarchyMenuEntry->Panel = SNew(SFolderHierarchyPanel);

	ProductionMenuEntries.Add(SequencerSettingsMenuEntry);
	ProductionMenuEntries.Add(NamingTokensMenuEntry);
	ProductionMenuEntries.Add(AssetNamingMenuEntry);
	ProductionMenuEntries.Add(FolderHierarchyMenuEntry);

	// Make any registered extensions
	MakeMenuExtensions();

	// Construct the list view for the user menu entries
	UserMenuListView = SNew(SListView<TSharedPtr<FProductionWizardMenuEntry>>)
		.ListItemsSource(&UserMenuEntries)
		.SelectionMode(ESelectionMode::Single)
		.OnGenerateRow(this, &SProductionWizard::OnGenerateMenuRow)
		.OnSelectionChanged(this, &SProductionWizard::OnUserMenuSelectionChanged);

	// Construct the list view for the production menu entries
	ProductionMenuListView = SNew(SListView<TSharedPtr<FProductionWizardMenuEntry>>)
		.ListItemsSource(&ProductionMenuEntries)
		.SelectionMode(ESelectionMode::Single)
		.OnGenerateRow(this, &SProductionWizard::OnGenerateMenuRow)
		.OnSelectionChanged(this, &SProductionWizard::OnProductionMenuSelectionChanged);

	// The initial selection upon opening the wizard is the first user menu entry (revision control)
	UserMenuListView->SetSelection(UserMenuEntries[0]);
	ContentPanelSlot->AttachWidget(UserMenuEntries[0]->Panel.ToSharedRef());

	return SNew(SBorder)
		.BorderImage(FCineAssemblyToolsStyle::Get().GetBrush("ProductionWizard.RecessedBackground"))
		.Padding(16.0f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 0.0f, 0.0f, 8.0f)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("UserSetupMenuGroup", "User Setup"))
						.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 0.0f, 0.0f, 8.0f)
				[
					UserMenuListView.ToSharedRef()
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 0.0f, 0.0f, 8.0f)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("ProductionSettingsMenuGroup", "Production Settings"))
						.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					ProductionMenuListView.ToSharedRef()
				]
		];
}

TSharedRef<ITableRow> SProductionWizard::OnGenerateMenuRow(TSharedPtr<FProductionWizardMenuEntry> MenuEntry, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FProductionWizardMenuEntry>>, OwnerTable)
		.ShowSelection(true)
		.Padding(FMargin(4.0f, 4.0f, 0.0f, 4.0f))
		.Content()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SImage).Image(MenuEntry->Icon.GetIcon())
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock).Text(MenuEntry->Label)
				]
		];
}

void SProductionWizard::OnUserMenuSelectionChanged(TSharedPtr<FProductionWizardMenuEntry> SelectedEntry, ESelectInfo::Type SelectInfo)
{
	if (SelectedEntry)
	{
		ProductionMenuListView->ClearSelection();
		ContentPanelSlot->AttachWidget(SelectedEntry->Panel.ToSharedRef());
	}
}

void SProductionWizard::OnProductionMenuSelectionChanged(TSharedPtr<FProductionWizardMenuEntry> SelectedEntry, ESelectInfo::Type SelectInfo)
{
	if (SelectedEntry)
	{
		UserMenuListView->ClearSelection();
		ContentPanelSlot->AttachWidget(SelectedEntry->Panel.ToSharedRef());
	}
}

#undef LOCTEXT_NAMESPACE
