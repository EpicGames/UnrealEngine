// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Views/SListView.h"

/** Structure for the list view of menu entires */
struct FProductionWizardMenuEntry
{
	/** Displayed menu entry text */
	FText Label;

	/** Displayed menu entry icon */
	FSlateIcon Icon;

	/** Widget to attach to the content panel slot when this menu entry is selected */
	TSharedPtr<SWidget> Panel;
};

/**
 * Main UI widget for the Production Wizard Tool
 */
class SProductionWizard : public SCompoundWidget
{
public:
	SProductionWizard() = default;

	SLATE_BEGIN_ARGS(SProductionWizard) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	/** Creates the menu on the left side of the production wizard */
	TSharedRef<SWidget> MakeMenuPanel();

	/** Creates the buttons on the bottom of the production wizard */
	TSharedRef<SWidget> MakeButtonsPanel();

	/** Generates a row in the menu list views */
	TSharedRef<ITableRow> OnGenerateMenuRow(TSharedPtr<FProductionWizardMenuEntry> MenuEntry, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback when one of the user menu entries is selected, which will cause its panel to be displayed on the right side of the production wizard */
	void OnUserMenuSelectionChanged(TSharedPtr<FProductionWizardMenuEntry> SelectedEntry, ESelectInfo::Type SelectInfo);

	/** Callback when one of the production menu entries is selected, which will cause its panel to be displayed on the right side of the production wizard */
	void OnProductionMenuSelectionChanged(TSharedPtr<FProductionWizardMenuEntry> SelectedEntry, ESelectInfo::Type SelectInfo);

	/** Cycles backwards through the available panels */
	FReply OnBackClicked();

	/** Cycles forwards through the available panels */
	FReply OnNextClicked();

	/** Adds User Settings and Production Settings extensions to the Menu Panel. */
	void MakeMenuExtensions();

private:
	/** Slots for the menu and menu entry content panels */
	SSplitter::FSlot* MenuPanelSlot = nullptr;
	SSplitter::FSlot* ContentPanelSlot = nullptr;

	/** List items sources for the menu list views */
	TArray<TSharedPtr<FProductionWizardMenuEntry>> UserMenuEntries;
	TArray<TSharedPtr<FProductionWizardMenuEntry>> ProductionMenuEntries;

	/** Menu list views, which allow the user to select a step in the production wizard */
	TSharedPtr<SListView<TSharedPtr<FProductionWizardMenuEntry>>> UserMenuListView;
	TSharedPtr<SListView<TSharedPtr<FProductionWizardMenuEntry>>> ProductionMenuListView;
};
