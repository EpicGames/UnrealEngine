// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRevisionControlPanel.h"

#include "CineAssemblyToolsStyle.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SRevisionControlPanel"

void SRevisionControlPanel::Construct(const FArguments& InArgs)
{
	TSharedRef<SLayeredImage> SourceControlIcon =
		SNew(SLayeredImage)
		.ColorAndOpacity(FSlateColor::UseForeground())
		.Image(FCineAssemblyToolsStyle::Get().GetBrush("Icons.RevisionControl"));

	SourceControlIcon->AddLayer(TAttribute<const FSlateBrush*>::CreateStatic(&SRevisionControlPanel::GetSourceControlIconBadge));

	ChildSlot
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
				.Padding(16.0f)
				[
					SNew(SVerticalBox)

					// Title
					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 0.0f, 0.0f, 4.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("RevisionControlTitle", "User Setup"))
								.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.TitleFont"))
						]

					// Heading
					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 0.0f, 0.0f, 4.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("RevisionControlHeading", "Revision Control"))
								.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.HeadingFont"))
						]

					// Info Text 1
					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 0.0f, 0.0f, 8.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("RevisionControlInfoText1", "Revision control helps you collaborate with your team members and back up changes to your project."))
								.AutoWrapText(true)
						]

					// Info Text 2
					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 0.0f, 0.0f, 16.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("RevisionControlInfoText2", "If your project is in a revision control system, use the settings here to connect the Unreal editor to your project's repository."))
								.AutoWrapText(true)
						]

					// Connect to Source Control Button
					+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Left)
						.Padding(0.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(SButton)
								.ContentPadding(FMargin(2.0f))
								.OnClicked_Lambda([]() -> FReply
									{
										// Opens the source control login dialog
										ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
										SourceControlModule.ShowLoginDialog(FSourceControlLoginClosed(), ELoginWindowMode::Modal, EOnLoginWindowStartup::PreserveProvider);
										return FReply::Handled();
									})
								[
									SNew(SHorizontalBox)

									+ SHorizontalBox::Slot()
										.AutoWidth()
										.Padding(0.0f, 0.0f, 4.0f, 0.0f)
										[
											SourceControlIcon
										]

									+ SHorizontalBox::Slot()
										.AutoWidth()
										[
											SNew(STextBlock)
												.Text_Lambda([]() -> FText
													{
														if (ISourceControlModule::Get().IsEnabled())
														{
															return LOCTEXT("ChangeRevisionControlSettings", "Change Revision Control Settings");
														}
														return LOCTEXT("ConnectToRevisionControl", "Connect to Revision Control");
													})
										]
								]
						]
				]
		];
}

const FSlateBrush* SRevisionControlPanel::GetSourceControlIconBadge()
{
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	if (SourceControlModule.IsEnabled())
	{
		if (!SourceControlModule.GetProvider().IsAvailable())
		{
			static const FSlateBrush* WarningBrush = FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Badges.RevisionControlWarning").GetIcon();
			return WarningBrush;
		}
		else
		{
			static const FSlateBrush* ConnectedBrush = FSlateIcon(FCineAssemblyToolsStyle::StyleName, "Badges.RevisionControlConnected").GetIcon();
			return ConnectedBrush;
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
