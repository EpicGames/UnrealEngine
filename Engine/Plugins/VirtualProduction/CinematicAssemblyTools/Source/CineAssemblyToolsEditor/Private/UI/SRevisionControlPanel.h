// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Widgets/DeclarativeSyntaxSupport.h"

/**
 * UI for the Revision Control panel in the Production Wizard
 */
class SRevisionControlPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRevisionControlPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	/** Returns the icon badge to indicate connection status */
	static const FSlateBrush* GetSourceControlIconBadge();
};
