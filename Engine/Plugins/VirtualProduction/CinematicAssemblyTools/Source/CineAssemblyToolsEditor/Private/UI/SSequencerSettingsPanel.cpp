// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerSettingsPanel.h"

#include "CineAssemblyToolsStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ProductionSettings.h"
#include "UI/SActiveProductionCombo.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SFrameRatePicker.h"

#define LOCTEXT_NAMESPACE "SSequencerSettingsPanel"

void SSequencerSettingsPanel::Construct(const FArguments& InArgs)
{
	UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();

	auto IsActiveProductionValid = []() -> bool
		{
			const UProductionSettings* const ProductionSettings = GetDefault<UProductionSettings>();
			TOptional<const FCinematicProduction> ActiveProduction = ProductionSettings->GetActiveProduction();
			return ActiveProduction.IsSet();
		};

	// Get a text representation of the active subsequence priority to display in the combo button below
	auto GetSubsequencePriorityText = [ProductionSettings]() -> FText
		{
			if (ProductionSettings->GetActiveSubsequencePriority() == ESubsequencePriority::TopDown)
			{
				return LOCTEXT("TopDownText", "Top Down");
			}
			return LOCTEXT("BottomUpText", "Bottom Up");
		};

	// Build the menu options for the subsequence priority combo button below
	FMenuBuilder SubsequencePriorityMenuBuilder(true, nullptr);
	{
		SubsequencePriorityMenuBuilder.AddMenuEntry(
			LOCTEXT("TopDownText", "Top Down"),
			LOCTEXT("TopDownText", "Top Down"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([ProductionSettings]() { ProductionSettings->SetActiveSubsequencePriority(ESubsequencePriority::TopDown); })),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		SubsequencePriorityMenuBuilder.AddMenuEntry(
			LOCTEXT("BottomUpText", "Bottom Up"),
			LOCTEXT("BottomUpText", "Bottom Up"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([ProductionSettings]() { ProductionSettings->SetActiveSubsequencePriority(ESubsequencePriority::BottomUp); })),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}

	ChildSlot
		[
			SNew(SVerticalBox)

				// Active Production Selector
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SActiveProductionCombo)
				]

				// Separator
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator)
						.Orientation(Orient_Horizontal)
						.Thickness(2.0f)
				]

				// Sequencer Settings Panel
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
						.Padding(16.0f)
						[
							SNew(SVerticalBox)

								// Title
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 4.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("SequencerSettingsTitle", "Production Settings"))
										.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.TitleFont"))
								]

								// Heading
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 4.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("SequencerSettingsHeading", "Sequencer Settings"))
										.Font(FCineAssemblyToolsStyle::Get().GetFontStyle("ProductionWizard.HeadingFont"))
								]

								// Info Text
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 16.0f)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("SequencerSettingsInfoText", "Configure basic settings that apply to all Level Sequences and Cinematic Assemblies."))
								]

								// Frame Rate setting
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 8.0f)
								[
									SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
										.FillWidth(0.3f)
										.VAlign(VAlign_Center)
										[
											SNew(STextBlock).Text(LOCTEXT("FrameRateText", "Frame Rate"))
										]

										+ SHorizontalBox::Slot()
										.FillWidth(0.7f)
										.VAlign(VAlign_Center)
										[
											SNew(SFrameRatePicker)
												.HasMultipleValues(false)
												.Value_Lambda([ProductionSettings]() { return ProductionSettings->GetActiveDisplayRate(); })
												.OnValueChanged_Lambda([ProductionSettings](FFrameRate NewFrameRate) { ProductionSettings->SetActiveDisplayRate(NewFrameRate); })
												.IsEnabled_Lambda(IsActiveProductionValid)
										]
								]

								// Start Frame setting
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 8.0f)
								[
									SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
										.FillWidth(0.3f)
										.VAlign(VAlign_Center)
										[
											SNew(STextBlock).Text(LOCTEXT("SequencerStartFrameText", "Sequencer Start Frame"))
										]

										+ SHorizontalBox::Slot()
										.FillWidth(0.7f)
										.VAlign(VAlign_Center)
										[
											SNew(SNumericEntryBox<int32>)
												.AllowSpin(false)
												.Value_Lambda([ProductionSettings]() { return ProductionSettings->GetActiveStartFrame(); })
												.OnValueCommitted_Lambda([ProductionSettings](int32 NewValue, ETextCommit::Type CommitType) { ProductionSettings->SetActiveStartFrame(NewValue); })
												.IsEnabled_Lambda(IsActiveProductionValid)
										]
								]

								// Subsequence Priority setting
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 8.0f)
								[
									SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
										.FillWidth(0.3f)
										.VAlign(VAlign_Center)
										[
											SNew(STextBlock).Text(LOCTEXT("SubsequencePriorityText", "Subsequence Priority"))
										]

										+ SHorizontalBox::Slot()
										.FillWidth(0.7f)
										.VAlign(VAlign_Center)
										[
											SNew(SComboButton)
												.IsEnabled_Lambda(IsActiveProductionValid)
												.MenuContent()
												[
													SubsequencePriorityMenuBuilder.MakeWidget()
												]
												.ButtonContent()
												[
													SNew(STextBlock)
														.Text_Lambda(GetSubsequencePriorityText)
														.Font(FAppStyle::Get().GetFontStyle("Normal"))
												]
										]
								]
						]
				]
		];
}

#undef LOCTEXT_NAMESPACE
