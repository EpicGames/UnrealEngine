// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Widgets/DeclarativeSyntaxSupport.h"

/**
 * UI for the Sequencer Settings panel in the Production Wizard
 */
class SSequencerSettingsPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSequencerSettingsPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
};
