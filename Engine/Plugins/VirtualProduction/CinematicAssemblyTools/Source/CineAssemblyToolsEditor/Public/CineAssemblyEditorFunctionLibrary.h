// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "AssetRegistry/AssetData.h"
#include "CineAssemblyFactory.h"
#include "NamingTokenData.h"

#include "MovieSceneSubAssemblySection.h"
#include "MovieSceneSubAssemblyTrack.h"

#include "CineAssemblyEditorFunctionLibrary.generated.h"

#define UE_API CINEASSEMBLYTOOLSEDITOR_API

class UCineAssembly;
class UCineAssemblySchema;
class UTexture2D;
class UWorld;

/**
 * When opening a level, these options tell us how to handle unsaved changes in the currently open level
 */
USTRUCT(BlueprintType)
struct FCineAssemblyLevelSaveOptions
{
	GENERATED_BODY()

	/** Whether the currently open level should be saved before opening a different level */
	UPROPERTY(BlueprintReadWrite, Category = "Cine Assembly Tools")
	bool bSaveCurrentLevel = true;

	/** Whether the user should be prompted before saving the current level */
	UPROPERTY(BlueprintReadWrite, Category = "Cine Assembly Tools", meta = (EditCondition = bSaveCurrentLevel))
	bool bPromptForUnsavedChanges = true;
};

/**
 * Library of functions that expose CineAssembly functionality to editor scripting
 */
UCLASS()
class UCineAssemblyEditorFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Opens a CineAssembly in Sequencer, and optionally opens its associated Level */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", meta = (AdvancedDisplay = "SaveOptions"))
	static UE_API void OpenAssembly(UCineAssembly* CineAssembly, bool bOpenAssociatedLevel = true, FCineAssemblyLevelSaveOptions SaveOptions = FCineAssemblyLevelSaveOptions());

	/**
	 * Opens the Level associated with the input CineAssembly (if it has one).
	 * Returns false only if the input assembly was invalid, or if its associated level was valid but could not be opened.
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", meta = (AdvancedDisplay = "SaveOptions"))
	static UE_API bool OpenAssociatedLevel(UCineAssembly* CineAssembly, FCineAssemblyLevelSaveOptions SaveOptions = FCineAssemblyLevelSaveOptions());

	/**
	 * Create a new CineAssembly asset using the input Schema, Level, and Metadata.
	 * It is important that any metadata required for resolving asset naming tokens is provided to this function so that the Assembly and SubAssemblies are all named correctly.
	 *
	 * @param Schema             The schema to initialize the new assembly from
	 * @param Level              Optional Level to associate with the new assembly
	 * @param ParentAssembly     Optional parent assembly reference
	 * @param Metadata           Optional metadata overrides to apply to the assembly (key/value string pairs)
	 * @param Path               The content browser path where the assembly should be created
	 * @param NameOverride       Optional name for the assembly. If empty, the default name defined by the schema is used.
	 *
	 * @return The newly created assembly, or nullptr if creation failed
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", meta = (AutoCreateRefTerm = "Metadata"))
	static UE_API UCineAssembly* CreateAssembly(UCineAssemblySchema* Schema, TSoftObjectPtr<UWorld> Level, TSoftObjectPtr<UCineAssembly> ParentAssembly, const TMap<FString, FString>& Metadata, const FString& Path, const FString& NameOverride = FString());

	/**
	 * Creates an in-memory (transient) CineAssembly object, with defaults configured from the input Schema, ready for further configuration.
	 * SubAssembly objects (as specified by the Schema) are also created in-memory and can be obtained by calling GetSubAssemblies().
	 * Once configuration is complete, call FinalizeConfiguredAssembly() to persist the Assembly (and SubAssemblies) as a real asset.
	 * Other persist APIs (duplicate/rename/save) on the transient CineAssembly are not supported.
	 *
	 * @param Schema             The schema to initialize the new assembly from
	 * @param Level              Optional Level to associate with the new assembly
	 * @param ParentAssembly     Optional parent assembly reference
	 * @param Metadata           Optional metadata overrides to apply to the assembly (key/value string pairs)
	 *
	 * @return The transient assembly, or nullptr if creation failed
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", meta = (AutoCreateRefTerm = "Metadata"))
	static UE_API UCineAssembly* CreateAssemblyToConfigure(UCineAssemblySchema* Schema, TSoftObjectPtr<UWorld> Level, TSoftObjectPtr<UCineAssembly> ParentAssembly, const TMap<FString, FString>& Metadata);

	/**
	 * Persists an in-memory, pre-configured CineAssembly object (typically returned by CreateAssemblyToConfigure()) to the specified path.
	 *
	 * @param ConfiguredAssembly     The transient CineAssembly object to persist
	 * @param Path                   The content browser path where the assembly should be created
	 * @param NameOverride           Optional name for the assembly. If empty, the configured name (tokens resolved) is used.
	 *
	 * @return The persisted CineAssembly asset, or nullptr if the operation failed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	static UE_API UCineAssembly* FinalizeConfiguredAssembly(UCineAssembly* ConfiguredAssembly, const FString& Path, const FString& NameOverride = FString());

	/**
	 * Duplicates a CineAssembly asset at the specified path.
	 *
	 * @param SourceAssembly                   The assembly to duplicate
	 * @param DuplicatePath                    The content browser path where the duplicate should be created
	 * @param Metadata                         Optional metadata overrides to apply to the assembly (key/value string pairs)
	 * @param bDuplicateSubsequences           If true, recursively duplicate all descendant subsequences. If false, subsequences will reference the same sequences as the source Assembly.
	 * @param bDuplicateAssociatedAssets       If true, the schema-defined associated assets are duplicated from the source assembly and its managed SubAssemblies.
	 * @param NameOverride                     Optional name for the duplicate. If empty, the source assembly's name is used.
	 * @param ExternalAssetPreference          Preference for handling assets (subsequences, sublevels, etc.) that live outside the source assembly's content tree
	 *
	 * @return The duplicated assembly, or nullptr if duplication failed
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", meta = (AutoCreateRefTerm = "Metadata", AdvancedDisplay = "ExternalAssetPreference"))
	static UE_API UCineAssembly* DuplicateAssembly(UCineAssembly* SourceAssembly, const FString& DuplicatePath, const TMap<FString, FString>& Metadata, bool bDuplicateSubsequences = true, bool bDuplicateAssociatedAssets = true, const FString& NameOverride = FString(), EDuplicateExternalAssetPreference ExternalAssetPreference = EDuplicateExternalAssetPreference::DuplicateIntoAssemblyFolder);

	/**
	 * Creates an in-memory (transient) duplicate of the source CineAssembly, ready for further configuration.
	 * If bDuplicateSubsequences is true, duplicate SubAssembly objects are also created in-memory and can be obtained by calling GetSubAssemblies().
	 * Once configuration is complete, call FinalizeDuplicateAssembly() to persist the duplicate Assembly (and SubAssemblies) as a real asset. 
	 * Other persist APIs (duplicate/rename/save) on the transient CineAssembly are not supported.
	 *
	 * @param SourceAssembly            The assembly to duplicate
	 * @param Metadata                  Optional metadata overrides to apply to the assembly (key/value string pairs)
	 * @param bDuplicateSubsequences    If true, each managed SubAssembly (as specified by the Schema) is also duplicated in-memory. Additional subsequences can be duplicated during FinalizeDuplicateAssembly().
	 *                                  If false, the duplicate assembly will maintain a reference to source assembly's original SubAssemblies.
	 *
	 * @return The transient duplicate, or nullptr if duplication failed
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", meta = (AutoCreateRefTerm = "Metadata"))
	static UE_API UCineAssembly* DuplicateAssemblyToConfigure(UCineAssembly* SourceAssembly, const TMap<FString, FString>& Metadata, bool bDuplicateSubsequences = true);

	/**
	 * Persists an in-memory, pre-configured CineAssembly duplicate (typically returned by DuplicateAssemblyToConfigure()) to the specified path,
	 * and optionally duplicates the source assembly's unmanaged subsequences and associated assets.
	 *
	 * @param ConfiguredDuplicateAssembly    The transient assembly to persist
	 * @param DuplicatePath                  The content browser path where the duplicate should be created
	 * @param NameOverride                   Optional name for the assembly. If empty, the configured name (tokens resolved) is used.
	 * @param bDuplicateSubsequences         If true, unmanaged subsequences are duplicated using ExternalAssetPreference.
	 * @param bDuplicateAssociatedAssets     If true, the schema-defined associated assets are duplicated from the source assembly and its managed SubAssemblies.
	 * @param ExternalAssetPreference        Preference for handling assets (subsequences, sublevels, etc.) that live outside the source assembly's content tree
	 *
	 * @return The now-persisted assembly, or nullptr if the input was not a transient assembly or the path was invalid
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", meta = (AdvancedDisplay = "ExternalAssetPreference"))
	static UE_API UCineAssembly* FinalizeDuplicateAssembly(UCineAssembly* ConfiguredDuplicateAssembly, const FString& DuplicatePath, const FString& NameOverride = FString(), bool bDuplicateSubsequences = true, bool bDuplicateAssociatedAssets = true, EDuplicateExternalAssetPreference ExternalAssetPreference = EDuplicateExternalAssetPreference::DuplicateIntoAssemblyFolder);

	/**
	 * Creates a new CineAssemblySchema asset at the specified path.
	 *
	 * @param SchemaName              The name of the schema, used as the "assembly type" for assemblies made from it
	 * @param AssetPath               The content browser path where the schema should be created
	 * @param DefaultAssemblyName     The default name pattern for assemblies created from this schema
	 * @param RelativeAssemblyPath    The default path (relative to the content root) where assemblies created from this schema should be placed
	 * @param Description             A user-facing text description of the schema
	 * @param ParentSchema            Restricts assemblies made from this schema to only allow setting the Parent Assembly to an Assembly with this Parent Schema
	 * @param ThumbnailImage          The thumbnail image to use for this schema and assemblies built from it
	 * @param bIsDataOnly             Mark this schema as data-only
	 * @param bIsHidden               Hide this schema from the UI when creating new assemblies
	 *
	 * @return The new schema, or nullptr if creation failed
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", meta = (AdvancedDisplay = "Description, ParentSchema, ThumbnailImage, bIsDataOnly, bIsHidden"))
	static UE_API UCineAssemblySchema* CreateSchema(
		const FString& SchemaName,
		const FString& AssetPath,
		const FString& DefaultAssemblyName = FString(),
		const FString& RelativeAssemblyPath = FString(),
		const FString& Description = FString(),
		UCineAssemblySchema* ParentSchema = nullptr,
		UTexture2D* ThumbnailImage = nullptr,
		bool bIsDataOnly = false,
		bool bIsHidden = false
	);

	/**
	 * Adds a new template SubAssembly track and section to a Schema's template sequence.
	 * When an Assembly is created from this Schema, a new sequence will be initialized from the provided template object.
	 *
	 * @param Schema             The schema whose template sequence will be modified
	 * @param TrackType          Whether this SubAssembly represents a Shot track or Subsequence track
	 * @param SubAssemblyName    The name for the new SubAssembly sequence
	 * @param Label              The semantic label for the new section. If None (default), a unique default label is auto-assigned.
	 * @param RelativePath       The path (relative to the assembly root) where the new SubAssembly should be created
	 * @param TemplateObject     The template object (UCineAssemblySchema, UCineAssembly, or ULevelSequence) to use when initializing the SubAssembly
	 *
	 * @return The new SubAssemblySection, or null if creation failed
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	static UE_API UMovieSceneSubAssemblySection* AddSubAssemblyTemplate(
		UCineAssemblySchema* Schema,
		ESubAssemblyTrackType TrackType = ESubAssemblyTrackType::SubsequenceTrack,
		const FString& SubAssemblyName = FString(),
		FName Label = NAME_None,
		const FString& RelativePath = FString(),
		UPARAM(meta = (AllowedClasses = "/Script/LevelSequence.LevelSequence, /Script/CineAssemblyTools.CineAssembly, /Script/CineAssemblyTools.CineAssemblySchema"))
		UObject* TemplateObject = nullptr);

	/**
	 * Adds a new reference SubAssembly track and section to a Schema's template sequence.
	 * When an Assembly is created from this Schema, the section will reference the provided sequence directly.
	 *
	 * @param Schema             The schema whose template sequence will be modified
	 * @param TrackType          Whether this SubAssembly represents a Shot track or Subsequence track
	 * @param Sequence           The existing sequence (UCineAssembly or ULevelSequence) to reference
	 *
	 * @return The new SubAssemblySection, or null if creation failed
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	static UE_API UMovieSceneSubAssemblySection* AddSubAssemblyReference(
		UCineAssemblySchema* Schema,
		ESubAssemblyTrackType TrackType = ESubAssemblyTrackType::SubsequenceTrack,
		UMovieSceneSequence* Sequence = nullptr);

	/**
	 * Adds a new Associated Asset descriptor to a Schema's template sequence.
	 * When an Assembly is created from this Schema, the CineAssembly factory will create an asset of the specified class at the specified relative path, and include it in the Assembly's asset tree.
	 *
	 * EXPERIMENTAL: The CinematicAssemblyTools plugin officially supports UWorld for associated assets through the CineAssemblySchema UI. Associating assets of other class types may work, but some are untested.
	 * 
	 * Note: Only asset classes are supported. Non-asset classes, such as AActor or UActorComponent, will be rejected at runtime.
	 * Note: If AssetClass is UCineAssembly, the call is routed to AddSubAssemblyTemplate and the returned AssetID will be invalid. Consider calling AddSubAssemblyTemplate instead.
	 *
	 * @param Schema          The schema whose template sequence will be modified
	 * @param AssetClass      The class of asset to create alongside each assembly. Only asset classes are supported. Non-asset classes, such as AActor or UActorComponent, will be rejected at runtime.
	 * @param AssetName       Optional template name for the asset. If empty, the factory's default asset name is used.
	 * @param Label           Optional semantic label. If empty, a unique label is auto-assigned from the asset's display name.
	 * @param RelativePath    Optional path, relative to the assembly root, where the asset should be created
	 * @param TemplateAsset   Optional template asset to duplicate when the factory creates this descriptor's asset. Must match AssetClass. If null, a new asset of AssetClass is created instead.
	 *
	 * @return The stable AssetID for the new descriptor. The AssetID will be invalid if the operation failed, or if the call was routed to AddSubAssemblyTemplate (see note above).
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", meta = (ScriptMethod))
	static UE_API FGuid AddAssociatedAsset(
		UCineAssemblySchema* Schema,
		TSubclassOf<UObject> AssetClass,
		const FString& AssetName = FString(),
		FName Label = NAME_None,
		const FString& RelativePath = FString(),
		UObject* TemplateAsset = nullptr);

	/** Removes an Associated Asset descriptor from a Schema's template sequence by AssetID. */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", meta = (ScriptMethod))
	static UE_API void RemoveAssociatedAsset(UCineAssemblySchema* Schema, FGuid AssetID);

	/** Returns a copy of the Associated Asset descriptor with the given AssetID on the Schema's template sequence. To mutate, call one of the appropriate setter functions with the descriptor's AssetID. */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", DisplayName = "Get Associated Asset Desc", meta = (ScriptMethod = "GetAssociatedAssetDesc"))
	static UE_API FAssemblyAssociatedAssetDesc GetSchemaAssociatedAssetDesc(UCineAssemblySchema* Schema, FGuid AssetID);

	/** Returns a copy of the Associated Asset descriptor with the given AssetID on the Assembly. To mutate, call one of the appropriate setter functions with the descriptor's AssetID. */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", DisplayName = "Get Associated Asset Desc", meta = (ScriptMethod = "GetAssociatedAssetDesc"))
	static UE_API FAssemblyAssociatedAssetDesc GetAssemblyAssociatedAssetDesc(UCineAssembly* Assembly, FGuid AssetID);

	/** Returns copies of all Associated Asset descriptors on the Schema's template sequence. To mutate one, call one of the appropriate setter functions with the descriptor's AssetID. */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", DisplayName = "Get Associated Asset Descs", meta = (ScriptMethod = "GetAssociatedAssetDescs"))
	static UE_API TArray<FAssemblyAssociatedAssetDesc> GetSchemaAssociatedAssetDescs(UCineAssemblySchema* Schema);

	/** Returns copies of all Associated Asset descriptors on the Assembly. To mutate one, call one of the appropriate setter functions with the descriptor's AssetID. */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", DisplayName = "Get Associated Asset Descs", meta = (ScriptMethod = "GetAssociatedAssetDescs"))
	static UE_API TArray<FAssemblyAssociatedAssetDesc> GetAssemblyAssociatedAssetDescs(UCineAssembly* Assembly);

	/** Sets the AssetClass of the Associated Asset descriptor matching the AssetID in the Schema's template sequence. Schema-only because AssetClass is a structural property shared across all assemblies. */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", meta = (ScriptMethod))
	static UE_API void SetAssociatedAssetClass(UCineAssemblySchema* Schema, FGuid AssetID, TSubclassOf<UObject> NewAssetClass);

	/** Sets the RelativePath of the Associated Asset descriptor matching the AssetID in the Schema's template sequence. Schema-only because RelativePath is a structural property shared across all assemblies. */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", meta = (ScriptMethod))
	static UE_API void SetAssociatedAssetRelativePath(UCineAssemblySchema* Schema, FGuid AssetID, const FString& NewRelativePath);

	/** 
	 * Sets the TemplateAsset of the Associated Asset descriptor matching the AssetID in the Schema's template sequence. Schema-only because TemplateAsset is a structural property shared across all assemblies.
	 * Note: Operation will fail if the input TemplateAsset does not match the existing AssetClass filter of the Associated Asset descriptor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", meta = (ScriptMethod))
	static UE_API void SetAssociatedAssetTemplate(UCineAssemblySchema* Schema, FGuid AssetID, UObject* NewTemplateAsset);

	/** Sets the AssetName of the Associated Asset descriptor matching the AssetID in the Schema's template sequence. */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", DisplayName = "Set Associated Asset Name", meta = (ScriptMethod = "SetAssociatedAssetName"))
	static UE_API void SetSchemaAssociatedAssetName(UCineAssemblySchema* Schema, FGuid AssetID, const FString& NewAssetName);

	/** Sets the Label of the Associated Asset descriptor matching the AssetID in the Schema's template sequence. */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", DisplayName = "Set Associated Asset Label", meta = (ScriptMethod = "SetAssociatedAssetLabel"))
	static UE_API void SetSchemaAssociatedAssetLabel(UCineAssemblySchema* Schema, FGuid AssetID, FName NewLabel);

	/**
	 * Sets the AssetName of the Associated Asset descriptor matching the AssetID in the Assembly.
	 * Note: Operation will fail if descriptor's asset has already been created.
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", DisplayName = "Set Associated Asset Name", meta = (ScriptMethod = "SetAssociatedAssetName"))
	static UE_API void SetAssemblyAssociatedAssetName(UCineAssembly* Assembly, FGuid AssetID, const FString& NewAssetName);

	/** Sets the Label of the Associated Asset descriptor matching the AssetID in the Assembly. */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", DisplayName = "Set Associated Asset Label", meta = (ScriptMethod = "SetAssociatedAssetLabel"))
	static UE_API void SetAssemblyAssociatedAssetLabel(UCineAssembly* Assembly, FGuid AssetID, FName NewLabel);

	/**
	 * Sets the bShouldCreate flag of the Associated Asset descriptor matching the AssetID in the Assembly.
	 * Note: This operation has no effect if the descriptor's asset has already been created.
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools", DisplayName = "Set Associated Asset Should Create", meta = (ScriptMethod = "SetAssociatedAssetShouldCreate"))
	static UE_API void SetAssemblyAssociatedAssetShouldCreate(UCineAssembly* Assembly, FGuid AssetID, bool bShouldCreate);

	/** Get the default asset name for a new Associated Asset of the given class */
	static UE_API FString GetDefaultAssetNameForClass(const UClass* AssetClass);

	/** Get the default unique Label for a new Associated Asset of the given class on the given Schema */
	static UE_API FName MakeDefaultAssociatedAssetLabel(const UCineAssemblySchema* Schema, const UClass* AssetClass);

	/**
	 * Evaluates a tokenized string using the input Assembly for context. Always considers tokens in the "cat" namespace.
	 *
	 * @param TokenString    The template string containing tokens to resolve
	 * @param Assembly       The assembly context used to evaluate tokens
	 *
	 * @return The full FNamingTokenResultData results, including the evaluated text and per-token information
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	static UE_API FNamingTokenResultData EvaluateTokenString(const FString& TokenString, UCineAssembly* Assembly);

	/**
	 * Constructs a CineAssembly Naming Token context object for the input Assembly.
	 *
	 * @param Assembly    The assembly context used to evaluate tokens
	 *
	 * @return The new context object which can be passed to the NamingTokensEngineSubsystem's EvaluateTokenString function.
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	static UE_API UObject* BuildTokenContext(UCineAssembly* Assembly);

	/**
	 * Returns asset data for every CineAssembly asset created from the input Schema.
	 * If the input is null, this function returns all CineAssembly assets that have no schema set.
	 *
	 * @param Schema    The schema to search for, or null to find assemblies with no schema
	 *
	 * @return Asset data for every assembly whose schema matches the input
	 */
	UFUNCTION(BlueprintCallable, Category = "Cine Assembly Tools")
	static UE_API TArray<FAssetData> FindAssembliesBySchema(const UCineAssemblySchema* Schema);
};

#undef UE_API
