// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"

#include "CineAssemblyEditorSubsystem.generated.h"

#define UE_API CINEASSEMBLYTOOLSEDITOR_API

class UCineAssembly;
class UCineAssemblySchema;

/** Broadcast when a CineAssembly asset has been created */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnCineAssemblyCreatedDynamic, UCineAssembly*, NewAssembly);

/** Broadcast when a CineAssemblySchema asset has been created */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnCineAssemblySchemaCreatedDynamic, UCineAssemblySchema*, NewSchema);

/** Broadcast when a CineAssembly asset has been duplicated. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnCineAssemblyDuplicatedDynamic, UCineAssembly*, Duplicate, UCineAssembly*, Source);

/** Broadcast when adding or editing a metadata value on a CineAssembly asset */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnCineAssemblyMetadataChangedDynamic, UCineAssembly*, Assembly, const FString&, Key);

/**
 * Editor Subsystem for CineAssemblyTools
 */
UCLASS(MinimalAPI)
class UCineAssemblyEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	//~ Begin UEditorSubsystem interface
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	//~ End UEditorSubsystem interface

	/**
	 * Broadcast when a CineAssembly asset has been created
	 * Note: This event may broadcast multiple times if the CineAssembly creates any SubAssembly assets
	 */
	UPROPERTY(BlueprintAssignable, Category = "Cine Assembly Tools")
	FOnCineAssemblyCreatedDynamic OnAssemblyCreated;

	/** Broadcast when a CineAssemblySchema asset has been created */
	UPROPERTY(BlueprintAssignable, Category = "Cine Assembly Tools")
	FOnCineAssemblySchemaCreatedDynamic OnSchemaCreated;

	/**
	 * Broadcast when a CineAssembly asset has been duplicated.
	 * Note: OnAssemblyCreated will also be broadcast.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Cine Assembly Tools")
	FOnCineAssemblyDuplicatedDynamic OnAssemblyDuplicated;

	/** Broadcast when adding or editing a metadata value on a CineAssembly asset */
	UPROPERTY(BlueprintAssignable, Category = "Cine Assembly Tools")
	FOnCineAssemblyMetadataChangedDynamic OnAssemblyMetadataChanged;

private:
	FDelegateHandle OnAssemblyMetadataChangedHandle;
};

#undef UE_API
