// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Textures/SlateIcon.h"

class SWidget;

/** Callback for constructing a new SWidget. */
DECLARE_DELEGATE_RetVal(TSharedRef<SWidget>, FGetWidget);

/**
 * Interface for CineAssemblyToolsEditor module
 */
class ICineAssemblyToolsEditorModule : public IModuleInterface
{
public:
	/** Get the implementation instance for the module. */
	static ICineAssemblyToolsEditorModule& Get()
	{
		return FModuleManager::Get().LoadModuleChecked<ICineAssemblyToolsEditorModule>("CineAssemblyToolsEditor");
	}

	/** Gets if this module is loaded. */
	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("CineAssemblyToolsEditor");
	}

	/** Virtual destructor. */
	virtual ~ICineAssemblyToolsEditorModule() = default;

	/** 
	 * Register a new production extension struct type. 
	 * 
	 * An ExtendedData production extension type is a USTRUCT type that is registered as additional data to each Production object.
	 * The struct must be a standard USTRUCT and registered to the productions using:
	 * 
	 * ```
	 * USTRUCT()
	 * struct FMyStruct 
	 * { 
	 *		GENERATED_BODY()
	 * 
	 *		UPROPERTY(EditAnywhere, Category=Default)
	 *		int MyIntMember;
	 * }
	 * 
	 * {
	 *		ICineAssemblyToolsEditorModule& CatEdModule = ICineAssemblyToolsEditorModule::Get();
	 *		CatEdModule.RegisterProductionExtension(*FMyStruct::StaticStruct());
	 * }
	 * ```
	 * 
	 * Once registered, each production will have its own instance of the extended data struct type.
	 * See FCinematicProduction for more on how to access and modify the data.
	 */
	virtual void RegisterProductionExtension(const UScriptStruct& DataScriptStruct) = 0;

	/** Unregister a Production extension struct type. */
	virtual void UnregisterProductionExtension(const UScriptStruct& DataScriptStruct) = 0;

	/**
	 * Register a customization for the Production Wizard for the given extended data struct type.
	 * 
	 * A callback can be provided to construct a custom widget, or leave unbound if the default details view is desired. You can then override the label,
	 * icon and if the Production Selection combo box is shown in the wizard for the data tab. 
	 * 
	 * If bHideInWizard is given as true, then the tab is hidden from the Production Wizard completely for the given struct type.
	 * 
	 * The extended data struct type should already have been registered as a Production extension using RegisterProductionExtension.
	 */
	virtual void RegisterProductionWizardCustomization(const UScriptStruct& ForDataScriptStruct,
		FGetWidget MakeCustomWidget, TAttribute<FText> Label, TAttribute<FSlateIcon> Icon, bool bShowProductionSelection, bool bHideInWizard) = 0;

	/** Unregister the registered customization for the given extended data struct type. */
	virtual void UnregisterProductionWizardCustomization(const UScriptStruct& ForDataScriptStruct) = 0;

	/** 
	 * Register a custom User Setting tab to show in the Production wizard by providing a callback to create the SWidget to show, as well as customizations to label and icon to display.
	 * 
	 * If no label is provided, Name will be used as the display label on the Production Wizard interface.
	 */
	virtual void RegisterProductionWizardUserSettings(const FName Name, FGetWidget MakeCustomWidget,
		TAttribute<FText> Label, TAttribute<FSlateIcon> Icon) = 0;

	/** Unregister the custom User Settings tab from the Production Wizard with the given name. */
	virtual void UnregisterProductionWizardUserSettings(const FName Name) = 0;
};
