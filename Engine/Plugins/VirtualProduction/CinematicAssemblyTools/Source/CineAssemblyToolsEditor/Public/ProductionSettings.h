// Copyright Epic Games, Inc. All Rights Reserved.
#pragma  once

#include "Engine/DeveloperSettings.h"

#include "Misc/FrameRate.h"
#include "ScopedModifyProductionExtendedData.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructView.h"

#include "ProductionSettings.generated.h"

#define UE_API CINEASSEMBLYTOOLSEDITOR_API

DECLARE_MULTICAST_DELEGATE(FOnProductionListChanged);
DECLARE_MULTICAST_DELEGATE(FOnActiveProductionChanged);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnProductionExtendedDataExported, const FCinematicProduction& /* SenderProduction */, const UScriptStruct* /* ExportedStruct */);

/**
 * Options for determining the hierarchical bias of subsequences
 */
UENUM()
enum class ESubsequencePriority : uint8
{
	TopDown,
	BottomUp
};

/**
 * Properties of a folder in the production's template folder hierarchy
 */
USTRUCT()
struct FFolderTemplate
{
	GENERATED_BODY()

	UPROPERTY(config, EditAnywhere, Category = "Default")
	FString InternalPath;

	UPROPERTY(config, EditAnywhere, Category = "Default")
	bool bCreateIfMissing = true;
};

/**
 * Collection of production settings to override the project/editor behavior
 */
USTRUCT(BlueprintType)
struct FCinematicProduction
{
	GENERATED_BODY()

	friend class FScopedModifyProductionExtendedData;

public:
	FCinematicProduction();

	/** Unique ID of the production */
	UPROPERTY(BlueprintReadOnly, Category = "Default", meta = (IgnoreForMemberInitializationTest))
	FGuid ProductionID;

	/** Production Name */
	UPROPERTY(config, EditAnywhere, BlueprintReadOnly, Category = "Default")
	FString ProductionName;

	/** The default frame rate to set for new Level Sequences */
	UPROPERTY(config, EditAnywhere, BlueprintReadOnly, Category = "Default")
	FFrameRate DefaultDisplayRate;

	/** The default frame number (using the default frame rate) that new Level Sequences should start at */
	UPROPERTY(config, EditAnywhere, BlueprintReadOnly, Category = "Default")
	int32 DefaultStartFrame;

	/** Controls whether subsequences override parent sequences, or vice versa */
	UPROPERTY(config, EditAnywhere, BlueprintReadOnly, Category = "Default")
	ESubsequencePriority SubsequencePriority = ESubsequencePriority::BottomUp;

	/** List of Naming Token namespaces that should not be evaluated */
	UPROPERTY(config, EditAnywhere, BlueprintReadOnly, Category = "Default")
	TSet<FString> NamingTokenNamespaceDenyList;

	/** List of default names for specific asset types */
	UPROPERTY(config, VisibleAnywhere, BlueprintReadOnly, Category = "Default")
	TMap<TObjectPtr<const UClass>, FString> DefaultAssetNames;

	/** List of folder paths that represent a template folder hierarchy to be used for this production */
	UPROPERTY(config, EditAnywhere, Category = "Default")
	TArray<FFolderTemplate> TemplateFolders;

	//~ Extended Data
public:
	/**
	* Attempts to load Extended Data of the given type, and update the data in the production.
	* This function always loads the data from ExtendedDataConfig if found, regardless of it has already been loaded or not.
	* 
	* The given Struct type must have been registered with ICineAssemblyToolsEditorModule as a ProductionExtension.
	*
	* Returns a pointer to the loaded struct or nullptr if the type is not registered with ICineAssemblyToolsEditorModule.
	* The returned pointer is only valid until the next allocation of ExtendedData.
	*/
	UE_API FInstancedStruct* LoadExtendedData(const UScriptStruct& ForStruct);

	/**
	* Attempts to find Extended Data of the given type, loading it if required and updating the data in the production. If the data is already loaded,
	* then the existing data is returned and not loaded.
	* 
	* The given Struct type must have been registered with ICineAssemblyToolsEditorModule as a ProductionExtension.
	* 
	* To modify the returned FInstancedStruct and ensure that it is correctly re-exported back into the production after modification so that it is correctly saved into
	* the config, use FScopedModifyProductionExtendedData.
	* 
	* e.g.
	*
	* ```
	* if (FInstancedStruct* Data = Production.FindOrLoadExtendedData(*FMyStruct::StaticStruct()))
	* {
	*		FScopedModifyProductionExtendedData ModifyGuard(Production, FMyStruct::StaticStruct());
	*		FMyStruct& MyStructRef = Data->GetMutable();
	* 
	*		...
	*		// Modify MyStructRef
	*		...
	* }
	* ```
	* 
	* Returns a pointer to the loaded struct or nullptr if the type is not registered with ICineAssemblyToolsEditorModule.
	* The returned pointer is only valid until the next allocation of ExtendedData.
	*/
	UE_API FInstancedStruct* FindOrLoadExtendedData(const UScriptStruct& ForStruct);

	/**
	* Attempts to find existing Extended Data of the given type. If the data does not exist or has not been loaded, returns a nullptr.
	* Use FindOrLoadExtendedData if you are unsure if the data has already been loaded, or ensure LoadExtendedData has already been called for this struct type.
	* 
	* The returned pointer is null if the extended data was not found, and if found, is only valid until the next allocation of ExtendedData.
	*/
	UE_API const FInstancedStruct* FindExtendedData(const UScriptStruct& ForStruct) const;

	/**
	 * Export the extended data ready to save to Config by exporting the given struct type into ExtendedDataConfig.
	 * The ExtendedDataConfig map is not emptied as part of this process.
	 */
	UE_API void ExportExtendedData(const UScriptStruct& ForData);

	/** Delegate called when the one or more ExtendedData structs are exported to ExtendedDataConfig. */
	FOnProductionExtendedDataExported& OnExtendedDataExported() { return ExtendedDataExported; }

	/** Helper function to get the compile time checked member name of the ExtendedData property. */
	UE_API static const FName GetExtendedDataMemberName();

protected:
	/**
	 * Instance specific ExtendedData. This is saved to the config by exporting to text.
	 * Ensure modifications are guarded by a FScopedModifyProductionExtendedData instance or a call to ExportExtendedData.
	 */
	UPROPERTY(Transient, SkipSerialization, EditAnywhere, Category = "Default", NoClear, EditFixedSize, meta = (DisplayThumbnail = "false"))
	TMap<FSoftObjectPath, FInstancedStruct> ExtendedData;

private:
	/**
	 * Loads all registered ExtendedData from config loaded data and adds any additionally registered extended data types.
	 * Exports the data back into the ExtendedDataConfig struct once loading is complete.
	 */
	void LoadAllExtendedData();

	/** Import the text as ExtendedData. Returns the the newly created instance if import was successful. */
	TOptional<FInstancedStruct*> ImportExtendedData(const FString& ImportText);

	/** Export the InstancedStruct into ExtendedDataConfig. */
	void ExportExtendedData(const FInstancedStruct& InData);

	/** Uniquely add an instance of the given struct type, returning a reference to either the existing data or the newly created data. */
	FInstancedStruct& AddUniqueExtendedData(const UScriptStruct& ScriptStruct);

	/** Broadcast the ExtendedDataExported if bBlockNotifyExtendedDataExported is false with the given exported struct type. */
	void NotifyExtendedDataExported(const UScriptStruct* ExportedStruct);

	/** Delegate called when the one or more ExtendedData structs are exported to ExtendedDataConfig. */
	FOnProductionExtendedDataExported ExtendedDataExported;

	/** Whether the ExtendedDataExported delegate should be blocked or triggered when the ExtendedData is exported. */
	bool bBlockNotifyExtendedDataExported = false;

	/** 
	 * Property used to serialize the extended data to configuration without requiring the types are present.
	 * 
	 * The text representation is stored independently of the live ExtendedData to handle ExtendedData types that may not be able
	 * to be loaded due to coming from unloaded or missing modules.
	 * 
	 * The map is keyed with a FSoftObjectPath which represents the PathName of the data UScriptStruct.
	 */
	UPROPERTY(config)
	TMap<FSoftObjectPath, FString> ExtendedDataConfig;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnProductionCreated, const FCinematicProduction&, Production);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnProductionSettingsChanged, const FCinematicProduction&, Production);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnActiveProductionSet, const FCinematicProduction&, ActiveProduction, bool, bIsValid);

/**
 * Cinematic Production Settings
 */
UCLASS(config=Engine, defaultconfig)
class UProductionSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UProductionSettings() = default;

	/** Maximum number of characters allowed in a Production Name */
	UE_API constexpr static int32 ProductionNameMaxLength = 256;

	//~ Begin UDeveloperSettings overrides
	virtual FName GetContainerName() const override;
	virtual FName GetCategoryName() const override;
	virtual FName GetSectionName() const override;
	//~ End UDeveloperSettings overrides

	//~ Begin UObject overrides
	virtual void PostInitProperties() override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	virtual void PostReloadConfig(FProperty* PropertyThatWasLoaded) override;
	virtual void PostEditUndo() override;
	//~ End UObject overrides

	/** The customization is allowed access to the private properties */
	friend class FProductionSettingsCustomization;

	/** Returns a copy of the production list */
	const TArray<FCinematicProduction> GetProductions() const;

	/** Returns a copy of the production matching the input production ID (if it exists) */
	TOptional<const FCinematicProduction> GetProduction(FGuid ProductionID) const;

	/** Adds a new empty production to the list */
	void AddProduction();

	/** Adds the input production to the list */
	void AddProduction(const FCinematicProduction& ProductionToAdd);

	/** Adds a duplicate of the input production to the list */
	void DuplicateProduction(FGuid ProductionID);

	/** Removes the production matching the input ID from the list */
	void DeleteProduction(FGuid ProductionID);

	/** Renames the production matching the input ID */
	void RenameProduction(FGuid ProductionID, FString NewName);

	/** Returns a copy of the active production (if there is one) */
	TOptional<const FCinematicProduction> GetActiveProduction() const;

	/** Returns the unique ID of the active production */
	UE_API FGuid GetActiveProductionID() const;

	/** Sets the active production based on the input production ID */
	void SetActiveProduction(FGuid ProductionID);

	/** Returns true if the input production ID matches the ID of the active production */
	bool IsActiveProduction(FGuid ProductionID) const;

	/** Returns the DefaultDisplayRate of the active production, or the underlying level sequence setting if there is no active production */
	FFrameRate GetActiveDisplayRate() const;

	/** Returns the DefaultStartFrame of the active production, or the value based on the underlying movie scene tools setting if there is no active production */
	int32 GetActiveStartFrame() const;

	/** Returns the SubsequencePriority of the active production, or the default config value if there is no active production */
	ESubsequencePriority GetActiveSubsequencePriority() const;

	/** Sets the DefaultDisplayRate of the production matching the input ID */
	void SetDisplayRate(FGuid ProductionID, FFrameRate DisplayRate);

	/** Sets the DefaultStartFrame of the production matching the input ID */
	void SetStartFrame(FGuid ProductionID, int32 StartFrame);

	/** Sets the SubsequencePriority of the production matching the input ID */
	void SetSubsequencePriority(FGuid ProductionID, ESubsequencePriority Priority);

	/** Adds a Naming Token namespace to the DenyList of the production matching the input ID */
	void AddNamespaceToDenyList(FGuid ProductionID, const FString& Namespace);

	/** Removes a Naming Token namespace from the DenyList of the production matching the input ID */
	void RemoveNamespaceFromDenyList(FGuid ProductionID, const FString& Namespace);

	/** Adds a new entry into the DefaultAssetNames map of the production matching the input ID */
	void AddAssetNaming(FGuid ProductionID, const UClass* AssetClass, const FString& DefaultName);

	/** Removes an entry from the DefaultAssetNames map of the production matching the input ID */
	void RemoveAssetNaming(FGuid ProductionID, const UClass* AssetClass);

	/** Add a new path to the input production's list of template folders */
	void AddTemplateFolder(FGuid ProductionID, const FString& Path, bool bCreateIfMissing=true);

	/** Removes a path from the input production's list of template folders */
	void RemoveTemplateFolder(FGuid ProductionID, const FString& Path);

	/** Sets the input production's template folder hierarchy to the input array of template folders */
	void SetTemplateFolderHierarchy(FGuid ProductionID, const TArray<FFolderTemplate>& TemplateHierarchy);

	/** Sets the DefaultDisplayRate of the active production */
	void SetActiveDisplayRate(FFrameRate DisplayRate);

	/** Sets the DefaultStartFrame of the active production */
	void SetActiveStartFrame(int32 StartFrame);

	/** Sets the SubsequencePriority of the active production */
	void SetActiveSubsequencePriority(ESubsequencePriority Priority);

	/** Returns a new unique production name */
	FString GetUniqueProductionName() const;
	FString GetUniqueProductionName(const FString& BaseName) const;

	/** Checks if the level editor toolbar is enabled. */
	bool IsLevelEditorToolbarEnabled() const { return bEnableProductionsLevelEditorToolbar; }
	
	/** Returns the delegate that broadcasts when a production is added/removed */
	FOnProductionListChanged& OnProductionListChanged() { return ProductionListChangedDelegate; }

	/** Returns the delegate that broadcasts when the active production changes */
	FOnActiveProductionChanged& OnActiveProductionChanged() { return ActiveProductionChangedDelegate; }

	/** Given a production ID and data struct, tries to get a mutable view on the productions extended data of the matching type. */
	template <typename TStruct>
	TStructView<TStruct> GetMutableProductionExtendedData(FGuid ProductionID)
	{
		if (FStructView StructView = GetMutableProductionExtendedData(ProductionID, *TStruct::StaticStruct()); StructView.IsValid())
		{
			return TStructView<TStruct>(StructView.Get<TStruct>());
		}
		return TStructView<TStruct>();
	}

	/** Given a production ID and data struct, tries to get a mutable view on the productions extended data of the matching type. */
	UE_API FStructView GetMutableProductionExtendedData(FGuid ProductionID, const UScriptStruct& DataStruct);

	/** Given a production ID and data struct, tries to get a view on the productions extended data of the matching type. */
	template <typename TStruct>
	TConstStructView<TStruct> GetProductionExtendedData(FGuid ProductionID) const
	{
		if (FConstStructView StructView = GetProductionExtendedData(ProductionID, *TStruct::StaticStruct()); StructView.IsValid())
		{
			return TConstStructView<TStruct>(StructView.Get<const TStruct>());
		}
		return TConstStructView<TStruct>();
	}

	/** Given a production ID and data struct, tries to get a view on the productions extended data of the matching type. */
	UE_API FConstStructView GetProductionExtendedData(FGuid ProductionID, const UScriptStruct& DataStruct) const;

	/** Given a production ID and struct view, tries to set the struct in the productions extended data of the matching type. */
	UE_API bool SetProductionExtendedData(FGuid ProductionID, const FConstStructView& DataStructView);

	/** 
	 * Get a FScopedModifyProductionExtendedData instance for the production matching the given production ID. This will allow modifying
	 * multiple extended data of a configured Production, ensuring modifications are exported correctly, ready to be saved back to the config with the
	 * new updates.
	 * 
	 * e.g.
	 * 
	 * ```
	 * const FGuid TargetProductionId; // Set from elsewhere or use GetActiveProductionId()
	 * UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	 * FScopedModifyProductionExtendedData ModifyGuard = ProductionSettings->ModifyProductionExtendedData(TargetProductionId);
	 * 
	 * if (TStructView<MyStruct> DataView1 = ProductionSettings->GetMutableProductionExtendedData<MyStruct>(TargetProductionId); DataView1.IsValid())
	 * {
	 *		MyStruct& MyStructRef = DataView1.Get();
	 *		...
	 *		// Modify MyStructRef
	 *		...
	 * }
	 * 
	 * if (TStructView<MyOtherStruct> DataView2 = ProductionSettings->GetMutableProductionExtendedData<MyOtherStruct>(TargetProductionId); DataView2.IsValid())
	 * {
	 *		MyOtherStruct& MyOtherStructRef = DataView2.Get();
	 *		...
	 *		// Modify MyOtherStructRef
	 *		...
	 * }
	 * ```
	 * 
	 * For blueprints, use nodes GetProductionExtendedData (ProductionFunctionLibrary) and SetProductionExtendedData (ProductionFunctionLibrary).
	 * 
	 */
	[[nodiscard]] UE_API FScopedModifyProductionExtendedData ModifyProductionExtendedData(FGuid ProductionID);

	/** Get a FScopedModifyProductionExtendedData instance for the production matching the given production ID and extended data struct type. This will allow modifying
	 * the targeted extended data of a configured Production, ensuring modifications are exported correctly, ready to be saved back to the config with the
	 * new updates.
	 * 
	 * e.g.
	 * 
	 * ```
	 * const FGuid TargetProductionId; // Set from elsewhere or use GetActiveProductionId()
	 * UProductionSettings* ProductionSettings = GetMutableDefault<UProductionSettings>();
	 * if (TStructView<MyStruct> DataView = ProductionSettings->GetMutableProductionExtendedData<MyStruct>(TargetProductionId); DataView.IsValid())
	 * {
	 *		FScopedModifyProductionExtendedData ModifyGuard = ProductionSettings->ModifyProductionExtendedData(TargetProductionId);
	 *		MyStruct& MyStructRef = DataView.Get();
	 * 
	 *		...
	 *		// Modify MyStructRef
	 *		...
	 * }
	 * ```
	 * 
	 * For blueprints, use nodes GetProductionExtendedData (ProductionFunctionLibrary) and SetProductionExtendedData (ProductionFunctionLibrary).
	 * 
	 */
	[[nodiscard]] UE_API FScopedModifyProductionExtendedData ModifyProductionExtendedData(FGuid ProductionID, const UScriptStruct& ExtendedDataStruct);

	/** Get the checked member name for the Productions property. */
	UE_API static FName GetProductionsPropertyMemberName();

public:
	/** Blueprint-assignable delegate that broadcasts when a new production is created */
	UPROPERTY(BlueprintAssignable, Category = "Cinematic Production", meta = (DisplayName = "On Production Created"))
	FOnProductionCreated OnProductionCreatedDynamic;

	/** Blueprint-assignable delegate that broadcasts when a production's settings are modified */
	UPROPERTY(BlueprintAssignable, Category = "Cinematic Production", meta = (DisplayName = "On Production Settings Changed"))
	FOnProductionSettingsChanged OnProductionSettingsChangedDynamic;

	/** Blueprint-assignable delegate that broadcasts when the active production changes */
	UPROPERTY(BlueprintAssignable, Category = "Cinematic Production", meta = (DisplayName = "On Active Production Set"))
	FOnActiveProductionSet OnActiveProductionSetDynamic;

private:
	/** Returns true if the input string is a valid Production name */
	bool IsValidProductionName(const FString& ProductionName) const;

	/** Applies overrides to various project settings based on the active production settings */
	void ApplyProjectOverrides();

	/** Overrides the DefaultDisplayRate in the level sequence project settings based on the active production setting */
	void OverrideDefaultDisplayRate();

	/** Overrides the DefaultStartTime in the movie scene tools project settings based on the active production settings */
	void OverrideDefaultStartTime();

	/** Writes out the Hierarchical bias value (based on the Subsequence Priority setting) to EditorPerProjectUserSettings.ini */
	void OverrideSubsequenceHierarchicalBias();

	/** Overrides the DefaultAssetNames property of the asset tools project settings based on the active production setting */
	void OverrideDefaultAssetNames();

	/** Apply the active production's namespace deny list to the input set of namespace names */
	void FilterNamingTokenNamespaces(TSet<FString>& Namespaces);

	/** Set the active production name and write out the new value to EditorPerProjectUserSettings.ini */
	void SetActiveProductionName();

	/** Cache the default project settings that are overridden by the active production, used to reset when there is no active production */
	void CacheProjectDefaults();

	/** Perform any setup after the engine has finished initilization. */
	void OnPostEngineInit();

	/** Returns a pointer to the active production (if it exists) */
	const FCinematicProduction* GetActiveProductionPtr() const;

	/** Try to update the default config file (will attempt to make the file writable if needed) */
	void UpdateConfig();

	/** Update the config and broadcast OnProductionSettingsChangedDynamic for the given production */
	void NotifySettingsChange(const FCinematicProduction& Production);

	/** Reload all Production extended data into the current Productions. */
	void ReloadProductionsExtendedData();

	/** Binds to events on all productions. */
	void BindToProductions();

	/** Handler for when a Productions ExtendedData is updated. */
	void HandleProductionExtendedDataUpdated(const FCinematicProduction& SenderProduction, const UScriptStruct* ForData);

	/** Given a property changed chain event and a node representing the productions property, handle any property changes that have occured to the Productions property. */
	void HandleProductionsChangeChainProperty(const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode& ProductionsChainNode, FPropertyChangedChainEvent& WithinPropertyChangedEvent);

private:
	/** Name of the active production */
	UPROPERTY(VisibleAnywhere, Category = "Default", meta = (DisplayName = "Active Production"))
	FString ActiveProductionName;

	/** List of available productions in this project */
	UPROPERTY(config, EditAnywhere, Category = "Default")
	TArray<FCinematicProduction> Productions;

	/** Enable the production level editor toolbar. */
	UPROPERTY(config, EditAnywhere, Category = "Default", meta = (ConfigRestartRequired = true, DisplayName = "Enable Production Toolbar"))
	bool bEnableProductionsLevelEditorToolbar = true;

	/** ID of the active production (in the Productions array) */
	FGuid ActiveProductionID;

	/** Cached default project settings that are overridden by the active production, used to reset when there is no active production */
	FString ProjectDefaultDisplayRate;
	float ProjectDefaultStartTime;

	/** Asset classes for which Production Settings has currently registered a default name override with AssetTools. Tracked so we can unregister if the active production changes. */
	TSet<const UClass*> RegisteredAssetNameClasses;

	/** Original tooltip text for sequencer settings */
	FString OriginalDefaultDisplayRateTooltip;
	FString OriginalDefaultStartTimeTooltip;

	/** Delegate that broadcasts when a production is added/removed */
	FOnProductionListChanged ProductionListChangedDelegate;

	/** Delegate that broadcasts when the active production changes */
	FOnActiveProductionChanged ActiveProductionChangedDelegate;
};

#undef UE_API
