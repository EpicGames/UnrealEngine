// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/StructView.h"

struct FCinematicProduction;

/**
* Helper class for scoping modifications to a Productions extended data.
* On destruction of the instance, Finish is called which will call update on the TargetProduction if it has not been already.
* Finish can also be called at any time if desired, and will not then be called on destruction.
*
* Use of this class is to create a locally scoped instance before making any modifications, and allowing it to fall out of scope
* or manually calling Finish when you wish those updates to propogate.
*
* eg.
* ```
* {
*	if (FInstancedStruct* ExtendedData = Production.FindOrLoadExtendedData(*FMyStruct::StaticStruct()))
*	{
*		FScopedModifyProductionExtendedData ModifyGuard(Production);
*		...
*		// Modify ExtendedData
*		...
*	}
* }
* ```
*
* You can specify a specific ExtendedData UScriptStruct type to call update only for that type. If this is not provided at construction time,
* then all ExtendedData is updated on Finish.
*/
class CINEASSEMBLYTOOLSEDITOR_API FScopedModifyProductionExtendedData
{
public:
	/** Default constructor that constructs into an inactive state. */
	FScopedModifyProductionExtendedData() = default;

	/** Given a reference to a production, create a scoped modify instance. */
	FScopedModifyProductionExtendedData(FCinematicProduction& InProduction);

	/**
	* Given a reference to a production and a specific data struct type, create a scoped modify instance.
	*
	* If InTargetStruct is not found as an ExtendedData type within the Production, the scoped modify instance is not considered active.
	*/
	FScopedModifyProductionExtendedData(FCinematicProduction& InProduction, const UScriptStruct& InTargetStruct);

	/** Destructor that will call Finish. */
	virtual ~FScopedModifyProductionExtendedData();

	/**
	* Returns if this instance is active.
	* It is considered active if it has not yet finished and has a valid Production reference and target data struct if supplied.
	*/
	bool IsActive() const;

	/** Finish this scoped modify and update the extended data on the target production. */
	void Finish();

private:
	/** Original value of the block flag for the production. */
	bool bOriginalBlockValue = false;

	/** Start the modification and blocking of exported signals. */
	void Start();

	/** View onto the target production object that is being modified. */
	TStructView<FCinematicProduction> TargetProduction;

	/** Optional target struct type that is being updated as part of this modification. */
	TOptional<const UScriptStruct*> TargetStruct;

	/** Whether this instance has had Finish called on it or not. */
	bool bHasFinished = false;
};
